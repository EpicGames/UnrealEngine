// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalysisCache.h"

#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Async/MappedFileHandle.h"
#include "Containers/StringView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"
#include "Memory/SharedBuffer.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/BufferWriter.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryReader.h"

DEFINE_LOG_CATEGORY_STATIC(LogAnalysisCache, Log, All);

namespace TraceServices {
//////////////////////////////////////////////////////////////////////
FAnalysisCache::FFileContents::FFileContents(const TCHAR* FilePath)
	: CacheFilePath(FilePath)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	if (FParse::Param(FCommandLine::Get(), TEXT("disableanalysiscache")))
	{
		UE_LOGF(LogAnalysisCache, Display, "Putting cache in transient mode.");
		bTransientMode = true;
		return;
	}
	
	// Opening the file we can encounter one of 3 scenarios:
	// 1. File does not exist, create on first save
	// 2. File exist, we can read the contents
	// 3. File exist but we could not open the file for reading. Multiple processes are competing. Put the cache in transient mode
	if (const int64 FileSize = PlatformFile.FileSize(*CacheFilePath); FileSize > 0)
	{
		if (const TUniquePtr<IFileHandle> File(PlatformFile.OpenRead(*CacheFilePath)); File.IsValid())
		{
			if (const bool Result = Load(); !Result)
			{
				UE_LOGF(LogAnalysisCache, Error, "Failed to open cache file table of contents.");
				bTransientMode = true;
				//todo: Recover by deleting file?
			}

			// Additional sanity check. A common error scenario is that Insights crashed after writing block but before
			// committing them to the table of contents. Detect that scenario here.
			uint32 MinimalExpectedSizePerVersion[] = {0, ReservedSizeV1, ReservedSizeV2};
			if (Version < UE_ARRAY_COUNT(MinimalExpectedSizePerVersion) && FileSize > MinimalExpectedSizePerVersion[Version] && Blocks.IsEmpty())
			{
				UE_LOGF(LogAnalysisCache, Error, "Cache file has written several blocks but table of contents contains no blocks. This is likely caused by abnormal program termination. Please delete \"%ls\". Putting cache in transient mode.", *CacheFilePath);
				IndexEntries.Empty();
				bTransientMode = true;
				return;
			}

			UE_LOGF(LogAnalysisCache, VeryVerbose, "Cache contains %d blocks:", Blocks.Num());
			UE_LOGF(LogAnalysisCache, VeryVerbose, "   %10ls   %10ls   %13ls   %13ls   %13ls", TEXT("Cache index"), TEXT("Block index"), TEXT("Offset"), TEXT("Uncompressed"), TEXT("Compressed"));

			for (const FFileContents::FBlockEntry& Block : Blocks)
			{
				UE_LOG(LogAnalysisCache, VeryVerbose, TEXT("   %10d   %10d   %10" UINT64_FMT " kb   %10" UINT64_FMT " kb   %10" UINT64_FMT " kb"), GetCacheId(Block.BlockKey), GetBlockIndex(Block.BlockKey), Block.Offset / 1024, Block.UncompressedSize / 1024, Block.CompressedSize / 1024);
			}

			//todo: At this point we can check the file size and if it doesn't match we can trim unknown segments.
		}
		else
		{
			// Unable to open for read. Most likely this is because another instance is using the file
			UE_LOGF(LogAnalysisCache, Warning,
			       "Unable to read the cache file %ls, possibly already open in another session. Putting cache in transient mode.",
			       *CacheFilePath);
			bTransientMode = true;
		}
	}

	// If we haven't established a version use current
	if (!Version)
	{
		Version = CurrentVersion;
	}
}

//////////////////////////////////////////////////////////////////////
FAnalysisCache::FFileContents::~FFileContents()
{
	// Save the table of contents.
	if (!Blocks.IsEmpty())
	{
		if (const bool Result = Save(); !Result )
		{
			UE_LOGF(LogAnalysisCache, Error, "Failed to update cache files table of contents.");
		}
	}
}

//////////////////////////////////////////////////////////////////////
FCacheId FAnalysisCache::FFileContents::GetId(const TCHAR* Name, uint16 Flags)
{
	const FIndexEntry* Entry = Algo::FindByPredicate(IndexEntries, [Name](const FIndexEntry& Entry){ return Entry.Name.Equals(Name); });
	if (Entry)
	{
		return Entry->Id;
	}

	// Name was not previously registered, create new entry
	const uint32 NewId = IndexEntries.Num() + 1;
	FIndexEntry& NewEntry = IndexEntries.AddZeroed_GetRef();
	NewEntry.Name = Name;
	NewEntry.Id = NewId;
	NewEntry.Flags = uint32(Flags);
	return NewId;
}

//////////////////////////////////////////////////////////////////////
uint16 FAnalysisCache::FFileContents::GetFlags(FCacheId InId)
{
	const FIndexEntry* Entry = Algo::FindBy(IndexEntries, InId, &FIndexEntry::Id);
	if (Entry)
	{
		return uint16(Entry->Flags & 0xffff);
	}
	return 0;
}
	
//////////////////////////////////////////////////////////////////////
FMutableMemoryView FAnalysisCache::FFileContents::GetUserData(FCacheId InId)
{
	FIndexEntry* Entry = Algo::FindBy(IndexEntries, InId, &FIndexEntry::Id);
	if (Entry)
	{
		return FMutableMemoryView(Entry->UserData, UserDataSize);
	}
	return FMutableMemoryView();
}

//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::Save()
{
	IFileHandle* File = GetFileHandleForWrite();
	if (!File)
	{
		return true;
	}

	File->Seek(0);

	check(Version > 0); // Version should always be set here
	switch(Version)
	{
	case 1:
		return SaveVersion1(File);
	case 2:
		return SaveVersion2(File);
	}

	return false;
}

//////////////////////////////////////////////////////////////////////
void FAnalysisCache::FFileContents::SaveVersion1Index(FCbWriter& Writer)
{
	Writer.BeginArray(ANSITEXTVIEW("Index"));
	for (auto Entry : IndexEntries)
	{
		Writer.BeginObject();
		Writer << ANSITEXTVIEW("N") << Entry.Name;
		Writer << ANSITEXTVIEW("I") << Entry.Id;
		Writer << ANSITEXTVIEW("F") << Entry.Flags;
		Writer.AddBinary(ANSITEXTVIEW("UD"), &Entry.UserData, UserDataSize);
		Writer.EndObject();
	}
	Writer.EndArray();
	Writer.BeginArray(ANSITEXTVIEW("Blocks"));
	for (const FBlockEntry& Entry : Blocks)
	{
		Writer.AddBinary(&Entry, sizeof(FBlockEntry));
	}
	Writer.EndArray();
	
}

//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::SaveVersion1(IFileHandle* File)
{
	FCbWriter Writer;
	Writer.BeginObject();
	Writer << "Version" << 1;
	SaveVersion1Index(Writer);
	Writer.EndObject();
	
	FCbPackage Package(Writer.Save().AsObject());

	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(ReservedSizeV1);
	FBufferWriter BufferWriter(Buffer.GetData(), Buffer.GetSize());
	Package.Save(BufferWriter);
	
	return File->Write((uint8*)Buffer.GetData(), Buffer.GetSize());
}

//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::SaveVersion2(IFileHandle* File)
{
	check (Version != 0);
	
	// Header
	ANSICHAR Magic[] = "UC";
	File->Write((uint8*)Magic,2);
	File->Write((uint8*)&Version, 4);
	checkSlow(File->Tell() == IndexOffset);

	// Write the index
	FCbWriter Writer;
	Writer.BeginObject();
	SaveVersion1Index(Writer);
	Writer.EndObject();
	
	FCbPackage Package(Writer.Save().AsObject());
	
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(ReservedSizeV2);
	FBufferWriter BufferWriter(Buffer.GetData(), Buffer.GetSize());
	Package.Save(BufferWriter);
	
	return File->Write((uint8*)Buffer.GetData(), Buffer.GetSize());
}

//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::Load()
{
	IFileHandle* File = GetFileHandleForRead();
	if (!File)
	{
		return true;
	}

	// Read header
	Version = ReadHeader(File);
	if (Version > CurrentVersion)
	{
		UE_LOGF(LogAnalysisCache, Warning, "Cache file of unknown version (%u), cannot load.", Version);
		return false;
	}

	UE_LOGF(LogAnalysisCache, Display, "Loading cache file (version %u).", Version);

	switch(Version)
	{
	case 1:
		return LoadVersion1(File);
	case 2:
		return LoadVersion2(File);
	}

	return false;
}

//////////////////////////////////////////////////////////////////////
uint32 FAnalysisCache::FFileContents::ReadHeader(IFileHandle* File)
{
	uint32 Version = 0;
	uint8 Header[IndexOffset];
	File->Seek(0);
	File->Read((uint8*)&Header, IndexOffset);
	if (Header[0] != 'U' || Header[1] != 'C')
	{
		// Version 1 lacked a magic header
		Version = 1;
		File->Seek(0);
	}
	else
	{
		// Expect version number after magic
		Version = *(uint32*)&Header[2];
	}

	return Version;
}

//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::WriteHeader(IFileHandle* File, uint32 Version)
{
	bool bSuccess = true;
	File->Seek(0);
	ANSICHAR Magic[] = "UC";
	bSuccess &= File->Write((uint8*)Magic,2);
	bSuccess &= File->Write((uint8*)&Version, 4);
	checkSlow(File->Tell() == IndexOffset);
	return bSuccess;
}

//////////////////////////////////////////////////////////////////////
void FAnalysisCache::FFileContents::LoadVersion1Index(const FCbPackage& Package)
{
	FCbArrayView IndexArray = Package.GetObject().Find(ANSITEXTVIEW("Index")).AsArrayView();
	IndexEntries.Reserve(static_cast<int32>(IndexArray.Num()));
	for (FCbFieldView IndexEntry : IndexArray)
	{
		FCbObjectView IndexEntryObj = IndexEntry.AsObjectView();
		FUtf8StringView NameView = IndexEntryObj[ANSITEXTVIEW("N")].AsString();
		const uint32 Id = IndexEntryObj[ANSITEXTVIEW("I")].AsUInt32();
		const uint32 Flags = IndexEntryObj[ANSITEXTVIEW("F")].AsUInt32();
		FMemoryView UserData = IndexEntryObj[ANSITEXTVIEW("UD")].AsBinaryView();
		FIndexEntry& Entry = IndexEntries.AddZeroed_GetRef();
		Entry.Name = FString(NameView);
		Entry.Id = Id;
		Entry.Flags = Flags;
		FMutableMemoryView UserDataDst(&Entry.UserData, UserDataSize);
		FMutableMemoryView Remainder = UserDataDst.CopyFrom(UserData);
		check(Remainder.GetSize() == 0);
	}

	FCbArrayView BlockArray = Package.GetObject().Find(ANSITEXTVIEW("Blocks")).AsArrayView();
	Blocks.Reserve(static_cast<int32>(BlockArray.Num()));
	
	for (FCbFieldView BlockEntryView : BlockArray)
	{
		FBlockEntry& Block = Blocks.AddZeroed_GetRef();
		FMutableMemoryView BlockView = FMutableMemoryView(&Block, sizeof(FBlockEntry));
		BlockView.CopyFrom(BlockEntryView.AsBinaryView());
	}
}

//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::LoadVersion1(IFileHandle* File)
{
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(ReservedSizeV1);
	File->Read((uint8*)Buffer.GetData(), Buffer.GetSize());
	
	FMemoryReaderView Ar(MakeArrayView<uint8>((uint8*)Buffer.GetData(), IntCastChecked<int32>(Buffer.GetSize())));
	
	FCbPackage Package;
	if (!Package.TryLoad(Ar))
	{
		return false;
	}
	
	Version = Package.GetObject().Find("Version").AsUInt32();
	
	LoadVersion1Index(Package);

	return true;
}

	
//////////////////////////////////////////////////////////////////////
bool FAnalysisCache::FFileContents::LoadVersion2(IFileHandle* File)
{
	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(ReservedSizeV2);
	File->Read((uint8*)Buffer.GetData(), Buffer.GetSize());

	FMemoryReaderView Ar(MakeArrayView<uint8>((uint8*)Buffer.GetData(), IntCastChecked<int32>(Buffer.GetSize())));

	FCbPackage Package;
	if (!Package.TryLoad(Ar))
	{
		return false;
	}

	LoadVersion1Index(Package);

	return true;
}
	
//////////////////////////////////////////////////////////////////////
uint64 FAnalysisCache::FFileContents::UpdateBlock(FMemoryView Block, BlockKeyType BlockKey)
{
	IFileHandle* File = GetFileHandleForWrite();
	if (!File)
	{
		return 0;
	}

	const int32 EntryIndex = Algo::BinarySearchBy(Blocks, BlockKey, [&](const FBlockEntry& InEntry) { return InEntry.BlockKey; });
	const FIoHash CurrentHash = FIoHash::HashBuffer(Block);
	
	if (EntryIndex != INDEX_NONE)
	{
		FBlockEntry& Entry = Blocks[EntryIndex];
		// todo: This only works if size hasn't changed!
		if (CurrentHash != Entry.Hash)
		{
			//todo: Add compression
			File->Seek(Entry.Offset);
			if(!File->Write((const uint8*)Block.GetData(), Block.GetSize()))
			{
				UE_LOG(LogAnalysisCache, Error, TEXT("Failed to update block 0x%x at offset %" UINT64_FMT " kb"), BlockKey, Entry.Offset / 1024);
				return 0;
			}

			Entry.Hash = CurrentHash;
			return Block.GetSize();
		}
	}
	else
	{
		// Write to end of file and add to blocks array
		const bool bSeekResult = File->SeekFromEnd(0);
		check(bSeekResult);
		const uint64 Offset = File->Tell();
		checkf(Offset >= ReservedSizeV2, TEXT("Offset (%" UINT64_FMT ") is larger than reserved size (%u)"), Offset, ReservedSizeV2);
		if(!File->Write((const uint8*) Block.GetData(), Block.GetSize()))
		{
			UE_LOG(LogAnalysisCache, Error, TEXT("Failed to update block 0x%x at offset %" UINT64_FMT " kb"), BlockKey, Offset / 1024);
			return 0;
		}

		// Insert entry in blocks list and sort array
		Blocks.Emplace(FBlockEntry{BlockKey, 0, Offset, 0, Block.GetSize(), CurrentHash});
		Algo::SortBy(Blocks, [](const FBlockEntry& Entry){ return Entry.BlockKey; });

		return Block.GetSize();
	}

	return 0;
}

//////////////////////////////////////////////////////////////////////
uint64 FAnalysisCache::FFileContents::LoadBlock(FMutableMemoryView Block, BlockKeyType BlockKey)
{
	IFileHandle* File = GetFileHandleForRead();
	if (!File)
	{
		return 0;
	}
	
	const int32 EntryIndex = Algo::BinarySearchBy(Blocks, BlockKey, [&](const FBlockEntry& InEntry) { return InEntry.BlockKey; });
	if (EntryIndex == INDEX_NONE)
	{
		UE_LOGF(LogAnalysisCache, Error, "Trying to load unknown block 0x%x.", BlockKey);
		return 0;
	}

	const FBlockEntry& Entry = Blocks[EntryIndex];
	check(Entry.UncompressedSize <= Block.GetSize());
	
	if (!File->Seek(Entry.Offset))
	{
		UE_LOG(LogAnalysisCache, Error, TEXT("Block 0x%x was located on an invalid offset %" UINT64_FMT " kb."), BlockKey, Entry.Offset / 1024);
		return 0;
	}

	//todo: Add compression
	if (!File->Read((uint8*)Block.GetData(), Entry.UncompressedSize))
	{
		UE_LOG(LogAnalysisCache, Error, TEXT("Unable to read block 0x%x on offset %" UINT64_FMT " kb with size %" UINT64_FMT " kb."), BlockKey, Entry.Offset / 1024, Entry.UncompressedSize / 1024);
		return 0;
	}

	return Block.GetSize();
}

//////////////////////////////////////////////////////////////////////
IFileHandle* FAnalysisCache::FFileContents::GetFileHandleForWrite()
{
	if (bTransientMode)
	{
		return nullptr;
	}
	
	if (CacheFileWrite.IsValid())
	{
		return CacheFileWrite.Get();
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const bool bCreated = !PlatformFile.FileExists(*CacheFilePath);
	
	CacheFileWrite = TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*CacheFilePath, true, true));
	if (!CacheFileWrite.IsValid())
	{
		// Unable to open for write. Most likely this is because another instance is using the file
		UE_LOGF(LogAnalysisCache, Warning,
			   "Unable to write to the cache file %ls, possibly already open in another session. Putting cache in transient mode.",
			   *CacheFilePath);
		bTransientMode = true;
		return nullptr;
	}

	if (bCreated)
	{
		// Save the table of contents to reserve space
		Save();
	}
	
	return CacheFileWrite.Get();
}
	
//////////////////////////////////////////////////////////////////////
IFileHandle* FAnalysisCache::FFileContents::GetFileHandleForRead()
{
	if (bTransientMode)
	{
		return nullptr;
	}
	
	if (CacheFile.IsValid())
	{
		return CacheFile.Get();
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	CacheFile = TUniquePtr<IFileHandle>(PlatformFile.OpenRead(*CacheFilePath, true));

	if (!CacheFile.IsValid())
	{
		// Unable to open for read. Most likely this is because another instance is using the file
		UE_LOGF(LogAnalysisCache, Warning,
			   "Unable to read the cache file %ls, possibly already open in another session. Putting cache in transient mode.",
			   *CacheFilePath);
		bTransientMode = true;
		return nullptr;
	}
	
	return CacheFile.Get();
}
	
//////////////////////////////////////////////////////////////////////
FAnalysisCache::FAnalysisCache(const TCHAR* Path)
	: Stats()
{
	// Find the cache file path.
	// todo: This will need to be refined as we move away from files

	// We expect to receive the full path to the session file
	FString CacheFilePath(Path);
	CacheFilePath = FPaths::SetExtension(CacheFilePath, TEXT(".ucache"));
	Contents = MakeUnique<FFileContents>(*CacheFilePath);

	// Build a dictionary of number of blocks per cache id.
	for (const FFileContents::FBlockEntry& Block : Contents->Blocks)
	{
		const uint32 CacheId = GetCacheId(Block.BlockKey);
		IndexBlockCount.FindOrAdd(CacheId)++;
	}
}

/////////////////////////////////////////////////////////////////////
FAnalysisCache::~FAnalysisCache()
{
	// Remove all references to cached block, forcing them to write.
	CachedBlocks.Empty();
	// Delete file contents wrapper
	Contents.Reset();
	
	UE_LOGF(LogAnalysisCache, Display, "Closing analysis cache, %0.2f MiB read, %0.2f MiB written.",
	       double(Stats.BytesRead) / (1024.0 * 1024.0), double(Stats.BytesWritten) / (1024.0 * 1024.0));
}

/////////////////////////////////////////////////////////////////////
uint32 FAnalysisCache::GetCacheId(const TCHAR* Name, uint16 Flags)
{
	return Contents->GetId(Name, Flags);
}

/////////////////////////////////////////////////////////////////////
FMutableMemoryView FAnalysisCache::GetUserData(FCacheId CacheId)
{
	return Contents->GetUserData(CacheId);
}

/////////////////////////////////////////////////////////////////////
FSharedBuffer FAnalysisCache::CreateBlocks(FCacheId CacheId, uint32 BlockCount)
{
	const uint32 BlockIndex = IndexBlockCount.FindOrAdd(CacheId);
	const BlockKeyType BlockKey = CreateBlockKey(CacheId, BlockIndex);

	// Allocate memory and make the shared buffer with freeing callback.
	const uint64 TotalBytes = IAnalysisCache::BlockSizeBytes * BlockCount;
	void* Block = FMemory::Malloc(TotalBytes, BlockAlignment);
	FMemory::Memzero(Block, TotalBytes);
	FSharedBuffer Blocks = FSharedBuffer::TakeOwnership(Block, TotalBytes, [this, CacheId, BlockIndex](void* InBlock, uint64 Size)
	{
		ReleaseBlocks((uint8*)InBlock, CacheId, BlockIndex, Size);
	});

	// Increment the block count
	IndexBlockCount[CacheId] += BlockCount;

	// Add the blocks into our internal caching mechanism
	if (!(Contents->GetFlags(CacheId) & ECacheFlags_NoGlobalCaching))
	{
		check(!CachedBlocks.Contains(BlockKey));
		CachedBlocks.Add(BlockKey, FSharedBuffer::MakeView(Blocks.GetView(), Blocks));
	}

	return Blocks;
}

/////////////////////////////////////////////////////////////////////
FSharedBuffer FAnalysisCache::GetBlocks(FCacheId CacheId, uint32 BlockIndexStart, uint32 BlockCount)
{
	const BlockKeyType CacheBlockKey = CreateBlockKey(CacheId, BlockIndexStart);

	const uint32 ExistingBlockCount = IndexBlockCount.FindOrAdd(CacheId);
	if (BlockIndexStart >= ExistingBlockCount || (BlockIndexStart + BlockCount) > ExistingBlockCount)
	{
		UE_LOGF(LogAnalysisCache, Error, "Block range %u to %u is invalid for cache id %u.", BlockIndexStart,
		       BlockIndexStart + BlockCount, CacheId);
		return FSharedBuffer();
	}
	
	// Look in our currently help block cache
	// todo: Cached blocks are only keyed on first block index. What if a different range is requested? Overlap.
	if (FSharedBuffer* Block = CachedBlocks.Find(CacheBlockKey))
	{
		return *Block;
	}

	// Allocate a contiguous chunk of memory that fits all the blocks
	const uint64 TotalBytes = IAnalysisCache::BlockSizeBytes * BlockCount;
	uint8* BlockBuffer = (uint8*)FMemory::Malloc(TotalBytes, BlockAlignment);

	for (uint32 Block = 0; Block < BlockCount; ++Block)
	{
		const BlockKeyType BlockKey = CreateBlockKey(CacheId, BlockIndexStart + Block);
		const FMutableMemoryView BlockView(BlockBuffer + (Block*IAnalysisCache::BlockSizeBytes), IAnalysisCache::BlockSizeBytes);
		const uint64 BytesRead = Contents->LoadBlock(BlockView, BlockKey);
		Stats.BytesRead += BytesRead;
	}
	
	// Take ownership of memory and register freeing callback.
	FSharedBuffer Blocks = FSharedBuffer::TakeOwnership(BlockBuffer, TotalBytes, [this, CacheId, BlockIndexStart](void* Block, uint64 Size)
	{
		ReleaseBlocks((uint8*)Block, CacheId, BlockIndexStart, Size);
	});

	// Add the blocks into our internal caching mechanism
	if (!(Contents->GetFlags(CacheId) & ECacheFlags_NoGlobalCaching))
	{
		CachedBlocks.Add(CacheBlockKey, Blocks);
	}

	return Blocks;
}

/////////////////////////////////////////////////////////////////////
void FAnalysisCache::ReleaseBlocks(uint8* BlockBuffer, FCacheId CacheId, uint32 BlockIndexStart, uint64 Size)
{
	const uint32 BlockCount = IntCastChecked<uint32>(Size / IAnalysisCache::BlockSizeBytes);
	for (uint32 Block = 0; Block < BlockCount; ++Block)
	{
		const void* BlockStart = BlockBuffer + (Block * IAnalysisCache::BlockSizeBytes);
		const FMemoryView BlockView = FMemoryView(BlockStart, IAnalysisCache::BlockSizeBytes);
		const BlockKeyType BlockKey = CreateBlockKey(CacheId, BlockIndexStart + Block);
		const uint64 BytesWritten = Contents->UpdateBlock(BlockView, BlockKey);
		Stats.BytesWritten += BytesWritten;
	}
}

/////////////////////////////////////////////////////////////////////
} //namespace TraceServices
