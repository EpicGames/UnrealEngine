// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "IO/IoHash.h"
#include "Memory/SharedBuffer.h"
#include "Templates/UniquePtr.h"
#include "Containers/Map.h"
#include "TraceServices/Model/AnalysisCache.h"

class IMappedFileRegion;
class IMappedFileHandle;
class FCbPackage;

namespace TraceServices {

class IAnalysisSession;
	
//////////////////////////////////////////////////////////////////////
class FAnalysisCache : public IAnalysisCache
{
public:
	explicit FAnalysisCache(const TCHAR* Path);
	virtual ~FAnalysisCache() override;

	virtual FCacheId GetCacheId(const TCHAR* Name, uint16 Flags) override;
	virtual FMutableMemoryView GetUserData(FCacheId CacheId) override;
	virtual FSharedBuffer CreateBlocks(FCacheId CacheId, uint32 Count) override;
	virtual FSharedBuffer GetBlocks(FCacheId CacheId, uint32 BlockIndexStart, uint32 BlockCount = 1) override;
	
private:
	/**
	 * Compound key identifying a block.
	 * Warning: Changing this type also affects file contents.
	 */
	typedef uint32 BlockKeyType;

	/**
	 * Wrapper for file contents
	 */
	struct FFileContents
	{
		// Current file format version
		static constexpr uint32 CurrentVersion = 2;
		// Number of bytes at the start of the file reserved for table of contents and header.
		static constexpr uint32 ReservedSizeV1 = 512 * 1024;
		static constexpr uint32 ReservedSizeV2 = 1024 * 1024;
		// Offset of the table of contents in bytes
		static constexpr uint32 IndexOffset = 6;
		// Size for the index object 
		static constexpr uint32 ReservedSizeIndex = ReservedSizeV2 - IndexOffset;
		// Number of bytes allocated for user data (per named entry)
		static constexpr uint32 UserDataSize = 64;

		explicit FFileContents(const TCHAR* FilePath);
		~FFileContents();
		FCacheId GetId(const TCHAR* Name, uint16 Flags);
		FMutableMemoryView GetUserData(FCacheId Id);
		uint16 GetFlags(FCacheId Id);
		bool Save();
		bool Load();
		uint64 UpdateBlock(FMemoryView Block, BlockKeyType BlockKey);
		uint64 LoadBlock(FMutableMemoryView Block, BlockKeyType BlockKey);
		IFileHandle* GetFileHandleForWrite();
		IFileHandle* GetFileHandleForRead();

		struct FIndexEntry
		{
			FString Name;
			uint32 Id;
			uint32 Flags;
			uint8 UserData[UserDataSize];
		};
		
		struct FBlockEntry
		{
			BlockKeyType BlockKey;
			uint32 Flags;
			uint64 Offset;
			uint64 CompressedSize;
			uint64 UncompressedSize;
			FIoHash Hash;
		};
		
		FString CacheFilePath;
		TUniquePtr<IFileHandle> CacheFile;
		TUniquePtr<IFileHandle> CacheFileWrite;
		TArray<FIndexEntry> IndexEntries;
		TArray<FBlockEntry> Blocks;
		// When this is set blocks or table of contents are never written.
		bool bTransientMode = false;
		uint32 Version = 0;

	private:
		static uint32 ReadHeader(IFileHandle* File);
		static bool WriteHeader(IFileHandle* File, uint32 Version);
		void LoadVersion1Index(const FCbPackage& Package);
		void SaveVersion1Index(FCbWriter& Writer);
		bool SaveVersion1(IFileHandle* File);
		bool SaveVersion2(IFileHandle* File);
		bool LoadVersion1(IFileHandle* File);
		bool LoadVersion2(IFileHandle* File);
	};

	/**
	 * Life time statistics
	 */
	struct FStats
	{
		uint64 BytesRead;
		uint64 BytesWritten;
	};
	
	/**
	 * Called when a shared buffer is released. Checks if content has change and updates cached block correspondingly
	 * and finally frees memory.
	 * @param BlockBuffer Allocated memory
	 * @param CacheId Unique cache id
	 * @param BlockIndexStart Block index for this cache entry
	 * @param Size of buffer
	 */
	void ReleaseBlocks(uint8* BlockBuffer, FCacheId CacheId, uint32 BlockIndexStart, uint64 Size);
	
	/**
	 * Creates a block key identifier
	 * @param CacheId Unique cache id
	 * @param BlockIndex Block index for this cache entry
	 * @return Unique key identifying a single block
	 */
	static BlockKeyType CreateBlockKey(FCacheId CacheId, uint32 BlockIndex)
	{
		// Use 3 top nibbles for cache id and the rest for block index. That gives a space for 4095 unique cache entries
		// each containing a maximum of 7 Gb of data.
		check(CacheId < 0xfff && BlockIndex < 0x000fffff && CacheId > 0);
		return CacheId << 20 | BlockIndex;
	}

	/**
	 * Extract the cache id from a block key
	 * @param BlockKey Unique key identifying a single block
	 * @return Unique cache id
	 */
	static uint32 GetCacheId(BlockKeyType BlockKey)
	{
		return (BlockKey & 0xfff00000) >> 20;
	}

	/**
	 * Extract the block index from a block key
	 * @param BlockKey Unique key identifying a single block
	 * @return Block index for this cache entry
	 */
	static uint32 GetBlockIndex(BlockKeyType BlockKey)
	{
		return BlockKey & 0x000fffff;
	}

	FAnalysisCache() = delete;
	FAnalysisCache(FAnalysisCache& Other) = delete;

	constexpr static uint32 BlockAlignment = 1024*4;
	
	TUniquePtr<FFileContents> Contents;
	FStats Stats;
	TMap<BlockKeyType,FSharedBuffer> CachedBlocks;
	TMap<uint32,uint32> IndexBlockCount;
};

}
