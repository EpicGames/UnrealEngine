// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/AnalysisService.h"
#include "AnalysisServicePrivate.h"

#include "HAL/PlatformFile.h"

// TraceAnalysis
#include "Trace/Analysis.h"
#include "Trace/DataStream.h"

// TraceServices
#include "Analyzers/BookmarksTraceAnalysis.h"
#include "Analyzers/LogTraceAnalysis.h"
#include "Analyzers/MiscTraceAnalysis.h"
#include "Analyzers/StringsAnalyzer.h"
#include "Model/BookmarksPrivate.h"
#include "Model/Channel.h"
#include "Model/CountersPrivate.h"
#include "Model/DefinitionProvider.h"
#include "Model/FramesPrivate.h"
#include "Model/LogPrivate.h"
#include "Model/MemoryPrivate.h"
#include "Model/NetProfilerProvider.h"
#include "Model/RegionsPrivate.h"
#include "Model/ScreenshotProviderPrivate.h"
#include "Model/ThreadsPrivate.h"
#include "ModuleServicePrivate.h"
#include "Trace/Analyzer.h"

namespace TraceServices
{

class FAnalyzerWrapper : public UE::Trace::IAnalyzer
{
public:
	FAnalyzerWrapper(TSharedRef<UE::Trace::IAnalyzer> InAnalyzer)
		: InnerAnalyzer(InAnalyzer)
	{
	}

	virtual ~FAnalyzerWrapper()
	{
	}

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
	{
		InnerAnalyzer->OnAnalysisBegin(Context);
	}

	virtual void OnAnalysisEnd(/*const FOnAnalysisEndContext& Context*/) override
	{
		InnerAnalyzer->OnAnalysisEnd(/*Context*/);
	}

	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
	{
		return InnerAnalyzer->OnEvent(RouteId, Style, Context);
	}

private:
	TSharedRef<UE::Trace::IAnalyzer> InnerAnalyzer;
};

// if IProvider ever gets member data, it will duplicate state because of "the diamond problem" with multiple inheritance
// if you hit this you should consider the implications for classes that implement multiple providers
static_assert(sizeof(IProvider) == sizeof(uintptr_t));

thread_local FAnalysisSessionLock* GThreadCurrentSessionLock;
thread_local int32 GThreadCurrentReadLockCount;
thread_local int32 GThreadCurrentWriteLockCount;

void FAnalysisSessionLock::ReadAccessCheck() const
{
	checkf(GThreadCurrentSessionLock == this && (GThreadCurrentReadLockCount > 0 || GThreadCurrentWriteLockCount > 0) , TEXT("Trying to read from session outside of a ReadScope"));
}

void FAnalysisSessionLock::WriteAccessCheck() const
{
	checkf(GThreadCurrentSessionLock == this && GThreadCurrentWriteLockCount > 0, TEXT("Trying to write to session outside of an EditScope"));
}

void FAnalysisSessionLock::BeginRead()
{
	check(!GThreadCurrentSessionLock || GThreadCurrentSessionLock == this);
	checkf(GThreadCurrentWriteLockCount == 0, TEXT("Trying to lock for read while holding write access"));
	if (GThreadCurrentReadLockCount++ == 0)
	{
		GThreadCurrentSessionLock = this;
		RWLock.ReadLock();
	}
}

void FAnalysisSessionLock::EndRead()
{
	check(GThreadCurrentReadLockCount > 0);
	if (--GThreadCurrentReadLockCount == 0)
	{
		RWLock.ReadUnlock();
		GThreadCurrentSessionLock = nullptr;
	}
}

void FAnalysisSessionLock::BeginEdit()
{
	check(!GThreadCurrentSessionLock || GThreadCurrentSessionLock == this);
	checkf(GThreadCurrentReadLockCount == 0, TEXT("Trying to lock for edit while holding read access"));
	if (GThreadCurrentWriteLockCount++ == 0)
	{
		GThreadCurrentSessionLock = this;
		RWLock.WriteLock();
	}
}

void FAnalysisSessionLock::EndEdit()
{
	check(GThreadCurrentWriteLockCount > 0);
	if (--GThreadCurrentWriteLockCount == 0)
	{
		RWLock.WriteUnlock();
		GThreadCurrentSessionLock = nullptr;
	}
}

FAnalysisSession::FAnalysisSession(uint32 InTraceId, const TCHAR* SessionName, TUniquePtr<UE::Trace::IInDataStream>&& InDataStream)
	: Name(SessionName)
	, TraceId(InTraceId)
	, DurationSeconds(0.0)
	, Allocator(32 << 20)
	, StringStore(Allocator)
	, Cache(*Name)
	, DataStream(MoveTemp(InDataStream))
{
}

FAnalysisSession::~FAnalysisSession()
{
	for (int32 AnalyzerIndex = Analyzers.Num() - 1; AnalyzerIndex >= 0; --AnalyzerIndex)
	{
		delete Analyzers[AnalyzerIndex];
	}
}

void FAnalysisSession::Start()
{
	UE::Trace::FAnalysisContext Context;
	for (UE::Trace::IAnalyzer* Analyzer : ReadAnalyzers())
	{
		Context.AddAnalyzer(*Analyzer);
	}
	Context.SetMessageDelegate(UE::Trace::FMessageDelegate::CreateRaw(this, &FAnalysisSession::OnAnalysisMessage));
	Context.SetOnCompletedDelegate(UE::Trace::FOnCompletedDelegate::CreateRaw(this, &FAnalysisSession::OnAnalysisComplete));
	Processor = Context.Process(*DataStream);
}

void FAnalysisSession::Stop(bool bAndWait) const
{
	if (!bDataStreamClosed.exchange(true))
	{
		DataStream->Close();
	}
	Processor.Stop();
	if (bAndWait)
	{
		Wait();
	}
}

void FAnalysisSession::Wait() const
{
	Processor.Wait();
}

void FAnalysisSession::OnAnalysisComplete()
{
	if (!bDataStreamClosed.exchange(true))
	{
		DataStream->Close();
	}
}

void FAnalysisSession::EnumerateMetadata(TFunctionRef<void(const FTraceSessionMetadata& Metadata)> Callback) const
{
	Lock.ReadAccessCheck();
	for (const auto& KV : Metadata)
	{
		Callback(KV.Value);
	}
}

void FAnalysisSession::AddMetadata(FName InName, int64 InValue)
{
	Lock.WriteAccessCheck();
	FTraceSessionMetadata& Value = Metadata.Add(InName);
	Value.Name = InName;
	Value.Type = FTraceSessionMetadata::EType::Int64;
	Value.Int64Value = InValue;
}

void FAnalysisSession::AddMetadata(FName InName, double InValue)
{
	Lock.WriteAccessCheck();
	FTraceSessionMetadata& Value = Metadata.Add(InName);
	Value.Name = InName;
	Value.Type = FTraceSessionMetadata::EType::Double;
	Value.DoubleValue = InValue;
}

void FAnalysisSession::AddMetadata(FName InName, FString InValue)
{
	Lock.WriteAccessCheck();
	FTraceSessionMetadata& Value = Metadata.Add(InName);
	Value.Name = InName;
	Value.Type = FTraceSessionMetadata::EType::String;
	Value.StringValue = InValue;
}

uint32 FAnalysisSession::GetNumPendingMessages() const
{
	return PendingMessagesCount.load();
}

TArray<FAnalysisMessage> FAnalysisSession::DrainPendingMessages()
{
	Lock.WriteAccessCheck();
	PendingMessagesCount.store(0);
	return MoveTemp(PendingMessages);
}

void FAnalysisSession::AddAnalyzer(UE::Trace::IAnalyzer* Analyzer)
{
	check(Analyzer != nullptr);
	Analyzers.Add(Analyzer);
}

void FAnalysisSession::AddAnalyzer(TSharedRef<UE::Trace::IAnalyzer> Analyzer)
{
	Analyzers.Add(new FAnalyzerWrapper(Analyzer));
}

void FAnalysisSession::AddProvider(const FName& InName, TSharedPtr<IProvider> Provider, TSharedPtr<IEditableProvider> EditableProvider)
{
	Providers.Add(InName, MakeTuple(Provider, EditableProvider));
}

const IProvider* FAnalysisSession::ReadProviderPrivate(const FName& InName) const
{
	const auto* FindIt = Providers.Find(InName);
	if (FindIt)
	{
		return FindIt->Key.Get();
	}
	else
	{
		return nullptr;
	}
}

IEditableProvider* FAnalysisSession::EditProviderPrivate(const FName& InName)
{
	const auto* FindIt = Providers.Find(InName);
	if (FindIt)
	{
		return FindIt->Value.Get();
	}
	else
	{
		return nullptr;
	}
}

void FAnalysisSession::OnAnalysisMessage(UE::Trace::EAnalysisMessageSeverity InSeverity, FStringView InMessage)
{
	EMessageSeverity::Type Severity = EMessageSeverity::Type::Info;
	switch(InSeverity)
	{
	case UE::Trace::EAnalysisMessageSeverity::Error: Severity = EMessageSeverity::Type::Error; break;
	case UE::Trace::EAnalysisMessageSeverity::Warning: Severity = EMessageSeverity::Type::Warning; break;
	case UE::Trace::EAnalysisMessageSeverity::Info: Severity = EMessageSeverity::Type::Info; break;
	}

	Lock.BeginEdit();
	PendingMessages.Push(FAnalysisMessage { Severity, FString(InMessage)});
	PendingMessagesCount.fetch_add(1);
	Lock.EndEdit();
}

FAnalysisService::FAnalysisService(FModuleService& InModuleService)
	: ModuleService(InModuleService)
{
}

FAnalysisService::~FAnalysisService()
{
}

TSharedPtr<const IAnalysisSession> FAnalysisService::Analyze(const TCHAR* SessionUri)
{
	TSharedPtr<const IAnalysisSession> AnalysisSession = StartAnalysis(SessionUri);
	AnalysisSession->Wait();
	return AnalysisSession;
}

TSharedPtr<const IAnalysisSession> FAnalysisService::StartAnalysis(const TCHAR* SessionUri)
{
	UE::Trace::FFileDataStream* FileStream = new UE::Trace::FFileDataStream();
	if (!FileStream->Open(SessionUri, /*bAllowWrite=*/true))
	{
		delete FileStream;
		return nullptr;
	}

	TUniquePtr<UE::Trace::IInDataStream> DataStream(FileStream);
	return StartAnalysis(~0, SessionUri, MoveTemp(DataStream));
}

TSharedPtr<const IAnalysisSession> FAnalysisService::StartAnalysis(uint32 TraceId, const TCHAR* SessionName, TUniquePtr<UE::Trace::IInDataStream>&& DataStream)
{
	TSharedRef<FAnalysisSession> Session = MakeShared<FAnalysisSession>(TraceId, SessionName, MoveTemp(DataStream));

	FAnalysisSessionEditScope _(*Session);

	TSharedPtr<FBookmarkProvider> BookmarkProvider = MakeShared<FBookmarkProvider>(*Session);
	Session->AddProvider(GetBookmarkProviderName(), BookmarkProvider, BookmarkProvider);

	TSharedPtr<FRegionProvider> RegionProvider = MakeShared<FRegionProvider>(*Session);
	Session->AddProvider(GetRegionProviderName(), RegionProvider, RegionProvider);

	TSharedPtr<FLogProvider> LogProvider = MakeShared<FLogProvider>(*Session);
	Session->AddProvider(GetLogProviderName(), LogProvider, LogProvider);

	TSharedPtr<FThreadProvider> ThreadProvider = MakeShared<FThreadProvider>(*Session);
	Session->AddProvider(GetThreadProviderName(), ThreadProvider, ThreadProvider);

	TSharedPtr<FFrameProvider> FrameProvider = MakeShared<FFrameProvider>(*Session);
	Session->AddProvider(GetFrameProviderName(), FrameProvider);

	TSharedPtr<FCounterProvider> CounterProvider = MakeShared<FCounterProvider>(*Session, *FrameProvider);
	Session->AddProvider(GetCounterProviderName(), CounterProvider, CounterProvider);

	TSharedPtr<FChannelProvider> ChannelProvider = MakeShared<FChannelProvider>();
	Session->AddProvider(GetChannelProviderName(), ChannelProvider);

	TSharedPtr<FScreenshotProvider> ScreenshotProvider = MakeShared<FScreenshotProvider>(*Session);
	Session->AddProvider(GetScreenshotProviderName(), ScreenshotProvider);

	TSharedPtr<FDefinitionProvider> DefProvider = MakeShared<FDefinitionProvider>(&Session.Get());
	Session->AddProvider(GetDefinitionProviderName(), DefProvider, DefProvider);

	Session->AddAnalyzer(new FMiscTraceAnalyzer(*Session, *ThreadProvider, *LogProvider, *FrameProvider, *ChannelProvider, *ScreenshotProvider, *RegionProvider));
	Session->AddAnalyzer(new FBookmarksAnalyzer(*Session, *BookmarkProvider, LogProvider.Get()));
	Session->AddAnalyzer(new FLogTraceAnalyzer(*Session, *LogProvider));
	Session->AddAnalyzer(new FStringsAnalyzer(*Session));

	ModuleService.OnAnalysisBegin(*Session);

	Session->Start();
	return Session;
}

} // namespace TraceServices
