// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalysisCache.h"
#include "Containers/HashTable.h"
#include "Common/StringStore.h"
#include "TraceServices/Containers/SlabAllocator.h"

// TraceAnalysis
#include "Trace/Analysis.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/AnalysisService.h"

namespace TraceServices
{

class FModuleService;

class FAnalysisSessionLock
{
public:
	void ReadAccessCheck() const;
	void WriteAccessCheck() const;

	void BeginRead();
	void EndRead();

	void BeginEdit();
	void EndEdit();

private:
	FRWLock RWLock;
};

class FAnalysisSession
	: public IAnalysisSession
{
public:
	FAnalysisSession(uint32 TraceId, const TCHAR* SessionName, TUniquePtr<UE::Trace::IInDataStream>&& InDataStream);
	virtual ~FAnalysisSession();

	void Start();
	virtual void Stop(bool bAndWait) const override;
	virtual void Wait() const override;
	virtual bool IsAnalysisComplete() const override { return !Processor.IsActive(); }

	virtual const TCHAR* GetName() const override { return *Name; }
	virtual uint32 GetTraceId() const override { return TraceId; }

	virtual double GetDurationSeconds() const override { Lock.ReadAccessCheck(); return DurationSeconds; }
	virtual void UpdateDurationSeconds(double Duration) override { Lock.WriteAccessCheck(); DurationSeconds = FMath::Max(Duration, DurationSeconds); }

	virtual double GetBaseDateTime() const override { return BaseDateTime; }
	virtual void SetBaseDateTime(double InBaseDateTime) override { BaseDateTime = InBaseDateTime; }
	
	virtual uint32 GetMetadataCount() const override { Lock.ReadAccessCheck(); return Metadata.Num(); }
	virtual void EnumerateMetadata(TFunctionRef<void(const FTraceSessionMetadata& Metadata)> Callback) const override;
	virtual void AddMetadata(FName InName, int64 InValue) override;
	virtual void AddMetadata(FName InName, double InValue) override;
	virtual void AddMetadata(FName InName, FString InValue) override;

	virtual uint32 GetNumPendingMessages() const override;
	virtual TArray<FAnalysisMessage> DrainPendingMessages() override;

	virtual ILinearAllocator& GetLinearAllocator() override { return Allocator; }
	virtual IStringStore& GetStringStore() override { return StringStore; }
	virtual const TCHAR* StoreString(const TCHAR* String) override { return StringStore.Store(String); }
	virtual const TCHAR* StoreString(const FStringView& String) override { return StringStore.Store(String); }

	virtual IAnalysisCache& GetCache() override { return Cache; }

	virtual void BeginRead() const override { Lock.BeginRead(); }
	virtual void EndRead() const override { Lock.EndRead(); }

	virtual void BeginEdit() override { Lock.BeginEdit(); }
	virtual void EndEdit() override { Lock.EndEdit(); }

	virtual void ReadAccessCheck() const override { return Lock.ReadAccessCheck(); }
	virtual void WriteAccessCheck() override { return Lock.WriteAccessCheck(); }

	virtual void AddAnalyzer(UE::Trace::IAnalyzer* Analyzer) override;
	virtual void AddAnalyzer(TSharedRef<UE::Trace::IAnalyzer> Analyzer) override;
	virtual void AddProvider(const FName& Name, TSharedPtr<IProvider> Provider, TSharedPtr<IEditableProvider> EditableProvider = nullptr) override;

	const TArray<UE::Trace::IAnalyzer*> ReadAnalyzers() { return Analyzers; }

private:
	virtual const IProvider* ReadProviderPrivate(const FName& Name) const override;
	virtual IEditableProvider* EditProviderPrivate(const FName& Name) override;

	void OnAnalysisMessage(UE::Trace::EAnalysisMessageSeverity Severity, FStringView Message);
	void OnAnalysisComplete();

private:
	mutable FAnalysisSessionLock Lock;
	FString Name;
	uint32 TraceId = 0;
	double DurationSeconds = 0.0;
	double BaseDateTime = 0.0;
	TMap<FName, FTraceSessionMetadata> Metadata;
	FSlabAllocator Allocator;
	FStringStore StringStore;
	FAnalysisCache Cache;
	TArray<UE::Trace::IAnalyzer*> Analyzers;
	TMap<FName, TTuple<TSharedPtr<IProvider>, TSharedPtr<IEditableProvider>>> Providers;
	TArray<FAnalysisMessage> PendingMessages;
	mutable std::atomic<uint32> PendingMessagesCount;
	mutable std::atomic<bool> bDataStreamClosed { false };
	mutable TUniquePtr<UE::Trace::IInDataStream> DataStream;
	mutable UE::Trace::FAnalysisProcessor Processor;
};

class FAnalysisService
	: public IAnalysisService
{
public:
	FAnalysisService(FModuleService& ModuleService);
	virtual ~FAnalysisService();
	virtual TSharedPtr<const IAnalysisSession> Analyze(const TCHAR* SessionUri) override;
	virtual TSharedPtr<const IAnalysisSession> StartAnalysis(const TCHAR* SessionUri) override;
	virtual TSharedPtr<const IAnalysisSession> StartAnalysis(uint32 TraceId, const TCHAR* SessionName, TUniquePtr<UE::Trace::IInDataStream>&& DataStream) override;

private:
	FModuleService& ModuleService;
};

} // namespace TraceServices
