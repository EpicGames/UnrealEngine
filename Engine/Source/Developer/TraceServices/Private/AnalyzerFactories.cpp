// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/AnalyzerFactories.h"
#include "Trace/DataStream.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/BookmarksTraceAnalysis.h"
#include "Analyzers/CountersTraceAnalysis.h"
#include "Analyzers/CpuProfilerTraceAnalysis.h"

TSharedPtr<TraceServices::IAnalysisSession> TraceServices::CreateAnalysisSession(uint32 InTraceId, const TCHAR* InSessionName, TUniquePtr<UE::Trace::IInDataStream>&& InDataStream)
{
	return MakeShared<FAnalysisSession>(InTraceId, InSessionName, MoveTemp(InDataStream));
}

TSharedPtr<UE::Trace::IAnalyzer> TraceServices::CreateBookmarksAnalyzer(IAnalysisSession& InSession, IEditableBookmarkProvider& InEditableBookmarkProvider)
{
	return MakeShared<FBookmarksAnalyzer>(InSession, InEditableBookmarkProvider, nullptr);
}

TSharedPtr<UE::Trace::IAnalyzer> TraceServices::CreateCountersAnalyzer(IAnalysisSession& InSession, IEditableCounterProvider& InEditableCounterProvider)
{
	return MakeShared<FCountersAnalyzer>(InSession, InEditableCounterProvider);
}

TSharedPtr<UE::Trace::IAnalyzer> TraceServices::CreateCpuProfilerAnalyzer(IAnalysisSession& InSession, IEditableTimingProfilerProvider& InEditableTimingProfilerProvider, IEditableThreadProvider& InEditableThreadProvider)
{
	return MakeShared<FCpuProfilerAnalyzer>(InSession, InEditableTimingProfilerProvider, InEditableThreadProvider);
}
