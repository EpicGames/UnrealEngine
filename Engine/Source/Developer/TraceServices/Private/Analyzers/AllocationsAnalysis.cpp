// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllocationsAnalysis.h"

#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MemoryTrace.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "Common/Utils.h"
#include "Model/AllocationsProvider.h"
#include "Model/MetadataProvider.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Callstack.h"

namespace TraceServices
{

#define INSIGHTS_MEM_TRACE_METADATA_TEST 0

namespace AllocationsAnalyzer::Private
{
	// version 1: UE 5.0
	// version 2: UE 5.2
	constexpr int32 MinSupportedVersion = 1;
	constexpr int32 MaxSupportedVersion = 2;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsAnalyzer::FAllocationsAnalyzer(IAnalysisSession& InSession, FAllocationsProvider& InAllocationsProvider, FMetadataProvider& InMetadataProvider)
	: Session(InSession)
	, AllocationsProvider(InAllocationsProvider)
	, MetadataProvider(InMetadataProvider)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsAnalyzer::~FAllocationsAnalyzer()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Init,                "Memory", "Init");
	Builder.RouteEvent(RouteId_Alloc,               "Memory", "Alloc");
	Builder.RouteEvent(RouteId_AllocSystem,         "Memory", "AllocSystem");
	Builder.RouteEvent(RouteId_AllocVideo,          "Memory", "AllocVideo");
	Builder.RouteEvent(RouteId_Free,                "Memory", "Free");
	Builder.RouteEvent(RouteId_FreeSystem,          "Memory", "FreeSystem");
	Builder.RouteEvent(RouteId_FreeVideo,           "Memory", "FreeVideo");
	Builder.RouteEvent(RouteId_ReallocAlloc,        "Memory", "ReallocAlloc");
	Builder.RouteEvent(RouteId_ReallocAllocSystem,  "Memory", "ReallocAllocSystem");
	Builder.RouteEvent(RouteId_ReallocFree,         "Memory", "ReallocFree");
	Builder.RouteEvent(RouteId_ReallocFreeSystem,   "Memory", "ReallocFreeSystem");
	Builder.RouteEvent(RouteId_HeapMarkAlloc,       "Memory", "HeapMarkAlloc");
	Builder.RouteEvent(RouteId_HeapUnmarkAlloc,     "Memory", "HeapUnmarkAlloc");
	Builder.RouteEvent(RouteId_UpdateAlloc,         "Memory", "UpdateAlloc");
	Builder.RouteEvent(RouteId_MemSwapOp,           "Memory", "MemorySwapOp");
	Builder.RouteEvent(RouteId_Marker,              "Memory", "Marker");
	Builder.RouteEvent(RouteId_TagSpec,             "Memory", "TagSpec");
	Builder.RouteEvent(RouteId_HeapSpec,            "Memory", "HeapSpec");
	Builder.RouteEvent(RouteId_MemScopeTag,         "Memory", "MemoryScope", true);
	Builder.RouteEvent(RouteId_MemScopePtr,         "Memory", "MemoryScopePtr", true);

#if INSIGHTS_MEM_TRACE_METADATA_TEST
	{
		FProviderEditScopeLock _(MetadataProvider);
		TagIdMetadataType = MetadataProvider.RegisterMetadataType(TEXT("MemTagId"), sizeof(TagIdType));
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsAnalyzer::OnAnalysisEnd()
{
	double SessionTime;
	{
		FAnalysisSessionEditScope _(Session);
		SessionTime = Session.GetDurationSeconds();
		if (LastMarkerSeconds > SessionTime)
		{
			Session.UpdateDurationSeconds(LastMarkerSeconds);
		}
	}
	const double Time = FMath::Max(SessionTime, LastMarkerSeconds);

	FProviderEditScopeLock _(AllocationsProvider);
	AllocationsProvider.EditOnAnalysisCompleted(Time);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAllocationsAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FAllocationsAnalyzer"));

	const auto& EventData = Context.EventData;
	HeapId RootHeap = EMemoryTraceRootHeap::SystemMemory;

	switch (RouteId)
	{
		case RouteId_Init:
		{
			const uint32 Version = EventData.GetValue<uint32>("Version", 0);

			using namespace AllocationsAnalyzer::Private;
			if (Version < MinSupportedVersion || Version > MaxSupportedVersion)
			{
				UE_LOGF(LogTraceServices, Error, "[MemAlloc] Version %u for Memory trace events is not supported by the current analyzer. Supported versions: [%u .. %u]", Version, MinSupportedVersion, MaxSupportedVersion);
				break;
			}

			const double Time = GetCurrentTime();
			MarkerPeriod = EventData.GetValue<uint32>("MarkerPeriod");

			const uint8 MinAlignment = EventData.GetValue<uint8>("MinAlignment");
			SizeShift = EventData.GetValue<uint8>("SizeShift");

			uint64 PlatformPageSize = EventData.GetValue<uint64>("PageSize");
			if (PlatformPageSize < 4096) // PageSize is missing from older traces
			{
				PlatformPageSize = 4096;
			}

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditInit(Time, MinAlignment, PlatformPageSize);
			break;
		}

		case RouteId_AllocSystem:
		case RouteId_AllocVideo:
		case RouteId_ReallocAllocSystem:
		{
			RootHeap = RouteId == RouteId_AllocVideo ? EMemoryTraceRootHeap::VideoMemory : EMemoryTraceRootHeap::SystemMemory;
		}
		// intentional fallthrough
		case RouteId_Alloc:
		case RouteId_ReallocAlloc:
		{
			// TODO: Can we have a struct mapping over the EventData?
			// Something like:
			//     const auto& Ev = (const FAllocEvent&)EventData.Get(); // probably not aligned
			// Or something like:
			//     FAllocEvent Event; // aligned
			//     EventData.CopyData(&Event, sizeof(Event));

			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const double Time = GetCurrentTime();

			const uint64 Address = EventData.GetValue<uint64>("Address");
			const uint32 CallstackId = EventData.GetValue<uint32>("CallstackId", 0);
			RootHeap = EventData.GetValue<uint8>("RootHeap", static_cast<uint8>(RootHeap));

			uint64 SizeUpper = EventData.GetValue<uint32>("Size");
			const uint8 SizeLowerMask = static_cast<uint8>((1u << SizeShift) - 1u);
			const uint8 AlignmentPow2_SizeLower = EventData.GetValue<uint8>("AlignmentPow2_SizeLower");
			const uint64 Size = (SizeUpper << SizeShift) | static_cast<uint64>(AlignmentPow2_SizeLower & SizeLowerMask);
			const uint32 Alignment = 1u << (AlignmentPow2_SizeLower >> SizeShift);

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditAlloc(ThreadId, Time, CallstackId, Address, Size, Alignment, RootHeap);
			if (RouteId == RouteId_ReallocAlloc || RouteId == RouteId_ReallocAllocSystem)
			{
				const uint8 Tracker = 0; // We only care about the default tracker for now.
				AllocationsProvider.EditPopTagFromPtr(ThreadId, Tracker);
			}
			break;
		}

		case RouteId_FreeSystem:
		case RouteId_FreeVideo:
		case RouteId_ReallocFreeSystem:
		{
			RootHeap = RouteId == RouteId_FreeVideo ? EMemoryTraceRootHeap::VideoMemory : EMemoryTraceRootHeap::SystemMemory;
		}
		// intentional fallthrough
		case RouteId_Free:
		case RouteId_ReallocFree:
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const double Time = GetCurrentTime();

			const uint64 Address = EventData.GetValue<uint64>("Address");
			const uint32 CallstackId = EventData.GetValue<uint32>("CallstackId", 0);
			RootHeap = EventData.GetValue<uint8>("RootHeap", static_cast<uint8>(RootHeap));

			FProviderEditScopeLock _(AllocationsProvider);
			if (RouteId == RouteId_ReallocFree || RouteId == RouteId_ReallocFreeSystem)
			{
				const uint8 Tracker = 0; // We only care about the default tracker for now.
				AllocationsProvider.EditPushTagFromPtr(ThreadId, Tracker, Address, RootHeap);
			}
			AllocationsProvider.EditFree(ThreadId, Time, CallstackId, Address, RootHeap);
			break;
		}

		case RouteId_HeapMarkAlloc:
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const double Time = GetCurrentTime();

			const uint64 Address = EventData.GetValue<uint64>("Address");
			const uint32 CallstackId = EventData.GetValue<uint32>("CallstackId", 0);
			const HeapId Heap = EventData.GetValue<uint16>("Heap", 0);
			const EMemoryTraceHeapAllocationFlags Flags = EventData.GetValue<EMemoryTraceHeapAllocationFlags>("Flags");

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditMarkAllocationAsHeap(ThreadId, Time, CallstackId, Address, Heap, Flags);
			break;
		}

		case RouteId_HeapUnmarkAlloc:
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const double Time = GetCurrentTime();

			const uint64 Address = EventData.GetValue<uint64>("Address");
			const uint32 CallstackId = EventData.GetValue<uint32>("CallstackId", 0);
			const HeapId Heap = EventData.GetValue<uint16>("Heap", 0);

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditUnmarkAllocationAsHeap(ThreadId, Time, CallstackId, Address, Heap);
			break;
		}

		case RouteId_UpdateAlloc: // added in UE 5.6
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const double Time = GetCurrentTime();

			const uint64 Address = EventData.GetValue<uint64>("Address");
			const uint32 CallstackId = EventData.GetValue<uint32>("CallstackId", 0);
			RootHeap = EventData.GetValue<uint8>("RootHeap", uint8(EMemoryTraceRootHeap::SystemMemory));

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditUpdateAlloc(ThreadId, Time, CallstackId, Address, RootHeap);
			break;
		}

		case RouteId_MemSwapOp: // added in UE 5.5
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const double Time = GetCurrentTime();

			const uint64 Address = EventData.GetValue<uint64>("Address");
			const uint32 CallstackId = EventData.GetValue<uint32>("CallstackId", 0);
			const uint32 CompressedSize = EventData.GetValue<uint32>("CompressedSize");
			const EMemoryTraceSwapOperation SwapOp = (EMemoryTraceSwapOperation)EventData.GetValue<uint8>("SwapOp");

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditSwapOp(ThreadId, Time, Address, SwapOp, CompressedSize, CallstackId);
			break;
		}

		case RouteId_Marker:
		{
			// If BaseCycle is 0, then Cycle is a 64-bit absolute value, otherwise Cycle is a 32-bit value (relative to BaseCycle).
			const uint64 Cycle = (BaseCycle == 0) ? EventData.GetValue<uint64>("Cycle") : BaseCycle + EventData.GetValue<uint32>("Cycle");

			if (ensure(Cycle >= LastMarkerCycle))
			{
				const double Seconds = Context.EventTime.AsSeconds(Cycle);
				check(Seconds >= LastMarkerSeconds);
				if (ensure((Seconds - LastMarkerSeconds < 5 * 60.0) || LastMarkerSeconds == 0.0f))
				{
					LastMarkerCycle = Cycle;
					LastMarkerSeconds = Seconds;
					{
						FAnalysisSessionEditScope _(Session);
						double SessionTime = Session.GetDurationSeconds();
						if (LastMarkerSeconds > SessionTime)
						{
							Session.UpdateDurationSeconds(LastMarkerSeconds);
						}
					}
				}
			}
			break;
		}

		case RouteId_TagSpec:
		{
			const TagIdType Tag = Context.EventData.GetValue<TagIdType>("Tag");
			const TagIdType Parent = Context.EventData.GetValue<TagIdType>("Parent");

			FString Display;
			Context.EventData.GetString("Display", Display); // Display is UE::Trace::AnsiString
			const TCHAR* DisplayString = Session.StoreString(*Display);

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditAddTagSpec(Tag, Parent, DisplayString);
			break;
		}

		case RouteId_HeapSpec:
		{
			const HeapId Id = EventData.GetValue<uint16>("Id");
			const HeapId ParentId = EventData.GetValue<uint16>("ParentId");
			const EMemoryTraceHeapFlags Flags = EventData.GetValue<EMemoryTraceHeapFlags>("Flags");

			FStringView Name;
			EventData.GetString("Name", Name); // Name is UE::Trace::WideString

			FProviderEditScopeLock _(AllocationsProvider);
			AllocationsProvider.EditHeapSpec(Id, ParentId, Name, Flags);
			break;
		}

		case RouteId_MemScopeTag: // "MemoryScope", see UE_MEMSCOPE
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const uint8 Tracker = 0; // We only care about the default tracker for now.

			if (Style == EStyle::EnterScope)
			{
				const TagIdType Tag = Context.EventData.GetValue<TagIdType>("Tag");
				{
					FProviderEditScopeLock _(AllocationsProvider);
					AllocationsProvider.EditPushTag(ThreadId, Tracker, Tag);
				}
#if INSIGHTS_MEM_TRACE_METADATA_TEST
				{
					FProviderEditScopeLock _(MetadataProvider);
					MetadataProvider.PushScopedMetadata(ThreadId, TagIdMetadataType, (void*)&Tag, sizeof(TagIdType));
				}
#endif
			}
			else if (ensure(Style == EStyle::LeaveScope))
			{
#if INSIGHTS_MEM_TRACE_METADATA_TEST
				{
					FProviderEditScopeLock _(MetadataProvider);
					MetadataProvider.PopScopedMetadata(ThreadId, TagIdMetadataType);
				}
#endif
				{
					FProviderEditScopeLock _(AllocationsProvider);
					AllocationsProvider.EditPopTag(ThreadId, Tracker);
				}
			}
			break;
		}

		case RouteId_MemScopePtr: // "MemoryScopePtr", see UE_MEMSCOPE_PTR
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const uint8 Tracker = 0; // We only care about the default tracker for now.

			if (Style == EStyle::EnterScope)
			{
				const uint64 Ptr = Context.EventData.GetValue<uint64>("Ptr");
				RootHeap = EventData.GetValue<uint8>("RootHeap", uint8(EMemoryTraceRootHeap::SystemMemory));

				{
					FProviderEditScopeLock _(AllocationsProvider);
					AllocationsProvider.EditPushTagFromPtr(ThreadId, Tracker, Ptr, RootHeap);
				}
#if INSIGHTS_MEM_TRACE_METADATA_TEST
				{
					FProviderEditScopeLock _(MetadataProvider);
					TagIdType Tag = 0; //TODO: AllocationsProvider.GetTagFromPtr(ThreadId, Tracker, Ptr, RootHeapId);
					MetadataProvider.PushScopedMetadata(ThreadId, TagIdMetadataType, (void*)&Tag, sizeof(TagIdType));
				}
#endif
			}
			else if (ensure(Style == EStyle::LeaveScope))
			{
#if INSIGHTS_MEM_TRACE_METADATA_TEST
				{
					FProviderEditScopeLock _(MetadataProvider);
					MetadataProvider.PopScopedMetadata(ThreadId, TagIdMetadataType);
				}
#endif
				{
					FProviderEditScopeLock _(AllocationsProvider);
					//check(AllocationsProvider.HasTagFromPtrScope(ThreadId, Tracker));
					AllocationsProvider.EditPopTagFromPtr(ThreadId, Tracker);
				}
			}
			break;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FAllocationsAnalyzer::GetCurrentTime() const
{
	return LastMarkerSeconds;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef INSIGHTS_MEM_TRACE_METADATA_TEST

} // namespace TraceServices
