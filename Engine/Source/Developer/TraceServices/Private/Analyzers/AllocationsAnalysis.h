// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices
{

class IAnalysisSession;

class FAllocationsProvider;
class FMetadataProvider;

class FAllocationsAnalyzer : public UE::Trace::IAnalyzer
{
private:
	enum : uint16
	{
		RouteId_Init,
		RouteId_Alloc,
		RouteId_AllocSystem,
		RouteId_AllocVideo,
		RouteId_Free,
		RouteId_FreeSystem,
		RouteId_FreeVideo,
		RouteId_ReallocAlloc,
		RouteId_ReallocAllocSystem,
		RouteId_ReallocFree,
		RouteId_ReallocFreeSystem,
		RouteId_HeapMarkAlloc,
		RouteId_HeapUnmarkAlloc,
		RouteId_UpdateAlloc, // added in UE 5.6
		RouteId_MemSwapOp, // added in UE 5.5
		RouteId_Marker,
		RouteId_TagSpec,
		RouteId_HeapSpec,
		RouteId_MemScopeTag,
		RouteId_MemScopePtr,
	};

public:
	FAllocationsAnalyzer(IAnalysisSession& Session, FAllocationsProvider& AllocationsProvider, FMetadataProvider& MetadataProvider);
	~FAllocationsAnalyzer();
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	double GetCurrentTime() const;

private:
	IAnalysisSession& Session;
	FAllocationsProvider& AllocationsProvider;
	FMetadataProvider& MetadataProvider;
	uint64 BaseCycle = 0;
	uint64 LastMarkerCycle = 0;
	double LastMarkerSeconds = 0.0;
	uint32 MarkerPeriod = 0;
	uint16 TagIdMetadataType = ~0;
	uint8 SizeShift = 0;
};

} // namespace TraceServices
