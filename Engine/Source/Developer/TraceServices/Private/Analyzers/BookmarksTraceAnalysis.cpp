// Copyright Epic Games, Inc. All Rights Reserved.

#include "BookmarksTraceAnalysis.h"

#include "Common/Utils.h"
#include "Common/FormatArgs.h"
#include "Common/ProviderLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "TraceServices/Model/Bookmarks.h"
#include "TraceServices/Model/Log.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

FBookmarksAnalyzer::FBookmarksAnalyzer(IAnalysisSession& InSession, IEditableBookmarkProvider& InEditableBookmarkProvider, IEditableLogProvider* InEditableLogProvider)
	: Session(InSession)
	, EditableBookmarkProvider(InEditableBookmarkProvider)
	, EditableLogProvider(InEditableLogProvider)
{
	if (EditableLogProvider)
	{
		// Todo: update this to use provider locking instead of session locking
		// FProviderEditScopeLock EditableLogProviderLock (*EditableLogProvider);
		FAnalysisSessionEditScope _(Session);
		BookmarkLogCategoryId = EditableLogProvider->RegisterCategory();
		FLogCategoryInfo& BookmarkLogCategory = EditableLogProvider->GetCategory(BookmarkLogCategoryId);
		BookmarkLogCategory.Name = TEXT("LogBookmark");
		BookmarkLogCategory.DefaultVerbosity = ELogVerbosity::All;
	}
}

void FBookmarksAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_BookmarkSpec, "Misc", "BookmarkSpec");
	Builder.RouteEvent(RouteId_Bookmark, "Misc", "Bookmark");
}

bool FBookmarksAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FBookmarksAnalyzer"));

	// Todo: update this to use provider locking instead of session locking
	FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_BookmarkSpec:
	{
		uint64 BookmarkPoint = EventData.GetValue<uint64>("BookmarkPoint");
		const TCHAR* BookmarkFormatString = nullptr;
		const TCHAR* BookmarkFile = nullptr;
		int32 BookmarkLine = EventData.GetValue<int32>("Line");

		FString FileName;
		if (EventData.GetString("FileName", FileName))
		{
			BookmarkFile = Session.StoreString(*FileName);

			FString FormatString;
			EventData.GetString("FormatString", FormatString);
			BookmarkFormatString = Session.StoreString(*FormatString);
		}
		else
		{
			const ANSICHAR* File = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
			BookmarkFile = Session.StoreString(ANSI_TO_TCHAR(File));
			BookmarkFormatString = Session.StoreString(reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + TCString<ANSICHAR>::Strlen(File) + 1));
		}

		{
			// Todo: update this to use provider locking instead of session locking
			// FProviderEditScopeLock EditableBookmarkProviderLock (EditableBookmarkProvider);
			EditableBookmarkProvider.UpdateBookmarkSpec(BookmarkPoint, BookmarkFormatString, BookmarkFile, BookmarkLine);
		}

		if (EditableLogProvider)
		{
			// Todo: update this to use provider locking instead of session locking
			// FProviderEditScopeLock EditableLogProviderLock (*EditableLogProvider);
			EditableLogProvider->UpdateMessageSpec(BookmarkPoint, BookmarkLogCategoryId, BookmarkFormatString, BookmarkFile, BookmarkLine, ELogVerbosity::Log);
		}
		break;
	}

	case RouteId_Bookmark:
	{
		uint64 BookmarkPoint = EventData.GetValue<uint64>("BookmarkPoint");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Timestamp = Context.EventTime.AsSeconds(Cycle);
		TArrayView<const uint8> FormatArgsView = FTraceAnalyzerUtils::LegacyAttachmentArray("FormatArgs", Context);
		uint32 CallstackId = EventData.GetValue<uint32>("CallstackId");

		{
			// Todo: update this to use provider locking instead of session locking
			// FProviderEditScopeLock EditableBookmarkProviderLock (EditableBookmarkProvider);
			EditableBookmarkProvider.AppendBookmark(BookmarkPoint, Timestamp, CallstackId, FormatArgsView.GetData());
		}

		if (EditableLogProvider)
		{
			// Todo: update this to use provider locking instead of session locking
			// FProviderEditScopeLock EditableLogProviderLock (*EditableLogProvider);
			EditableLogProvider->AppendMessage(BookmarkPoint, Timestamp, FormatArgsView);
		}
		break;
	}
	}

	return true;
}

} // namespace TraceServices
