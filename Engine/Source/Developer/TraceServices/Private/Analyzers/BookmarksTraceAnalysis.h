// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Templates/SharedPointer.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Common/PagedArray.h"

namespace TraceServices
{

class IAnalysisSession;
class IEditableBookmarkProvider;
class ILogProvider;
class IEditableLogProvider;

class FBookmarksAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FBookmarksAnalyzer(IAnalysisSession& Session, IEditableBookmarkProvider& EditableBookmarkProvider, IEditableLogProvider* InEditableLogProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_BookmarkSpec,
		RouteId_Bookmark,
	};

	IAnalysisSession& Session;
	IEditableBookmarkProvider& EditableBookmarkProvider;
	IEditableLogProvider* EditableLogProvider;
	uint64 BookmarkLogCategoryId = uint64(-1);

	enum
	{
		FormatBufferSize = 65536
	};
	TCHAR FormatBuffer[FormatBufferSize];
	TCHAR TempBuffer[FormatBufferSize];
};

} // namespace TraceServices
