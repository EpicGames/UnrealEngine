// Copyright Epic Games, Inc. All Rights Reserved.

#include "CallstacksAnalysis.h"

#include "Containers/StaticArray.h"
#include "HAL/LowLevelMemTracker.h"
#include "Model/CallstacksProvider.h"
#include "ProfilingDebugging/CallstackTrace.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/Utils.h"

namespace TraceServices
{

enum ERoutes
{
	RouteId_Callstack,
	RouteId_CallstackXORAndRLE,
	RouteId_CallstackDelta7bit,
	RouteId_CallstackDeltaVarInt
};
	
////////////////////////////////////////////////////////////////////////////////
FCallstacksAnalyzer::FCallstacksAnalyzer(IAnalysisSession& InSession, FCallstacksProvider* InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
	check(Provider != nullptr);
}

////////////////////////////////////////////////////////////////////////////////
void FCallstacksAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;
	Builder.RouteEvent(RouteId_Callstack, "Memory", "CallstackSpec");
	Builder.RouteEvent(RouteId_CallstackXORAndRLE, "Memory", "CallstackSpecXORAndRLE");
	Builder.RouteEvent(RouteId_CallstackDelta7bit, "Memory", "CallstackSpecDelta7bit");
	Builder.RouteEvent(RouteId_CallstackDeltaVarInt, "Memory", "CallstackSpecDeltaVarInt");
}

////////////////////////////////////////////////////////////////////////////////
bool FCallstacksAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FCallstacksAnalyzer"));

	const auto CompressionRoute = [&](auto UncompressFunction)
	{
		const TArrayReader<uint8>& CompressedData = Context.EventData.GetArray<uint8>("CompressedFrames");
		if (const uint32 Id = Context.EventData.GetValue<uint32>("CallstackId"))
		{
			TStaticArray<uint64, 255> Frames;
			uint32 TotalFrameCount = 0;
			const uint32 FrameCount = UncompressFunction(
				MakeArrayView(CompressedData.GetData(), CompressedData.Num()),
				Frames,
				TotalFrameCount
			);

			check(FrameCount <= 255);
			if (TotalFrameCount != FrameCount)
	        {
				UE_LOGF(LogTraceServices, Warning, "Callstack with Id=%u has %u frames, but it will be limited to %u frames!", Id, TotalFrameCount, FrameCount);
	        }
			Provider->AddCallstack(Id, Frames.GetData(), uint8(FrameCount));
		}
	};
	
	switch (RouteId)
	{
		case RouteId_Callstack:
		{
			const TArrayReader<uint64>& Frames = Context.EventData.GetArray<uint64>("Frames");
			uint8 NumFrames = (uint8)FMath::Min(255u, Frames.Num());
			if (const uint32 Id = Context.EventData.GetValue<uint32>("CallstackId"))
			{
				if (NumFrames != Frames.Num())
				{
					UE_LOGF(LogTraceServices, Warning, "Callstack with Id=%u has %u frames, but it will be limited to %u frames!", Id, Frames.Num(), NumFrames);
				}
				Provider->AddCallstack(Id, Frames.GetData(), NumFrames);
			}
			// Backward compatibility with legacy memory trace format (5.0-EA).
			else if (const uint64 Hash = Context.EventData.GetValue<uint64>("Id"))
			{
				if (NumFrames != Frames.Num())
				{
					UE_LOGF(LogTraceServices, Warning, "Callstack with Hash=%llu has %u frames, but it will be limited to %u frames!", Hash, Frames.Num(), NumFrames);
				}
				Provider->AddCallstackWithHash(Hash, Frames.GetData(), NumFrames);
			}
		}
		break;

		case RouteId_CallstackXORAndRLE:
		{
			CompressionRoute(&FCallstackXORAndRLE::Uncompress);
		}
		break;

		case RouteId_CallstackDelta7bit:
		{
			CompressionRoute(&FCallstackDelta7bit::Uncompress);
		}
		break;

		case RouteId_CallstackDeltaVarInt:
		{
			CompressionRoute(&FCallstackDeltaVarInt::Uncompress);
		}
		break;
	}

	return true;
}

} // namespace TraceServices
