// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

////////////////////////////////////////////////////////////////////////////////
namespace TraceServices
{

class IAnalysisSession;
class FCallstacksProvider;

////////////////////////////////////////////////////////////////////////////////
class FCallstacksAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FCallstacksAnalyzer(IAnalysisSession& Session, FCallstacksProvider* Provider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:

	IAnalysisSession& Session;
	FCallstacksProvider* Provider;
};

} // namespace TraceServices
