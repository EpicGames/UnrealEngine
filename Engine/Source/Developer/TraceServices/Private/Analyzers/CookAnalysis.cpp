// Copyright Epic Games, Inc. All Rights Reserved.
#include "CookAnalysis.h"

#include "Analyzers/MiscTraceAnalysis.h"
#include "Common/FormatArgs.h"
#include "Common/Utils.h"
#include "Model/CookProfilerProviderPrivate.h"
#include "ProfilingDebugging/CookStats.h"

#include <limits>

namespace TraceServices
{

FCookAnalyzer::FCookAnalyzer(IAnalysisSession& InSession, FCookProfilerProvider& InCookProfilerProvider)
	: Session(InSession)
	, CookProfilerProvider(InCookProfilerProvider)
{
}

FCookAnalyzer::~FCookAnalyzer()
{
}

void FCookAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Package, "CookTrace", "Package");
	Builder.RouteEvent(RouteId_PackageAssetClass, "CookTrace", "PackageAssetClass");

	//V1
	Builder.RouteEvent(RouteId_PackageStat, "CookTrace", "PackageStat");

	//V2
	Builder.RouteEvent(RouteId_PackageStatBeginScope, "CookTrace", "PackageStatBeginScope");
	Builder.RouteEvent(RouteId_PackageStatEndScope, "CookTrace", "PackageStatEndScope");
}

bool FCookAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_Package:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		FString Name;
		EventData.GetString("Name", Name);
		const TCHAR* PersistentName = Session.StoreString(Name);

		FProviderEditScopeLock ProviderEditScope(CookProfilerProvider);
		FPackageData* Package = CookProfilerProvider.EditPackage(Id);
		Package->Name = PersistentName;
		break;
	}
	case RouteId_PackageStat:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		double Value = Context.EventTime.AsSecondsAbsolute(EventData.GetValue<uint64>("Duration"));
		EPackageEventStatType StatType = (EPackageEventStatType) EventData.GetValue<uint8>("StatType");

		FProviderEditScopeLock ProviderEditScope(CookProfilerProvider);
		FPackageData* Package = CookProfilerProvider.EditPackage(Id);
		check(Package);
		switch (StatType)
		{
			case EPackageEventStatType::LoadPackage:
			{
				// We measure the Loadtime in multiple scopes so we receive many LoadPackage events.
				Package->LoadTimeIncl += Value;
				break;
			}
			case EPackageEventStatType::SavePackage:
			{
				Package->SaveTimeIncl = Value;
				break;
			}
			case EPackageEventStatType::BeginCacheForCookedPlatformData:
			{
				// A BeginCacheForCookedPlatformData event is received for each asset in the package. We add the values to get the total time for the package.
				Package->BeginCacheForCookedPlatformDataIncl += Value;
				break;
			}
			case EPackageEventStatType::IsCachedCookedPlatformDataLoaded:
			{
				// A IsCachedCookedPlatformDataLoaded event is received for each asset in the package. We add the values to get the total time for the package.
				Package->IsCachedCookedPlatformDataLoadedIncl += Value;
				break;
			}
			default:
			{
				break;
			}
		}
		break;
	}
	case RouteId_PackageStatBeginScope:
	{
		uint64 PackageId = EventData.GetValue<uint64>("Id");
		double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Time"));
		EPackageEventStatType StatType = (EPackageEventStatType)EventData.GetValue<uint8>("StatType");

		FProviderEditScopeLock ProviderEditScope(CookProfilerProvider);
		CookProfilerProvider.AddScopeEntry(Context.ThreadInfo.GetId(), PackageId, Timestamp, StatType, true);

		FAnalysisSessionEditScope _EditScope(Session);
		Session.UpdateDurationSeconds(Timestamp);

		break;
	}
	case RouteId_PackageStatEndScope:
	{
		uint64 PackageId = EventData.GetValue<uint64>("Id");
		double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Time"));
		EPackageEventStatType StatType = (EPackageEventStatType)EventData.GetValue<uint8>("StatType");

		FProviderEditScopeLock ProviderEditScope(CookProfilerProvider);
		CookProfilerProvider.AddScopeEntry(Context.ThreadInfo.GetId(), PackageId, Timestamp, StatType, false);

		FAnalysisSessionEditScope _EditScope(Session);
		Session.UpdateDurationSeconds(Timestamp);

		break; 
	}
	case RouteId_PackageAssetClass:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		FString ClassName;
		EventData.GetString("ClassName", ClassName);
		const TCHAR* PersistentClassName = Session.StoreString(ClassName);

		FProviderEditScopeLock ProviderEditScope(CookProfilerProvider);
		FPackageData* Package = CookProfilerProvider.EditPackage(Id);
		check(Package);
		Package->AssetClass = PersistentClassName;
		break;
	}
	}

	return true;
}

} // namespace TraceServices
