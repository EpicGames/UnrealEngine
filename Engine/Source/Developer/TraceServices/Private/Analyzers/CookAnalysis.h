// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "AnalysisServicePrivate.h"

namespace TraceServices
{
class FCookProfilerProvider;

class FCookAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FCookAnalyzer(IAnalysisSession& Session, FCookProfilerProvider& CookProfilerProvider);
	virtual ~FCookAnalyzer();

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	
private:
	enum : uint16
	{
		// Common
		RouteId_Package,
		RouteId_PackageAssetClass,

		// Version 1
		RouteId_PackageStat,

		//Version 2 (UE 5.5+)
		RouteId_PackageStatBeginScope,
		RouteId_PackageStatEndScope,
	};

	IAnalysisSession& Session;
	FCookProfilerProvider& CookProfilerProvider;
};

} // namespace TraceServices
