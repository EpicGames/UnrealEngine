// Copyright Epic Games, Inc. All Rights Reserved.

#include "CountersTraceAnalysis.h"

#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "Math/UnrealMathUtility.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Templates/UniquePtr.h"
#include "TraceServices/Model/Counters.h"

namespace TraceServices
{

class FCountersAnalyzer::FTraceStatsHelper 
{
	public:
		FTraceStatsHelper(IEditableCounterProvider& InEditableCounterProvider)
		{
			StatDefinitions = {
				{TraceCounterType_Float, CounterDisplayHint_Bandwidth, TEXT("Trace/Bandwidth/Sent")},
				{TraceCounterType_Float, CounterDisplayHint_Bandwidth, TEXT("Trace/Bandwidth/Emitted")},
				{TraceCounterType_Float, CounterDisplayHint_Percent, TEXT("Trace/BlockPoolUsage")},
				{TraceCounterType_Int, CounterDisplayHint_Memory, TEXT("Trace/Memory/MemoryUsed")},
				{TraceCounterType_Int, CounterDisplayHint_Memory, TEXT("Trace/Memory/BlockPoolAllocated")},
				{TraceCounterType_Int, CounterDisplayHint_Memory, TEXT("Trace/Memory/FixedBufferAllocated")},
				{TraceCounterType_Int, CounterDisplayHint_Memory, TEXT("Trace/Memory/CacheAllocated")},
				{TraceCounterType_Int, CounterDisplayHint_Memory, TEXT("Trace/Memory/SharedAllocated")},
			};

			for (auto& StatDefinition : StatDefinitions)
			{
				IEditableCounter* Counter = InEditableCounterProvider.CreateEditableCounter();
				if (StatDefinition.Type == TraceCounterType_Float)
				{
					Counter->SetIsFloatingPoint(true);
				}
				Counter->SetName(StatDefinition.DisplayName);
				Counter->SetDisplayHint(StatDefinition.DisplayHint);
				StatDefinition.Counter = Counter;
			}
		}

		void ProcessBwStatEvent(double Time, double DeltaTime, const FEventData& EventData)
		{
			const uint32 DeltaBytesSent = EventData.GetValue<uint32>("DeltaBytesSent");
			const uint32 DeltaBytesEmitted = EventData.GetValue<uint32>("DeltaBytesEmitted");

			StatDefinitions[0].Counter->SetValue(Time, float(DeltaBytesSent)/DeltaTime);
			StatDefinitions[1].Counter->SetValue(Time, float(DeltaBytesEmitted)/DeltaTime);
		}

		void ProcessMemStatEvent(double Time, const FEventData& EventData)
		{
			const float BlockPoolUsage = EventData.GetValue<float>("BlockPoolUsage");
			const uint64 MemoryUsed = EventData.GetValue<uint64>("MemoryUsed");
			const uint64 BlockPoolAllocated = EventData.GetValue<uint64>("BlockPoolAllocated");
			const uint32 FixedBufferAllocated = EventData.GetValue<uint32>("FixedBufferAllocated");
			const uint32 CacheAllocated = EventData.GetValue<uint32>("CacheAllocated");
			const uint32 SharedAllocated = EventData.GetValue<uint32>("SharedAllocated");

			StatDefinitions[2].Counter->SetValue(Time, BlockPoolUsage);
			StatDefinitions[3].Counter->SetValue(Time, int64(MemoryUsed));
			StatDefinitions[4].Counter->SetValue(Time, int64(BlockPoolAllocated));
			StatDefinitions[5].Counter->SetValue(Time, int64(FixedBufferAllocated));
			StatDefinitions[6].Counter->SetValue(Time, int64(CacheAllocated));
			StatDefinitions[7].Counter->SetValue(Time, int64(SharedAllocated));
		}

	private:
		struct FTraceStatsDefinition 
		{
			ETraceCounterType Type;
			ECounterDisplayHint DisplayHint;
			const TCHAR* DisplayName;
			IEditableCounter* Counter = nullptr;
		};

		TArray<FTraceStatsDefinition> StatDefinitions;
};

FCountersAnalyzer::FCountersAnalyzer(IAnalysisSession& InSession, IEditableCounterProvider& InEditableCounterProvider)
	: Session(InSession)
	, EditableCounterProvider(InEditableCounterProvider)
{
}

FCountersAnalyzer::~FCountersAnalyzer()
{
}

void FCountersAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Spec, "Counters", "Spec");
	Builder.RouteEvent(RouteId_SetValueInt, "Counters", "SetValueInt");
	Builder.RouteEvent(RouteId_SetValueFloat, "Counters", "SetValueFloat");
	Builder.RouteEvent(RouteId_TraceBwStats, "$Trace", "BandwidthStats");
	Builder.RouteEvent(RouteId_TraceMemStats, "$Trace", "MemoryStats");
}

bool FCountersAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FCountersAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_Spec:
	{
		uint16 CounterId = EventData.GetValue<uint16>("Id");
		ETraceCounterType CounterType = static_cast<ETraceCounterType>(EventData.GetValue<uint8>("Type"));
		ETraceCounterDisplayHint CounterDisplayHint = static_cast<ETraceCounterDisplayHint>(EventData.GetValue<uint8>("DisplayHint"));
		IEditableCounter* EditableCounter = EditableCounterProvider.CreateEditableCounter();
		if (CounterType == TraceCounterType_Float)
		{
			EditableCounter->SetIsFloatingPoint(true);
		}
		if (CounterDisplayHint == TraceCounterDisplayHint_Memory)
		{
			EditableCounter->SetDisplayHint(CounterDisplayHint_Memory);
		}
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		if (Name.IsEmpty())
		{
			UE_LOGF(LogTraceServices, Warning, "Invalid counter name for counter %u.", uint32(CounterId));
			Name = FString::Printf(TEXT("<noname counter %u>"), uint32(CounterId));
		}
		EditableCounter->SetName(Session.StoreString(*Name));
		EditableCountersMap.Add(CounterId, EditableCounter);
		break;
	}
	case RouteId_SetValueInt:
	{
		const double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));
		const int64 Value = EventData.GetValue<int64>("Value");
		const uint16 CounterId = EventData.GetValue<uint16>("CounterId");
		IEditableCounter* FindEditableCounter = EditableCountersMap.FindRef(CounterId);
		if (ensure(FindEditableCounter))
		{
			Session.UpdateDurationSeconds(Timestamp);
			FindEditableCounter->SetValue(Timestamp, Value);
		}
		break;
	}
	case RouteId_SetValueFloat:
	{
		const double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle"));
		const float Value = EventData.GetValue<float>("Value");
		const uint16 CounterId = EventData.GetValue<uint16>("CounterId");
		IEditableCounter* FindEditableCounter = EditableCountersMap.FindRef(CounterId);
		if (ensure(FindEditableCounter))
		{
			Session.UpdateDurationSeconds(Timestamp);
			FindEditableCounter->SetValue(Timestamp, Value);
		}
		break;
	}

	case RouteId_TraceBwStats:
	case RouteId_TraceMemStats:
	{
		if (!TraceStatsHelper)
		{
			TraceStatsHelper = MakeUnique<FTraceStatsHelper>(EditableCounterProvider);
		}
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double Time = Context.EventTime.AsSecondsAbsolute(Cycle);
		Session.UpdateDurationSeconds(Time);
		if (RouteId == RouteId_TraceBwStats)
		{
			const double Delta = FMath::Max(Context.EventTime.AsSecondsAbsolute(Cycle - LastStatTimeCycle), UE_DOUBLE_SMALL_NUMBER);
			TraceStatsHelper->ProcessBwStatEvent(Time, Delta, EventData);
			LastStatTimeCycle = Cycle;
		}
		else
		{
			TraceStatsHelper->ProcessMemStatEvent(Time, EventData);
		}
		break;
	}

	}

	return true;
}

} // namespace TraceServices
