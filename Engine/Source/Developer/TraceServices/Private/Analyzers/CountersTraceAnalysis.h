// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/Map.h"

namespace TraceServices
{

class IAnalysisSession;
class IEditableCounter;
class IEditableCounterProvider;

class FCountersAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FCountersAnalyzer(IAnalysisSession& Session, IEditableCounterProvider& EditableCounterProvider);
	~FCountersAnalyzer();
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_Spec,
		RouteId_SetValueInt,
		RouteId_SetValueFloat,
		RouteId_TraceBwStats,
		RouteId_TraceMemStats,
	};

	class FTraceStatsHelper;

	IAnalysisSession& Session;
	IEditableCounterProvider& EditableCounterProvider;
	TMap<uint16, IEditableCounter*> EditableCountersMap;
	TUniquePtr<FTraceStatsHelper> TraceStatsHelper;
	uint64 LastStatTimeCycle = 0;
};

} // namespace TraceServices
