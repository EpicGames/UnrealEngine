// Copyright Epic Games, Inc. All Rights Reserved.

#include "CpuProfilerTraceAnalysis.h"

#include "CborReader.h"
#include "CborWriter.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

// TraceServices
#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "Model/MonotonicTimeline.h"
#include "Model/ThreadsPrivate.h"
#include "TraceServices/Model/TimingProfiler.h"
#include "TraceServices/Utils.h"

#define CPUPROFILER_DEBUG_LOGF(Format, ...) //{ if (ThreadState.ThreadId == 2) FPlatformMisc::LowLevelOutputDebugStringf(Format, __VA_ARGS__); }
#define CPUPROFILER_DEBUG_BEGIN_EVENT(Time, Event) { ++TotalScopeCount; }
#define CPUPROFILER_DEBUG_END_EVENT(Time)
#define CPUPROFILER_LOG_API_L1(Format, ...) UE_LOGF(LogTraceServices, Log, Format, ##__VA_ARGS__)
#define CPUPROFILER_LOG_API_L2(Format, ...) //UE_LOGF(LogTraceServices, Verbose, Format, ##__VA_ARGS__)
#define CPUPROFILER_LOG_API_L3(Format, ...) //UE_LOGF(LogTraceServices, VeryVerbose, Format, ##__VA_ARGS__)

// debug functionality; adds detection and extra logging for timer name changes
#define CPUPROFILER_DETECT_TIMER_NAME_CHANGE 0

// debug functionality; filters the trace events by ThreadId
#define CPUPROFILER_FILTER_BY_THREAD(ThreadId, ReturnOrBreak) //{ if (ThreadId != 2) { ReturnOrBreak; } }

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FCpuProfilerAnalyzer::FCpuProfilerAnalyzer(IAnalysisSession& InSession, IEditableTimingProfilerProvider& InEditableTimingProfilerProvider, IEditableThreadProvider& InEditableThreadProvider)
	: Session(InSession)
	, EditableTimingProfilerProvider(InEditableTimingProfilerProvider)
	, EditableThreadProvider(InEditableThreadProvider)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCpuProfilerAnalyzer::~FCpuProfilerAnalyzer()
{
	for (auto& KV : ThreadStatesMap)
	{
		FThreadState* ThreadState = KV.Value;
		delete ThreadState;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_EventSpec,    "CpuProfiler", "EventSpec");
	Builder.RouteEvent(RouteId_EndThread,    "CpuProfiler", "EndThread");
	Builder.RouteEvent(RouteId_EventBatchV3, "CpuProfiler", "EventBatchV3"); // added in UE 5.6
	Builder.RouteEvent(RouteId_EventBatchV2, "CpuProfiler", "EventBatchV2"); // backward compatibility, added in UE 5.1, removed in 5.6
	Builder.RouteEvent(RouteId_EventBatch,   "CpuProfiler", "EventBatch"); // backward compatibility; removed in UE 5.1
	Builder.RouteEvent(RouteId_EndCapture,   "CpuProfiler", "EndCapture"); // backward compatibility; removed in UE 5.1
	Builder.RouteEvent(RouteId_MetadataSpec, "CpuProfiler", "MetadataSpec");
	Builder.RouteEvent(RouteId_Metadata,     "CpuProfiler", "Metadata");

	Builder.RouteLoggerEvents(RouteId_CpuScope, "Cpu", true); // scoped trace events
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::OnAnalysisEnd()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FCpuProfilerAnalyzer"));

	for (auto& KV : ThreadStatesMap)
	{
		FThreadState& ThreadState = *KV.Value;

		if (ThreadState.LastCycle != ~0ull) // if EndThread is not received
		{
			FAnalysisSessionEditScope _(Session);
			DispatchRemainingPendingEvents(ThreadState);
			EndOpenEvents(ThreadState, std::numeric_limits<double>::infinity());
		}

		check(ThreadState.PendingEvents.Num() == 0); // no pending events
		check(ThreadState.ScopeStack.Num() == 0); // no open events
	}

	bool bPossibleOutputEventTypeIssue = false;
	ScopeNameToTimerIdMap.ValueSort([](const FTimerInfo& A, const FTimerInfo& B) { return A.Count > B.Count; });
	for (auto& KV : ScopeNameToTimerIdMap)
	{
		if (KV.Value.Count < 1000)
		{
			break;
		}
		UE_LOGF(LogTraceServices, Warning, "[CpuProfiler] Timer defined %u times! (id=%d name=\"%ls\")", KV.Value.Count, KV.Value.Id, (const TCHAR*)KV.Key);
		bPossibleOutputEventTypeIssue = true;
	}
	if (bPossibleOutputEventTypeIssue)
	{
		UE_LOGF(LogTraceServices, Warning, "[CpuProfiler] Possible incorrect use of FCpuProfilerTrace::OutputEventType()!");
	}

	if (NumTimerWarnings > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[CpuProfiler] Number of timer/metadata warnings: %u", NumTimerWarnings);
	}

	UE_LOGF(LogTraceServices, Log, "[CpuProfiler] Analysis completed (%d threads, %d timers, %d spec ids, %llu scopes, %llu bytes, %.1f bytes/scope).",
		ThreadStatesMap.Num(),
		ScopeNameToTimerIdMap.Num(),
		SpecIdToTimerIdMap.Num(),
		TotalScopeCount,
		TotalEventSize,
		(double)TotalEventSize / (double)TotalScopeCount);

	// Clean-up...
	for (auto& KV : ThreadStatesMap)
	{
		FThreadState* ThreadState = KV.Value;
		delete ThreadState;
	}
	ThreadStatesMap.Reset();
	ThreadStatesMap.Shrink();
	SpecIdToTimerIdMap.Reset();
	SpecIdToTimerIdMap.Shrink();
	ScopeNameToTimerIdMap.Reset();
	ScopeNameToTimerIdMap.Shrink();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCpuProfilerAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FCpuProfilerAnalyzer"));

	const auto& EventData = Context.EventData;

	switch (RouteId)
	{

	case RouteId_EventSpec:
	{
		OnEventSpec(Context);
		break;
	}

	case RouteId_EndThread:
	{
		const uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		CPUPROFILER_FILTER_BY_THREAD(ThreadId, break);
		FThreadState& ThreadState = GetOrAddThreadState(ThreadId);

		if (ThreadState.LastCycle == ~0ull)
		{
			// Ignore timing events received after EndThread.
			break;
		}

		{
			FAnalysisSessionEditScope _(Session);
			DispatchRemainingPendingEvents(ThreadState);
		}

		const uint64 Cycle = EventData.GetValue<uint64>("Cycle", ThreadState.LastCycle); // added in UE 5.4
		if (Cycle != 0)
		{
			ensure(Cycle >= ThreadState.LastCycle);
			double Timestamp = Context.EventTime.AsSeconds(Cycle);
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Timestamp);
			EndOpenEvents(ThreadState, Timestamp);
		}

		check(ThreadState.PendingEvents.Num() == 0); // no pending events
		check(ThreadState.ScopeStack.Num() == 0); // no open events

		ThreadState.LastCycle = ~0ull;
		break;
	}

	case RouteId_EventBatchV3:
	case RouteId_EventBatchV2: // backward compatibility
	{
		const uint32 ThreadId = Context.ThreadInfo.GetId();
		CPUPROFILER_FILTER_BY_THREAD(ThreadId, break);
		FThreadState& ThreadState = GetOrAddThreadState(ThreadId);

		if (ThreadState.LastCycle == ~0ull)
		{
			// Ignore timing events received after EndThread.
			break;
		}

		TArrayView<const uint8> DataView = Context.EventData.GetArrayView<uint8>("Data");
		const uint32 BufferSize = DataView.Num();
		const uint8* BufferPtr = DataView.GetData();

		ProcessBufferV2(Context.EventTime, ThreadState, BufferPtr, BufferSize, RouteId == RouteId_EventBatchV3 ? 3 : 2);
		if (ThreadState.LastCycle != 0)
		{
			double Timestamp = Context.EventTime.AsSeconds(ThreadState.LastCycle);

			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Timestamp);
		}

		TotalEventSize += BufferSize;
		break;
	}

	case RouteId_EventBatch: // backward compatibility
	case RouteId_EndCapture: // backward compatibility
	{
		const uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		CPUPROFILER_FILTER_BY_THREAD(ThreadId, break);
		FThreadState& ThreadState = GetOrAddThreadState(ThreadId);

		if (ThreadState.LastCycle == ~0ull)
		{
			// Ignore timing events received after EndThread.
			break;
		}

		TArrayView<const uint8> DataView = FTraceAnalyzerUtils::LegacyAttachmentArray("Data", Context);
		const uint32 BufferSize = DataView.Num();
		const uint8* BufferPtr = DataView.GetData();

		ProcessBuffer(Context.EventTime, ThreadState, BufferPtr, BufferSize);

		if (RouteId == RouteId_EndCapture)
		{
			FAnalysisSessionEditScope _(Session);
			DispatchRemainingPendingEvents(ThreadState);
			if (ThreadState.LastCycle != 0)
			{
				double Timestamp = Context.EventTime.AsSeconds(ThreadState.LastCycle);
				Session.UpdateDurationSeconds(Timestamp);
				EndOpenEvents(ThreadState, Timestamp);
			}
			ThreadState.LastCycle = ~0ull;
		}
		else if (ThreadState.LastCycle != 0)
		{
			double Timestamp = Context.EventTime.AsSeconds(ThreadState.LastCycle);
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Timestamp);
		}

		TotalEventSize += BufferSize;
		break;
	}

	case RouteId_CpuScope:
		if (Style == EStyle::EnterScope)
		{
			OnCpuScopeEnter(Context);
		}
		else
		{
			OnCpuScopeLeave(Context);
		}
		break;

	case RouteId_MetadataSpec:
		OnMetadataSpec(Context);
		break;

	case RouteId_Metadata:
	{
		uint32 MetadataId = Context.EventData.GetValue<uint32>("Id");
		uint32 SpecId = Context.EventData.GetValue<uint32>("SpecId");
		TArray<uint8> Metadata(EventData.GetArrayView<uint8>("Metadata"));

		uint32 TimerId = GetOrAddTimer(SpecId);

		if (ensure(Metadata.Num() > 0))
		{
			// We don't know if this Metadata trace event or a timing event with metadata arrive first, so handle both cases.
			const uint32* FoundMetadataTimerId = MetadataIdToTimerIdMap.Find(MetadataId);
			if (FoundMetadataTimerId == nullptr)
			{
				FAnalysisSessionEditScope _(Session);
				TimerId = EditableTimingProfilerProvider.AddMetadata(TimerId, MoveTemp(Metadata));
				MetadataIdToTimerIdMap.Add(MetadataId, TimerId);
			}
			else
			{
				// Replace the placeholder metadata added if we received a timing event with this metadata first.
				FAnalysisSessionEditScope _(Session);
				EditableTimingProfilerProvider.SetMetadata(*FoundMetadataTimerId, MoveTemp(Metadata), TimerId);
			}
		}
		break;
	}

	} // switch (RouteId)

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::ProcessBuffer(const FEventTime& EventTime, FThreadState& ThreadState, const uint8* BufferPtr, uint32 BufferSize)
{
	FAnalysisSessionEditScope _(Session);

	uint64 LastCycle = ThreadState.LastCycle;

	CPUPROFILER_DEBUG_LOGF(TEXT("[%u] ProcessBuffer %llu (%.9f)\n"), ThreadState.ThreadId, LastCycle, EventTime.AsSeconds(LastCycle));

	check(EventTime.GetTimestamp() == 0);
	const uint64 BaseCycle = EventTime.AsCycle64();

	int32 RemainingPending = ThreadState.PendingEvents.Num();
	const FPendingEvent* PendingCursor = ThreadState.PendingEvents.GetData();

	const uint8* BufferEnd = BufferPtr + BufferSize;
	while (BufferPtr < BufferEnd)
	{
		uint64 DecodedCycle = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		uint64 ActualCycle = (DecodedCycle >> 1);

		// ActualCycle larger or equal to LastCycle means we have a new
		// base value.
		if (ActualCycle < LastCycle)
		{
			ActualCycle += LastCycle;
		}

		// If we late connect we will be joining the cycle stream mid-flow and
		// will have missed out on it's base timestamp. Reconstruct it here.
		if (ActualCycle < BaseCycle)
		{
			ActualCycle += BaseCycle;
		}

		// Dispatch pending events that are younger than the one we've just decoded.
		DispatchPendingEvents(LastCycle, ActualCycle, ThreadState, PendingCursor, RemainingPending, (DecodedCycle & 1ull) != 0);

		double ActualTime = EventTime.AsSeconds(ActualCycle);

		if (DecodedCycle & 1ull)
		{
			uint32 SpecId = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));
			uint32 TimerId = GetOrAddTimer(SpecId);

			FEventScopeState& ScopeState = ThreadState.ScopeStack.AddDefaulted_GetRef();
			ScopeState.StartCycle = ActualCycle;
			ScopeState.EventTypeId = TimerId;

			CPUPROFILER_DEBUG_LOGF(TEXT("[%u]  B=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
			FTimingProfilerEvent Event;
			Event.TimerIndex = TimerId;
			ThreadState.Timeline->AppendBeginEvent(ActualTime, Event);
			CPUPROFILER_DEBUG_BEGIN_EVENT(ActualTime, Event);
		}
		else
		{
			// If we receive mismatched end events ignore them for now.
			// This can happen for example because tracing connects to the store after events were traced. Those events can be lost.
			if (ThreadState.ScopeStack.Num() > 0)
			{
				ThreadState.ScopeStack.Pop();
				CPUPROFILER_DEBUG_LOGF(TEXT("[%u]  E=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
				ThreadState.Timeline->AppendEndEvent(ActualTime);
				CPUPROFILER_DEBUG_END_EVENT(ActualTime);
			}
		}

		check(ActualCycle > 0);
		LastCycle = ActualCycle;
	}
	check(BufferPtr == BufferEnd);

	if (RemainingPending == 0)
	{
		//CPUPROFILER_DEBUG_LOGF(TEXT("[%u] MetaEvents: %d added\n"), ThreadState.ThreadId, ThreadState.PendingEvents.Num());
		ThreadState.PendingEvents.Reset();
	}
	else
	{
		const int32 NumEventsToRemove = ThreadState.PendingEvents.Num() - RemainingPending;
		CPUPROFILER_DEBUG_LOGF(TEXT("[%u] MetaEvents: %d added, %d still pending\n"), ThreadState.ThreadId, NumEventsToRemove, RemainingPending);
		ThreadState.PendingEvents.RemoveAt(0, NumEventsToRemove);
	}

	ThreadState.LastCycle = LastCycle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::ProcessBufferV2(const FEventTime& EventTime, FThreadState& ThreadState, const uint8* BufferPtr, uint32 BufferSize, int32 Version)
{
	FAnalysisSessionEditScope _(Session);

	uint64 LastCycle = ThreadState.LastCycle;

	CPUPROFILER_DEBUG_LOGF(TEXT("[%u] ProcessBuffer %llu (%.9f)\n"), ThreadState.ThreadId, LastCycle, EventTime.AsSeconds(LastCycle));

	check(EventTime.GetTimestamp() == 0);
	const uint64 BaseCycle = EventTime.AsCycle64();

	int32 RemainingPending = ThreadState.PendingEvents.Num();
	const FPendingEvent* PendingCursor = ThreadState.PendingEvents.GetData();

	const uint8* BufferEnd = BufferPtr + BufferSize;
	while (BufferPtr < BufferEnd)
	{
		uint64 DecodedCycle = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		uint64 ActualCycle = (DecodedCycle >> 2);

		// ActualCycle larger or equal to LastCycle means we have a new
		// base value.
		if (ActualCycle < LastCycle)
		{
			ActualCycle += LastCycle;
		}

		// If we late connect we will be joining the cycle stream mid-flow and
		// will have missed out on it's base timestamp. Reconstruct it here.
		if (ActualCycle < BaseCycle)
		{
			ActualCycle += BaseCycle;
		}

		// Dispatch pending events that are younger than the one we've just decoded.
		DispatchPendingEvents(LastCycle, ActualCycle, ThreadState, PendingCursor, RemainingPending, (DecodedCycle & 1ull) != 0);

		double ActualTime = EventTime.AsSeconds(ActualCycle);

		if (DecodedCycle & 2ull)
		{
			constexpr uint32 CoroutineSpecId        = (1u << 31u) - 1u;
			constexpr uint32 CoroutineUnknownSpecId = (1u << 31u) - 2u;

			if (DecodedCycle & 1ull)
			{
				uint64 CoroutineId = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
				uint32 TimerScopeDepth = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));

				// Begins a "CoroTask" scoped timer.
				{
					if (CoroutineTimerId == ~0u)
					{
						CoroutineTimerId = AddCpuTimer(CoroutineSpecId, TEXT("Coroutine"));
					}

					TArray<uint8> CborData;
					{
						CborData.Reserve(256);
						FMemoryWriter MemoryWriter(CborData, false, true);
						FCborWriter CborWriter(&MemoryWriter, ECborEndianness::StandardCompliant);
						CborWriter.WriteContainerStart(ECborCode::Map, 2); // 2 is the FieldCount
						CborWriter.WriteValue("Id", 2);
						CborWriter.WriteValue(CoroutineId);
						CborWriter.WriteValue("C", 1); // continuation?
						CborWriter.WriteValue(false);
					}
					uint32 MetadataTimerId = EditableTimingProfilerProvider.AddMetadata(CoroutineTimerId, MoveTemp(CborData));

					FEventScopeState& ScopeState = ThreadState.ScopeStack.AddDefaulted_GetRef();
					ScopeState.StartCycle = ActualCycle;
					ScopeState.EventTypeId = MetadataTimerId;

					CPUPROFILER_DEBUG_LOGF(TEXT("[%u] *B=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
					FTimingProfilerEvent Event;
					Event.TimerIndex = MetadataTimerId;
					ThreadState.Timeline->AppendBeginEvent(ActualTime, Event);
					CPUPROFILER_DEBUG_BEGIN_EVENT(ActualTime, Event);
				}

				// Begins the CPU scoped timers (suspended in previous coroutine execution).
				{
					if (CoroutineUnknownTimerId == ~0u)
					{
						CoroutineUnknownTimerId = AddCpuTimer(CoroutineUnknownSpecId, TEXT("<unknown>"));
					}

					//TODO: Restore the saved stack of CPU scoped timers for this CoroutineId.
					for (uint32 i = 0; i < TimerScopeDepth; ++i)
					{
						FEventScopeState& ScopeState = ThreadState.ScopeStack.AddDefaulted_GetRef();
						ScopeState.StartCycle = ActualCycle;
						ScopeState.EventTypeId = CoroutineUnknownTimerId;

						CPUPROFILER_DEBUG_LOGF(TEXT("[%u] +B=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
						FTimingProfilerEvent Event;
						Event.TimerIndex = CoroutineUnknownTimerId;
						ThreadState.Timeline->AppendBeginEvent(ActualTime, Event);
						CPUPROFILER_DEBUG_BEGIN_EVENT(ActualTime, Event);
					}
				}
			}
			else
			{
				uint32 TimerScopeDepth = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));

				if (TimerScopeDepth != 0)
				{
					//TODO: Save current stack of CPU scoped timers (using id from metadata of CoroTask timer?)

					// Ends (suspends) the CPU scoped timers.
					for (uint32 i = 0; i < TimerScopeDepth; ++i)
					{
						// If we receive mismatched end events ignore them for now.
						// This can happen for example because tracing connects to the store after events were traced. Those events can be lost.
						if (ThreadState.ScopeStack.Num() > 0)
						{
							ThreadState.ScopeStack.Pop();
							CPUPROFILER_DEBUG_LOGF(TEXT("[%u] +E=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
							ThreadState.Timeline->AppendEndEvent(ActualTime);
							CPUPROFILER_DEBUG_END_EVENT(ActualTime);
						}
					}

					// Update the "continuation" (suspended or destroyed) metadata flag.
					if (ThreadState.ScopeStack.Num() > 0)
					{
						uint32 MetadataTimerId = ThreadState.ScopeStack.Top().EventTypeId;
						TArrayView<uint8> Metadata = EditableTimingProfilerProvider.GetEditableMetadata(MetadataTimerId);
						if (ensure(Metadata.Num() > 0))
						{
							// Change the last byte in metadata to "true".
							Metadata.GetData()[Metadata.Num() - 1] = (uint8)(ECborCode::Prim | ECborCode::True);
						}
					}
				}

				// Ends the "CoroTask" scoped timer.
				{
					// If we receive mismatched end events ignore them for now.
					// This can happen for example because tracing connects to the store after events were traced. Those events can be lost.
					if (ThreadState.ScopeStack.Num() > 0)
					{
						ThreadState.ScopeStack.Pop();
						CPUPROFILER_DEBUG_LOGF(TEXT("[%u] *E=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
						ThreadState.Timeline->AppendEndEvent(ActualTime);
						CPUPROFILER_DEBUG_END_EVENT(ActualTime);
					}
				}
			}
		}
		else
		{
			if (DecodedCycle & 1ull)
			{
				uint32 SpecId = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));

				uint32 TimerId = 0;
				if (Version == 3)
				{
					if (SpecId & 1u) // The last bit is set if this is a metadata id.
					{
						// Get the actual Metadata Id.
						uint32 MetadataId = SpecId >> 1;
						uint32* TimerIdPtr = MetadataIdToTimerIdMap.Find(MetadataId);
						if (TimerIdPtr == nullptr)
						{
							constexpr uint32 MetadataUnknownSpecId = (1u << 31u) - 3u;
							if (MetadataUnknownTimerId == ~0u)
							{
								MetadataUnknownTimerId = AddCpuTimer(MetadataUnknownSpecId, TEXT("<unknown>"));
							}

							// Add an empty placeholder metadata so we obtain a MetadataId to use as the TimerId. Will be replaced with the actual metadata if the metadata event arrives later.
							TimerId = EditableTimingProfilerProvider.AddMetadata(MetadataUnknownTimerId, TArray<uint8>());
							MetadataIdToTimerIdMap.Add(MetadataId, TimerId);
						}
						else
						{
							TimerId = *TimerIdPtr;
						}
					}
					else
					{
						// Get the actual Spec Id.
						SpecId = SpecId >> 1;
						TimerId = GetOrAddTimer(SpecId);
					}
				}
				else
				{
					TimerId = GetOrAddTimer(SpecId);
				}

				FEventScopeState& ScopeState = ThreadState.ScopeStack.AddDefaulted_GetRef();
				ScopeState.StartCycle = ActualCycle;
				ScopeState.EventTypeId = TimerId;

				CPUPROFILER_DEBUG_LOGF(TEXT("[%u]  B=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
				FTimingProfilerEvent Event;
				Event.TimerIndex = TimerId;
				ThreadState.Timeline->AppendBeginEvent(ActualTime, Event);
				CPUPROFILER_DEBUG_BEGIN_EVENT(ActualTime, Event);
			}
			else
			{
				// If we receive mismatched end events ignore them for now.
				// This can happen for example because tracing connects to the store after events were traced. Those events can be lost.
				if (ThreadState.ScopeStack.Num() > 0)
				{
					ThreadState.ScopeStack.Pop();
					CPUPROFILER_DEBUG_LOGF(TEXT("[%u]  E=%llu (%.9f)\n"), ThreadState.ThreadId, ActualCycle, ActualTime);
					ThreadState.Timeline->AppendEndEvent(ActualTime);
					CPUPROFILER_DEBUG_END_EVENT(ActualTime);
				}
			}
		}

		check(ActualCycle > 0);
		LastCycle = ActualCycle;
	}
	check(BufferPtr == BufferEnd);

	if (RemainingPending == 0)
	{
		//CPUPROFILER_DEBUG_LOGF(TEXT("[%u] MetaEvents: %d added\n"), ThreadState.ThreadId, ThreadState.PendingEvents.Num());
		ThreadState.PendingEvents.Reset();
	}
	else
	{
		const int32 NumEventsToRemove = ThreadState.PendingEvents.Num() - RemainingPending;
		CPUPROFILER_DEBUG_LOGF(TEXT("[%u] MetaEvents: %d added, %d still pending\n"), ThreadState.ThreadId, NumEventsToRemove, RemainingPending);
		ThreadState.PendingEvents.RemoveAt(0, NumEventsToRemove);
	}

	ThreadState.LastCycle = LastCycle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::DispatchPendingEvents(
	uint64& LastCycle,
	uint64 CurrentCycle,
	FThreadState& ThreadState,
	const FPendingEvent*& PendingCursor,
	int32& RemainingPending,
	bool bIsBeginEvent)
{
	if (ThreadState.bShouldIgnorePendingEvents)
	{
		PendingCursor += RemainingPending;
		RemainingPending = 0;
		return;
	}

	for (; RemainingPending > 0; RemainingPending--, PendingCursor++)
	{
		bool bEnter = true;
		uint64 PendingCycle = PendingCursor->Cycle;
		if (int64(PendingCycle) < 0)
		{
			PendingCycle = ~PendingCycle;
			bEnter = false;
		}

		if ((PendingCycle > CurrentCycle) ||
			(PendingCycle == CurrentCycle && !bIsBeginEvent))
		{
			break;
		}

		if (PendingCycle < LastCycle)
		{
			// Time needs to increase monotonically.
			// We are not allowing pending events (with metadata) older than regular CPU timing events.
			// When this happens we further ignore all pending events on this thread.
			// The issue can occur in late connect trace sessions with trace protocol <= 6 (i.e. the scoped events have relative timestamps).
			ThreadState.bShouldIgnorePendingEvents = true;
			PendingCursor += RemainingPending;
			RemainingPending = 0;
			UE_LOGF(LogTraceServices, Error, "[CpuProfiler] Detected non-monotonically increasing timestamp. Further CPU timing events with metadata are ignored on thread %u.", ThreadState.ThreadId);
			break;
		}

		// Update LastCycle in order to verify time (of following pending events) increases monotonically.
		LastCycle = PendingCycle;

		double PendingTime = PendingCursor->Time;

		if (bEnter)
		{
			CPUPROFILER_DEBUG_LOGF(TEXT("[%u] >B=%llu (%.9f)\n"), ThreadState.ThreadId, PendingCycle, PendingTime);
			FTimingProfilerEvent Event;
			Event.TimerIndex = PendingCursor->TimerId;
			ThreadState.Timeline->AppendBeginEvent(PendingTime, Event);
			CPUPROFILER_DEBUG_BEGIN_EVENT(PendingTime, Event);
		}
		else
		{
			CPUPROFILER_DEBUG_LOGF(TEXT("[%u] >E=%llu (%.9f)\n"), ThreadState.ThreadId, PendingCycle, PendingTime);
			ThreadState.Timeline->AppendEndEvent(PendingTime);
			CPUPROFILER_DEBUG_END_EVENT(PendingTime);
		}
	}

	ThreadState.LastCycle = LastCycle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::DispatchRemainingPendingEvents(FThreadState& ThreadState)
{
	int32 RemainingPending = ThreadState.PendingEvents.Num();
	if (RemainingPending > 0)
	{
		uint64 LastCycle = ThreadState.LastCycle;
		const FPendingEvent* PendingCursor = ThreadState.PendingEvents.GetData();
		DispatchPendingEvents(LastCycle, ~0ull, ThreadState, PendingCursor, RemainingPending, true);
		check(RemainingPending == 0);
		ThreadState.PendingEvents.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::EndOpenEvents(FThreadState& ThreadState, double Timestamp)
{
	while (ThreadState.ScopeStack.Num())
	{
		ThreadState.ScopeStack.Pop();
		CPUPROFILER_DEBUG_LOGF(TEXT("[%u] ~E=%llu (%.9f)\n"), ThreadState.ThreadId, ThreadState.LastCycle, Timestamp);
		ThreadState.Timeline->AppendEndEvent(Timestamp);
		CPUPROFILER_DEBUG_END_EVENT(Timestamp);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::OnCpuScopeEnter(const FOnEventContext& Context)
{
	if (Context.EventTime.GetTimestamp() == 0)
	{
		return;
	}

	uint32 ThreadId = Context.ThreadInfo.GetId();
	CPUPROFILER_FILTER_BY_THREAD(ThreadId, return);
	FThreadState& ThreadState = GetOrAddThreadState(ThreadId);

	if (ThreadState.bShouldIgnorePendingEvents)
	{
		return;
	}

	uint32 SpecId = Context.EventData.GetTypeInfo().GetId();
	SpecId = ~SpecId; // to keep out of the way of normal spec IDs.

	uint32 TimerId;
	if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
	{
		TimerId = *FoundTimerIdBySpecId;
	}
	else
	{
		CPUPROFILER_LOG_API_L2(TEXT("[CpuProfiler] OnCpuScopeEnter with a new timer"));
		FString ScopeName;
		ScopeName += Context.EventData.GetTypeInfo().GetName();
		TimerId = DefineUniqueTimer(SpecId, *ScopeName, nullptr, 0);
	}

	TArray<uint8> CborData;
	Context.EventData.SerializeToCbor(CborData);
	if (ensure(CborData.Num() > 0))
	{
		FAnalysisSessionEditScope _(Session);
		TimerId = EditableTimingProfilerProvider.AddMetadata(TimerId, MoveTemp(CborData));
	}

	uint64 Cycle = Context.EventTime.AsCycle64();
	double Time = Context.EventTime.AsSeconds();

	if (ensure(Cycle >= ThreadState.LastCycle))
	{
		check(ThreadState.LastPendingEventTime <= Time);
		ThreadState.LastPendingEventTime = Time;
		ThreadState.PendingEvents.Add({ Cycle, Time, TimerId });
	}
	else
	{
		// Time needs to increase monotonically.
		// We are not allowing pending events (with metadata) older than regular CPU timing events.
		// When this happens we further ignore all pending events on this thread.
		// The issue can occur in late connect trace sessions with trace protocol <= 6 (i.e. the scoped events have relative timestamps).
		ThreadState.bShouldIgnorePendingEvents = true;
		UE_LOGF(LogTraceServices, Error, "[CpuProfiler] Detected scope enter event non-monotonically increasing timestamp. Further CPU timing events with metadata are ignored on thread %u.", ThreadState.ThreadId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::OnCpuScopeLeave(const FOnEventContext& Context)
{
	if (Context.EventTime.GetTimestamp() == 0)
	{
		return;
	}

	uint32 ThreadId = Context.ThreadInfo.GetId();
	CPUPROFILER_FILTER_BY_THREAD(ThreadId, return);
	FThreadState& ThreadState = GetOrAddThreadState(ThreadId);

	if (ThreadState.bShouldIgnorePendingEvents)
	{
		return;
	}

	uint64 Cycle = Context.EventTime.AsCycle64();
	double Time = Context.EventTime.AsSeconds();

	if (ensure(Cycle >= ThreadState.LastCycle))
	{
		check(ThreadState.LastPendingEventTime <= Time);
		ThreadState.LastPendingEventTime = Time;
		ThreadState.PendingEvents.Add({ ~Cycle, Time, 0 });
	}
	else
	{
		// Time needs to increase monotonically.
		// We are not allowing pending events (with metadata) older than regular CPU timing events.
		// When this happens we further ignore all pending events on this thread.
		// The issue can occur in late connect trace sessions with trace protocol <= 6 (i.e. the scoped events have relative timestamps).
		ThreadState.bShouldIgnorePendingEvents = true;
		UE_LOGF(LogTraceServices, Error, "[CpuProfiler] Detected scope leave event with non-monotonically increasing timestamp. Further CPU timing events with metadata are ignored on thread %u.", ThreadState.ThreadId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::OnEventSpec(const FOnEventContext& Context)
{
	const auto& EventData = Context.EventData;

	uint32 SpecId = EventData.GetValue<uint32>("Id");

	const TCHAR* TimerName = nullptr;
	FString Name;
	if (EventData.GetString("Name", Name))
	{
		TimerName = *Name;
	}
	else
	{
		uint8 CharSize = EventData.GetValue<uint8>("CharSize");
		if (CharSize == sizeof(ANSICHAR))
		{
			const ANSICHAR* AnsiName = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
			Name = StringCast<TCHAR>(AnsiName).Get();
			TimerName = *Name;
		}
		else if (CharSize == 0 || CharSize == sizeof(TCHAR)) // 0 for backwards compatibility
		{
			TimerName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
		}
		else
		{
			Name = FString::Printf(TEXT("<invalid %u>"), SpecId);
			TimerName = *Name;
		}
	}

	if (TimerName[0] == 0)
	{
		Name = FString::Printf(TEXT("<noname %u>"), SpecId);
		TimerName = *Name;
	}

	FString File;
	uint32 Line = 0;
	if (EventData.GetString("File", File) && !File.IsEmpty())
	{
		Line = EventData.GetValue<uint32>("Line");
	}
	const TCHAR* FileName = !File.IsEmpty() ? *File : nullptr;

	CPUPROFILER_LOG_API_L2(TEXT("[CpuProfiler] EventSpec Id=%u Name=\"%ls\" File=\"%ls\" Line=%u"),
		SpecId, TimerName, FileName ? FileName : TEXT("N/A"), Line);

	const TCHAR* StoredTimerName = Session.StoreString(TimerName);
	DefineMergedTimer(SpecId, StoredTimerName, FileName, Line);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::OnMetadataSpec(const FOnEventContext& Context)
{
	const auto& EventData = Context.EventData;

	uint32 SpecId = EventData.GetValue<uint32>("Id");

	FString Name, NameFormat;
	EventData.GetString("Name", Name);
	EventData.GetString("NameFormat", NameFormat);
	TArrayView<const uint8> FieldNames = EventData.GetArrayView<uint8>("FieldNames");

	CPUPROFILER_LOG_API_L2(TEXT("[CpuProfiler] MetadataSpec Id=%u Name=\"%ls\" NameFormat=\"%ls\" Fields[]={%u bytes}"),
		SpecId, *Name, *NameFormat, FieldNames.Num());

	if (Name.Compare(NameFormat) == 0)
	{
		Name.Empty();
	}

	int Index = 0;
	while (Index < NameFormat.Len() && Index < Name.Len())
	{
		if (Name[Index] != NameFormat[Index])
		{
			break;
		}

		++Index;
	}

	if (Index > 1)
	{
		NameFormat.MidInline(Index);
	}

	if (Name.IsEmpty())
	{
		const FString Separators = TEXT("% (=");
		Index = 0;
		bool bIsInFormatSpecifier = false;
		while (Index < NameFormat.Len())
		{
			int32 SpecIndex = -1;
			if (Separators.FindChar(NameFormat[Index], SpecIndex))
			{
				Name = NameFormat.Left(Index);
				NameFormat.MidInline(Index);
				break;
			}

			++Index;
		}

		if (Name.IsEmpty())
		{
			if (!NameFormat.IsEmpty())
			{
				Name = NameFormat;
				NameFormat.Empty();
			}
			else
			{
				Name = Session.StoreString(TEXT("Unknown"));
			}
		}
	}

	FMetadataSpec Spec;
	if (!NameFormat.IsEmpty())
	{
		Spec.Format = Session.StoreString(NameFormat);
	}

	if (FieldNames.Num() > 0)
	{
		FMemoryReaderView MemoryReader(FieldNames);
		FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
		FCborContext CborContext;

		while (CborReader.ReadNext(CborContext))
		{
			if (CborContext.MajorType() == ECborCode::TextString)
			{
				FString Field = CborContext.AsString();
				Spec.FieldNames.Add(Session.StoreString(Field));
			}
		}
	}

	uint32 TimerId;
	if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
	{
		TimerId = *FoundTimerIdBySpecId;

		FAnalysisSessionEditScope _(Session);
		SetTimerName(SpecId, TimerId, *Name);
	}
	else
	{
		CPUPROFILER_LOG_API_L2(TEXT("[CpuProfiler] MetadataSpec with a new timer"));
		TimerId = DefineUniqueTimer(SpecId, *Name, nullptr, 0);
	}

	if (Spec.FieldNames.Num() > 0 || Spec.Format != nullptr)
	{
		FAnalysisSessionEditScope _(Session);

		uint32 MetadataSpecId = EditableTimingProfilerProvider.AddMetadataSpec(MoveTemp(Spec));

		const ITimingProfilerProvider* TimingProfilerProvider = EditableTimingProfilerProvider.GetReadProvider();
		if (TimingProfilerProvider)
		{
			const ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();
			const FTimingProfilerTimer* PreviousTimer = TimerReader.GetTimer(TimerId);
			if (PreviousTimer && PreviousTimer->HasValidMetadataSpecId())
			{
				TimerId = AddCpuTimer(SpecId, *Name, PreviousTimer->File, PreviousTimer->Line);

				if (++NumTimerWarnings < NumMaxWarnings)
				{
					const FMetadataSpec* PreviousMetadataSpec = TimingProfilerProvider->GetMetadataSpec(PreviousTimer->MetadataSpecId);
					check(PreviousMetadataSpec);
					UE_LOGF(LogTraceServices, Warning, "[CpuProfiler] MetadataSpec -- de-duplicated timer for SpecId %u -- old: %u (%ls) metadata=\"%ls\" (%d fields) --> new: %u (%ls) metadata=\"%ls\" (%d fields) !!!",
						SpecId,
						PreviousTimer->Id, PreviousTimer->Name, PreviousMetadataSpec->Format, PreviousMetadataSpec->FieldNames.Num(),
						TimerId, *Name, Spec.Format, Spec.FieldNames.Num());
				}
			}
		}

		EditableTimingProfilerProvider.SetMetadataSpec(TimerId, MetadataSpecId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FCpuProfilerAnalyzer::GetOrAddTimer(uint32 SpecId)
{
	if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
	{
		return *FoundTimerIdBySpecId;
	}

	// Add a timer with an "unknown" name.
	// The "unknown" timers are not merged by name, because the actual name
	// might be updated when an EventSpec event is received (for this SpecId).
	FAnalysisSessionEditScope _(Session);
	return AddCpuTimer(SpecId, *FString::Printf(TEXT("<unknown %u>"), SpecId));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FCpuProfilerAnalyzer::AddCpuTimer(uint32 SpecId, const TCHAR* TimerName, const TCHAR* File, uint32 Line)
{
	Session.WriteAccessCheck();

	// Add a new CPU timer.
	uint32 TimerId = EditableTimingProfilerProvider.AddCpuTimer(TimerName, File, Line);

	// Map the SpecId to the timer.
	SpecIdToTimerIdMap.Add(SpecId, TimerId);

	CPUPROFILER_LOG_API_L3(TEXT("[CpuProfiler] --> AddCpuTimer(SpecId=%u, Name=\"%ls\") --> TimerId=%u"), SpecId, TimerName, TimerId);

	return TimerId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FCpuProfilerAnalyzer::DefineMergedTimer(uint32 SpecId, const TCHAR* StoredTimerName, const TCHAR* File, uint32 Line)
{
	// Expected: StoredTimerName is already a pointer in a string store.

	// The CPU scoped events (timers) can be merged by name.
	// If there are multiple timers defined in code with same name,
	// those will appear in Insights as a single timer.

	uint32 TimerId;

	// Check if a timer with same name was already defined.
	FTimerInfo* FoundTimerIdByName = ScopeNameToTimerIdMap.Find((uint64)StoredTimerName);
	if (FoundTimerIdByName)
	{
		// Yes, a timer with same name was already defined.
		++(FoundTimerIdByName->Count);

		// Check if SpecId is already mapped to timer.
		if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
		{
			// Yes, SpecId was already mapped to a timer (ex. as an <unknown> timer).
			TimerId = *FoundTimerIdBySpecId;

			// Update name for mapped timer.
			FAnalysisSessionEditScope _(Session);
			SetTimerName(SpecId, TimerId, StoredTimerName);
			EditableTimingProfilerProvider.SetTimerLocation(TimerId, File, Line);

			// In this case, we do not remap the SpecId to the previously defined timer with same name.
			// This is because the two timers are already used in timelines.
			// So we will continue to use separate timers, even if those have same name.
		}
		else
		{
			// Use the previously defined timer with same name.
			TimerId = FoundTimerIdByName->Id;

			// Map this SpecId to the previously defined timer with same name.
			SpecIdToTimerIdMap.Add(SpecId, TimerId);
		}

		return TimerId;
	}
	else // No, this is the first timer with this name.
	{
		// Create a new timer.
		TimerId = DefineUniqueTimer(SpecId, StoredTimerName, File, Line);

		// Map the name to the timer.
		ScopeNameToTimerIdMap.Add((uint64)StoredTimerName, { TimerId, 1 });
	}

	return TimerId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FCpuProfilerAnalyzer::DefineUniqueTimer(uint32 SpecId, const TCHAR* TimerName, const TCHAR* File, uint32 Line)
{
	uint32 TimerId;

	// Check if SpecId is already mapped to timer.
	if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
	{
		// Yes, SpecId was already mapped to a timer (ex. as an <unknown> timer).
		TimerId = *FoundTimerIdBySpecId;

		// Update name for the mapped timer.
		FAnalysisSessionEditScope _(Session);
		SetTimerName(SpecId, TimerId, TimerName);
		EditableTimingProfilerProvider.SetTimerLocation(TimerId, File, Line);
	}
	else
	{
		// Define a new CPU timer.
		FAnalysisSessionEditScope _(Session);
		TimerId = AddCpuTimer(SpecId, TimerName, File, Line);
	}

	return TimerId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FCpuProfilerAnalyzer::GetTimerName(uint32 TimerId) const
{
	const ITimingProfilerProvider* TimingProfilerProvider = EditableTimingProfilerProvider.GetReadProvider();
	if (TimingProfilerProvider)
	{
		const ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();
		const FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerId);
		if (Timer)
		{
			return Timer->Name;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerAnalyzer::SetTimerName(uint32 SpecId, uint32 TimerId, const TCHAR* TimerName)
{
#if CPUPROFILER_DETECT_TIMER_NAME_CHANGE
	const TCHAR* PrevTimerName = GetTimerName(TimerId);
#endif

	Session.WriteAccessCheck();
	EditableTimingProfilerProvider.SetTimerName(TimerId, TimerName);

#if CPUPROFILER_DETECT_TIMER_NAME_CHANGE
	const TCHAR* NewTimerName = GetTimerName(TimerId);
	if (PrevTimerName != nullptr && PrevTimerName != NewTimerName && PrevTimerName[0] != TEXT('<'))
	{
		UE_LOGF(LogTraceServices, Warning, "[CpuProfiler] --> SetTimerName(SpecId=%u, TimerId=%u, Name=\"%ls\") : name was \"%ls\" !!!", SpecId, TimerId, TimerName, PrevTimerName);
	}
	else
	{
		CPUPROFILER_LOG_API_L2(TEXT("[CpuProfiler] --> SetTimerName(SpecId=%u, TimerId=%u, Name=\"%ls\""), SpecId, TimerId, TimerName);
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCpuProfilerAnalyzer::FThreadState& FCpuProfilerAnalyzer::GetOrAddThreadState(uint32 ThreadId)
{
	FThreadState* ThreadState = ThreadStatesMap.FindRef(ThreadId);
	if (!ThreadState)
	{
		FAnalysisSessionEditScope _(Session);

		ThreadState = new FThreadState();
		ThreadState->ThreadId = ThreadId;
		ThreadState->Timeline = &EditableTimingProfilerProvider.GetCpuThreadEditableTimeline(ThreadId);
		ThreadStatesMap.Add(ThreadId, ThreadState);

		// Just in case the rest of Insight's reporting/analysis doesn't know about
		// this thread, we'll explicitly add it. For fault tolerance.
		EditableThreadProvider.AddThread(ThreadId, nullptr, TPri_Normal);
	}
	return *ThreadState;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef CPUPROFILER_FILTER_BY_THREAD
#undef CPUPROFILER_DETECT_TIMER_NAME_CHANGE
#undef CPUPROFILER_LOG_API_L3
#undef CPUPROFILER_LOG_API_L2
#undef CPUPROFILER_LOG_API_L1
#undef CPUPROFILER_DEBUG_LOGF
#undef CPUPROFILER_DEBUG_BEGIN_EVENT
#undef CPUPROFILER_DEBUG_END_EVENT
