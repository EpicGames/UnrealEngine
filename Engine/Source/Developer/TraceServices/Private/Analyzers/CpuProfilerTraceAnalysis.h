// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"

// TraceAnalysis
#include "Trace/Analyzer.h"

// TraceServices
#include "Model/MonotonicTimeline.h"
#include "TraceServices/Model/TimingProfiler.h"

namespace TraceServices
{

class IAnalysisSession;
class IEditableTimingProfilerProvider;
class IEditableThreadProvider;

class FCpuProfilerAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FCpuProfilerAnalyzer(IAnalysisSession& Session, IEditableTimingProfilerProvider& InEditableTimingProfilerProvider, IEditableThreadProvider& InEditableThreadProvider);
	~FCpuProfilerAnalyzer();
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd(/*const FOnAnalysisEndContext& Context*/) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	struct FEventScopeState
	{
		uint64 StartCycle;
		uint32 EventTypeId;
	};

	struct FPendingEvent
	{
		uint64 Cycle;
		double Time;
		uint32 TimerId;
	};

	struct FThreadState
	{
		uint32 ThreadId = 0;
		TArray<FEventScopeState> ScopeStack;
		TArray<FPendingEvent> PendingEvents;
		IEditableTimeline<FTimingProfilerEvent>* Timeline = nullptr;
		uint64 LastCycle = 0;
		bool bShouldIgnorePendingEvents = false; // becomes true when we detect first pending event with incorrect timestamp (i.e < LastCycle)
		double LastPendingEventTime = 0;
		int32 MetadataEventsDepth = 0;
	};

	struct FTimerInfo
	{
		uint32 Id;
		uint32 Count;
	};

	enum : uint16
	{
		RouteId_EventSpec,
		RouteId_EndThread,
		RouteId_EventBatchV3,
		RouteId_EventBatchV2, // backward compatibility
		RouteId_EventBatch, // backward compatibility
		RouteId_EndCapture, // backward compatibility
		RouteId_CpuScope,
		RouteId_MetadataSpec,
		RouteId_Metadata,
	};

	void ProcessBuffer(const FEventTime& EventTime, FThreadState& ThreadState, const uint8* BufferPtr, uint32 BufferSize);
	void ProcessBufferV2(const FEventTime& EventTime, FThreadState& ThreadState, const uint8* BufferPtr, uint32 BufferSize, int32 Version);
	void DispatchPendingEvents(uint64& LastCycle, uint64 CurrentCycle, FThreadState& ThreadState, const FPendingEvent*& PendingCursor, int32& RemainingPending, bool bIsBeginEvent);
	void DispatchRemainingPendingEvents(FThreadState& ThreadState);
	void EndOpenEvents(FThreadState& ThreadState, double Timestamp);
	void OnCpuScopeEnter(const FOnEventContext& Context);
	void OnCpuScopeLeave(const FOnEventContext& Context);
	void OnEventSpec(const FOnEventContext& Context);
	void OnMetadataSpec(const FOnEventContext& Context);

	uint32 GetOrAddTimer(uint32 SpecId);
	uint32 AddCpuTimer(uint32 SpecId, const TCHAR* TimerName, const TCHAR* File = nullptr, uint32 Line = 0); // returns the TimerId
	uint32 DefineMergedTimer(uint32 SpecId, const TCHAR* TimerName, const TCHAR* File, uint32 Line); // returns the TimerId
	uint32 DefineUniqueTimer(uint32 SpecId, const TCHAR* TimerName, const TCHAR* File, uint32 Line); // returns the TimerId
	const TCHAR* GetTimerName(uint32 TimerId) const;
	void SetTimerName(uint32 SpecId, uint32 TimerId, const TCHAR* TimerName);

	FThreadState& GetOrAddThreadState(uint32 ThreadId);

private:
	IAnalysisSession& Session;
	IEditableTimingProfilerProvider& EditableTimingProfilerProvider;
	IEditableThreadProvider& EditableThreadProvider;

	TMap<uint32, FThreadState*> ThreadStatesMap;
	TMap<uint32, uint32> SpecIdToTimerIdMap; // SpecId --> TimerId
	TMap<uint64, FTimerInfo> ScopeNameToTimerIdMap; // (uint64)Name --> FTimerInfo
	TMap<uint32, uint32> MetadataIdToTimerIdMap; // MetadataId --> TimerId

	uint32 CoroutineTimerId = ~0;
	uint32 CoroutineUnknownTimerId = ~0;
	uint32 MetadataUnknownTimerId = ~0;
	uint64 TotalEventSize = 0;
	uint64 TotalScopeCount = 0;
	double BytesPerScope = 0.0;

	uint32 NumTimerWarnings = 0;
	static constexpr uint32 NumMaxWarnings = 100;
};

} // namespace TraceServices
