// Copyright Epic Games, Inc. All Rights Reserved.

#include "CsvProfilerTraceAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/Threads.h"

namespace TraceServices
{

FCsvProfilerAnalyzer::FCsvProfilerAnalyzer(IAnalysisSession& InSession, FCsvProfilerProvider& InCsvProfilerProvider, IEditableCounterProvider& InEditableCounterProvider, const IFrameProvider& InFrameProvider, const IThreadProvider& InThreadProvider)
	: Session(InSession)
	, CsvProfilerProvider(InCsvProfilerProvider)
	, EditableCounterProvider(InEditableCounterProvider)
	, FrameProvider(InFrameProvider)
	, ThreadProvider(InThreadProvider)
{
}

FCsvProfilerAnalyzer::~FCsvProfilerAnalyzer()
{
	OnAnalysisEnd();
}

void FCsvProfilerAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_RegisterCategory, "CsvProfiler", "RegisterCategory");
	Builder.RouteEvent(RouteId_DefineInlineStat, "CsvProfiler", "DefineInlineStat");
	Builder.RouteEvent(RouteId_DefineDeclaredStat, "CsvProfiler", "DefineDeclaredStat");
	Builder.RouteEvent(RouteId_BeginStat, "CsvProfiler", "BeginStat");
	Builder.RouteEvent(RouteId_EndStat, "CsvProfiler", "EndStat");
	Builder.RouteEvent(RouteId_BeginExclusiveStat, "CsvProfiler", "BeginExclusiveStat");
	Builder.RouteEvent(RouteId_EndExclusiveStat, "CsvProfiler", "EndExclusiveStat");
	Builder.RouteEvent(RouteId_CustomStatInt, "CsvProfiler", "CustomStatInt");
	Builder.RouteEvent(RouteId_CustomStatFloat, "CsvProfiler", "CustomStatFloat");
	Builder.RouteEvent(RouteId_Event, "CsvProfiler", "Event");
	Builder.RouteEvent(RouteId_Metadata, "CsvProfiler", "Metadata");
	Builder.RouteEvent(RouteId_BeginCapture, "CsvProfiler", "BeginCapture");
	Builder.RouteEvent(RouteId_EndCapture, "CsvProfiler", "EndCapture");
}

void FCsvProfilerAnalyzer::OnAnalysisEnd()
{
	for (FStatSeriesInstance* StatSeriesInstance : StatSeriesInstanceArray)
	{
		delete StatSeriesInstance;
	}
	StatSeriesInstanceArray.Empty();
	for (FStatSeriesDefinition* StatSeriesDefinition : StatSeriesDefinitionArray)
	{
		delete StatSeriesDefinition;
	}
	StatSeriesDefinitionArray.Empty();
	StatSeriesMap.Empty();
	StatSeriesStringMap.Empty();
	for (auto& KV : ThreadStatesMap)
	{
		FThreadState* ThreadState = KV.Value;
		delete ThreadState;
	}
	ThreadStatesMap.Empty();
}

bool FCsvProfilerAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FCsvProfilerAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_RegisterCategory:
	{
		int32 CategoryIndex = EventData.GetValue<int32>("Index");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		CategoryMap.Add(CategoryIndex, Session.StoreString(*Name));;
		break;
	}
	case RouteId_DefineInlineStat:
	{
		uint64 StatId = EventData.GetValue<uint64>("StatId");
		int32 CategoryIndex = EventData.GetValue<int32>("CategoryIndex");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("Name", Context);
		DefineStatSeries(StatId, *Name, CategoryIndex, true);
		break;
	}
	case RouteId_DefineDeclaredStat:
	{
		uint64 StatId = EventData.GetValue<uint64>("StatId");
		int32 CategoryIndex = EventData.GetValue<int32>("CategoryIndex");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		DefineStatSeries(StatId, *Name, CategoryIndex, false);
		break;
	}
	case RouteId_BeginStat:
	{
		HandleMarkerEvent(Context, false, true);
		break;
	}
	case RouteId_EndStat:
	{
		HandleMarkerEvent(Context, false, false);
		break;
	}
	case RouteId_BeginExclusiveStat:
	{
		HandleMarkerEvent(Context, true, true);
		break;
	}
	case RouteId_EndExclusiveStat:
	{
		HandleMarkerEvent(Context, true, false);
		break;
	}
	case RouteId_CustomStatInt:
	{
		HandleCustomStatEvent(Context, false);
		break;
	}
	case RouteId_CustomStatFloat:
	{
		HandleCustomStatEvent(Context, true);
		break;
	}
	case RouteId_Event:
	{
		HandleEventEvent(Context);
		break;
	}
	case RouteId_Metadata:
	{
		FString Key, Value;
		if (EventData.GetString("Key", Key))
		{
			EventData.GetString("Value", Value);
		}
		else
		{
			Key = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			Value = reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + EventData.GetValue<uint16>("ValueOffset"));
		}
		CsvProfilerProvider.SetMetadata(Session.StoreString(*Key), Session.StoreString(*Value));
		break;
	}
	case RouteId_BeginCapture:
	{
		RenderThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context, "RenderThreadId");
		RHIThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context, "RHIThreadId");
		uint32 CaptureStartFrame = FrameProvider.GetFrameNumberForTimestamp(TraceFrameType_Game, Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")));
		bEnableCounts = EventData.GetValue<bool>("EnableCounts");
		FString FileName = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("FileName", Context);
		const TCHAR* StoredFileName = Session.StoreString(*FileName);
		CsvProfilerProvider.StartCapture(StoredFileName, CaptureStartFrame);
		break;
	}
	case RouteId_EndCapture:
	{
		uint32 CaptureEndFrame = FrameProvider.GetFrameNumberForTimestamp(TraceFrameType_Game, Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")));
		for (FStatSeriesInstance* StatSeries : StatSeriesInstanceArray)
		{
			FlushAtEndOfCapture(*StatSeries, CaptureEndFrame);
		}

		CsvProfilerProvider.EndCapture(CaptureEndFrame);
	}
	}

	return true;
}

FCsvProfilerAnalyzer::FThreadState& FCsvProfilerAnalyzer::GetThreadState(uint32 ThreadId)
{
	FThreadState** FindIt = ThreadStatesMap.Find(ThreadId);
	if (FindIt)
	{
		return **FindIt;
	}
	FThreadState* ThreadState = new FThreadState();
	if (ThreadId == RenderThreadId || ThreadId == RHIThreadId)
	{
		ThreadState->FrameType = TraceFrameType_Rendering;
	}
	ThreadState->ThreadName = ThreadId == RenderThreadId ? TEXT("RenderThread") : ThreadProvider.GetThreadName(ThreadId);
	ThreadStatesMap.Add(ThreadId, ThreadState);
	return *ThreadState;
}

FCsvProfilerAnalyzer::FStatSeriesDefinition* FCsvProfilerAnalyzer::CreateStatSeries(const TCHAR* Name, int32 CategoryIndex)
{
	FStatSeriesDefinition* StatSeries = new FStatSeriesDefinition();
	StatSeries->Name = Session.StoreString(Name);
	StatSeries->CategoryIndex = CategoryIndex;
	StatSeries->ColumnIndex = StatSeriesDefinitionArray.Num();
	StatSeriesDefinitionArray.Add(StatSeries);
	return StatSeries;
}

void FCsvProfilerAnalyzer::DefineStatSeries(uint64 StatId, const TCHAR* Name, int32 CategoryIndex, bool bIsInline)
{
	FStatSeriesDefinition** FindIt = StatSeriesMap.Find(StatId);
	if (bIsInline && !FindIt)
	{
		TTuple<int32, FString> Key = TTuple<int32, FString>(CategoryIndex, Name);
		FindIt = StatSeriesStringMap.Find(Key);
		if (FindIt)
		{
			StatSeriesMap.Add(StatId, *FindIt);
		}
	}
	if (!FindIt)
	{
		FStatSeriesDefinition* StatSeries = CreateStatSeries(Name, CategoryIndex);
		StatSeriesMap.Add(StatId, StatSeries);
		if (bIsInline)
		{
			TTuple<int32, FString> Key = TTuple<int32, FString>(CategoryIndex, Name);
			StatSeriesStringMap.Add(Key, StatSeries);
		}
	}
}

const TCHAR* FCsvProfilerAnalyzer::GetStatSeriesName(const FStatSeriesDefinition* Definition, ECsvStatSeriesType Type, FThreadState& ThreadState, bool bIsCount)
{
	FString Name = Definition->Name;
	if (Type == CsvStatSeriesType_Timer || bIsCount)
	{
		// Add a /<Threadname> prefix
		Name = ThreadState.ThreadName + TEXT("/") + Name;
	}

	if (Definition->CategoryIndex > 0)
	{
		// Categorized stats are prefixed with <CATEGORY>/
		Name = FString(CategoryMap[Definition->CategoryIndex]) + TEXT("/") + Name;
	}

	if (bIsCount)
	{
		// Add a counts prefix
		Name = TEXT("COUNTS/") + Name;
	}

	if (Name.IsEmpty())
	{
		UE_LOGF(LogTraceServices, Warning, "Invalid counter name for CSV column %d.", Definition->ColumnIndex);
		Name = FString::Printf(TEXT("<noname CSV column %d>"), Definition->ColumnIndex);
	}

	return Session.StoreString(*Name);
}

FCsvProfilerAnalyzer::FStatSeriesInstance& FCsvProfilerAnalyzer::GetStatSeries(uint64 StatId, ECsvStatSeriesType Type, FThreadState& ThreadState)
{
	FStatSeriesDefinition* Definition;
	FStatSeriesDefinition** FindIt = StatSeriesMap.Find(StatId);
	if (!FindIt)
	{
		Definition = CreateStatSeries(*FString::Printf(TEXT("[unknown%d]"), UndefinedStatSeriesCount++), 0);
		StatSeriesMap.Add(StatId, Definition);
	}
	else
	{
		Definition = *FindIt;
	}

	if (ThreadState.StatSeries.Num() <= Definition->ColumnIndex)
	{
		ThreadState.StatSeries.AddZeroed(Definition->ColumnIndex + 1 - ThreadState.StatSeries.Num());
	}
	FStatSeriesInstance* Instance = ThreadState.StatSeries[Definition->ColumnIndex];
	if (Instance)
	{
		return *Instance;
	}

	Instance = new FStatSeriesInstance();
	StatSeriesInstanceArray.Add(Instance);
	ThreadState.StatSeries[Definition->ColumnIndex] = Instance;
	const TCHAR* StatSeriesName = GetStatSeriesName(Definition, Type, ThreadState, false);
	Instance->ProviderHandle = CsvProfilerProvider.AddSeries(StatSeriesName, Type);
	Instance->ProviderCountHandle = CsvProfilerProvider.AddSeries(GetStatSeriesName(Definition, Type, ThreadState, true), CsvStatSeriesType_CustomStatInt);
	Instance->Counter = EditableCounterProvider.CreateEditableCounter();
	Instance->Counter->SetName(StatSeriesName);
	Instance->Counter->SetIsFloatingPoint(Type != CsvStatSeriesType_CustomStatInt);
	Instance->Type = Type;
	Instance->FrameType = ThreadState.FrameType;

	return *Instance;
}

void FCsvProfilerAnalyzer::HandleMarkerEvent(const FOnEventContext& Context, bool bIsExclusive, bool bIsBegin)
{
	uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
	FThreadState& ThreadState = GetThreadState(ThreadId);
	uint64 StatId = Context.EventData.GetValue<uint64>("StatId");
	FTimingMarker Marker;
	Marker.StatId = StatId;
	Marker.bIsBegin = bIsBegin;
	Marker.bIsExclusive = bIsExclusive;
	Marker.Cycle = Context.EventData.GetValue<uint64>("Cycle");
	HandleMarker(Context, ThreadState, Marker);
}

void FCsvProfilerAnalyzer::HandleMarker(const FOnEventContext& Context, FThreadState& ThreadState, const FTimingMarker& Marker)
{
	// Handle exclusive markers. This may insert an additional marker before this one
	bool bInsertExtraMarker = false;
	FTimingMarker InsertedMarker;
	if (Marker.bIsExclusive & !Marker.bIsExclusiveInsertedMarker)
	{
		if (Marker.bIsBegin)
		{
			if (ThreadState.ExclusiveMarkerStack.Num() > 0)
			{
				// Insert an artificial end marker to end the previous marker on the stack at the same timestamp
				InsertedMarker = ThreadState.ExclusiveMarkerStack.Last();
				InsertedMarker.bIsBegin = false;
				InsertedMarker.bIsExclusiveInsertedMarker = true;
				InsertedMarker.Cycle = Marker.Cycle;

				bInsertExtraMarker = true;
			}
			ThreadState.ExclusiveMarkerStack.Add(Marker);
		}
		else
		{
			if (ThreadState.ExclusiveMarkerStack.Num() > 0)
			{
				ThreadState.ExclusiveMarkerStack.Pop(EAllowShrinking::No);
				if (ThreadState.ExclusiveMarkerStack.Num() > 0)
				{
					// Insert an artificial begin marker to resume the marker on the stack at the same timestamp
					InsertedMarker = ThreadState.ExclusiveMarkerStack.Last();
					InsertedMarker.bIsBegin = true;
					InsertedMarker.bIsExclusiveInsertedMarker = true;
					InsertedMarker.Cycle = Marker.Cycle;

					bInsertExtraMarker = true;
				}
			}
		}
	}
	if (bInsertExtraMarker)
	{
		HandleMarker(Context, ThreadState, InsertedMarker);
	}

	double Timestamp = Context.EventTime.AsSeconds(Marker.Cycle);
	uint32 FrameNumber = FrameProvider.GetFrameNumberForTimestamp(ThreadState.FrameType, Timestamp);
	if (Marker.bIsBegin)
	{
		ThreadState.MarkerStack.Push(Marker);
	}
	else
	{
		// Markers might not match up if they were truncated mid-frame, so we need to be robust to that
		if (ThreadState.MarkerStack.Num() > 0)
		{
			// Find the start marker (might not actually be top of the stack, e.g if begin/end for two overlapping stats are independent)
			bool bFoundStart = false;
			FTimingMarker StartMarker;

			for (int j = ThreadState.MarkerStack.Num() - 1; j >= 0; j--)
			{
				if (ThreadState.MarkerStack[j].StatId == Marker.StatId) // Note: only works with scopes!
				{
					StartMarker = ThreadState.MarkerStack[j];
					ThreadState.MarkerStack.RemoveAt(j, EAllowShrinking::No);
					bFoundStart = true;
					break;
				}
			}

			// TODO: if bFoundStart is false, this stat _never_ gets processed. Could we add it to a persistent list so it's considered next time?
			// Example where this could go wrong: staggered/overlapping exclusive stats ( e.g Abegin, Bbegin, AEnd, BEnd ), where processing ends after AEnd
			// AEnd would be missing
			if (bFoundStart)
			{
				check(Marker.StatId == StartMarker.StatId);
				check(Marker.Cycle >= StartMarker.Cycle);
				if (Marker.Cycle > StartMarker.Cycle)
				{
					const FEventTime& EventTime = Context.EventTime;
					double Elapsed = EventTime.AsSeconds(Marker.Cycle) - EventTime.AsSeconds(StartMarker.Cycle);
					FStatSeriesInstance& StatSeries = GetStatSeries(Marker.StatId, CsvStatSeriesType_Timer, ThreadState);
					SetTimerValue(StatSeries, FrameNumber, Elapsed * 1000.0, !Marker.bIsExclusiveInsertedMarker);
				}
			}
		}
	}
}

void FCsvProfilerAnalyzer::HandleCustomStatEvent(const FOnEventContext& Context, bool bIsFloat)
{
	uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
	FThreadState& ThreadState = GetThreadState(ThreadId);
	FStatSeriesInstance& StatSeries = GetStatSeries(Context.EventData.GetValue<uint64>("StatId"), bIsFloat ? CsvStatSeriesType_CustomStatFloat : CsvStatSeriesType_CustomStatInt, ThreadState);
	ECsvOpType OpType = static_cast<ECsvOpType>(Context.EventData.GetValue<uint8>("OpType"));
	uint32 FrameNumber = FrameProvider.GetFrameNumberForTimestamp(ThreadState.FrameType, Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Cycle")));
	if (bIsFloat)
	{
		float Value = Context.EventData.GetValue<float>("Value");
		SetCustomStatValue(StatSeries, FrameNumber, OpType, Value);
	}
	else
	{
		int32 Value = Context.EventData.GetValue<int32>("Value");
		SetCustomStatValue(StatSeries, FrameNumber, OpType, Value);
	}
}

void FCsvProfilerAnalyzer::HandleEventEvent(const FOnEventContext& Context)
{
	uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
	FThreadState& ThreadState = GetThreadState(ThreadId);
	uint64 Cycle = Context.EventData.GetValue<uint64>("Cycle");
	uint32 FrameNumber = FrameProvider.GetFrameNumberForTimestamp(ThreadState.FrameType, Context.EventTime.AsSeconds(Cycle));
	FString EventText = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Text", Context);
	int32 CategoryIndex = Context.EventData.GetValue<int32>("CategoryIndex");
	if (CategoryIndex > 0)
	{
		EventText = FString(CategoryMap[CategoryIndex]) + TEXT("/") + EventText;
	}
	CsvProfilerProvider.AddEvent(FrameNumber, Session.StoreString(*EventText));
}

void FCsvProfilerAnalyzer::Flush(FStatSeriesInstance& StatSeries)
{
	double CounterTimestamp;
	if (StatSeries.CurrentFrame == 0)
	{
		const FFrame* Frame = FrameProvider.GetFrame(StatSeries.FrameType, 0);
		check(Frame);
		CounterTimestamp = Frame->StartTime;
	}
	else
	{
		const FFrame* Frame = FrameProvider.GetFrame(StatSeries.FrameType, StatSeries.CurrentFrame - 1);
		const FFrame* NextFrame = FrameProvider.GetFrame(StatSeries.FrameType, StatSeries.CurrentFrame);
		check(NextFrame);
		CounterTimestamp = Frame->EndTime;
	}
	if (StatSeries.Type == CsvStatSeriesType_CustomStatInt)
	{
		CsvProfilerProvider.SetValue(StatSeries.ProviderHandle, StatSeries.CurrentFrame, StatSeries.CurrentValue.Value.AsInt);
		StatSeries.Counter->SetValue(CounterTimestamp, StatSeries.CurrentValue.Value.AsInt);
	}
	else
	{
		CsvProfilerProvider.SetValue(StatSeries.ProviderHandle, StatSeries.CurrentFrame, StatSeries.CurrentValue.Value.AsDouble);
		StatSeries.Counter->SetValue(CounterTimestamp, StatSeries.CurrentValue.Value.AsDouble);
	}
	if (bEnableCounts)
	{
		CsvProfilerProvider.SetValue(StatSeries.ProviderCountHandle, StatSeries.CurrentFrame, StatSeries.CurrentCount);
	}
	StatSeries.CurrentValue = FStatSeriesValue();
	StatSeries.CurrentCount = 0;
}

void FCsvProfilerAnalyzer::FlushIfNewFrame(FStatSeriesInstance& StatSeries, uint32 FrameNumber)
{
	if (FrameNumber != StatSeries.CurrentFrame && StatSeries.CurrentValue.bIsValid)
	{
		check(FrameNumber > StatSeries.CurrentFrame);
		Flush(StatSeries);
	}
	StatSeries.CurrentFrame = FrameNumber;
}

void FCsvProfilerAnalyzer::FlushAtEndOfCapture(FStatSeriesInstance& StatSeries, uint32 CaptureEndFrame)
{
	if (StatSeries.CurrentValue.bIsValid && StatSeries.CurrentFrame < CaptureEndFrame)
	{
		Flush(StatSeries);
	}
}

void FCsvProfilerAnalyzer::SetTimerValue(FStatSeriesInstance& StatSeries, uint32 FrameNumber, double ElapsedTime, bool bCount)
{
	FlushIfNewFrame(StatSeries, FrameNumber);

	StatSeries.CurrentValue.Value.AsDouble += ElapsedTime;
	StatSeries.CurrentValue.bIsValid = true;
	if (bCount)
	{
		++StatSeries.CurrentCount;
	}

}

void FCsvProfilerAnalyzer::SetCustomStatValue(FStatSeriesInstance& StatSeries, uint32 FrameNumber, ECsvOpType OpType, int32 Value)
{
	FlushIfNewFrame(StatSeries, FrameNumber);

	if (!StatSeries.CurrentValue.bIsValid)
	{
		// The first op in a frame is always a set. Otherwise min/max don't work
		OpType = CsvOpType_Set;
	}

	switch (OpType)
	{
	case CsvOpType_Set:
		StatSeries.CurrentValue.Value.AsInt = Value;
		break;
	case CsvOpType_Min:
		StatSeries.CurrentValue.Value.AsInt = FMath::Min(int64(Value), StatSeries.CurrentValue.Value.AsInt);
		break;
	case CsvOpType_Max:
		StatSeries.CurrentValue.Value.AsInt = FMath::Max(int64(Value), StatSeries.CurrentValue.Value.AsInt);
		break;
	case CsvOpType_Accumulate:
		StatSeries.CurrentValue.Value.AsInt += Value;
		break;
	}
	StatSeries.CurrentValue.bIsValid = true;
	++StatSeries.CurrentCount;
}

void FCsvProfilerAnalyzer::SetCustomStatValue(FStatSeriesInstance& StatSeries, uint32 FrameNumber, ECsvOpType OpType, float Value)
{
	FlushIfNewFrame(StatSeries, FrameNumber);

	if (!StatSeries.CurrentValue.bIsValid)
	{
		// The first op in a frame is always a set. Otherwise min/max don't work
		OpType = CsvOpType_Set;
	}

	switch (OpType)
	{
	case CsvOpType_Set:
		StatSeries.CurrentValue.Value.AsDouble = Value;
		break;
	case CsvOpType_Min:
		StatSeries.CurrentValue.Value.AsDouble = FMath::Min(double(Value), StatSeries.CurrentValue.Value.AsDouble);
		break;
	case CsvOpType_Max:
		StatSeries.CurrentValue.Value.AsDouble = FMath::Max(double(Value), StatSeries.CurrentValue.Value.AsDouble);
		break;
	case CsvOpType_Accumulate:
		StatSeries.CurrentValue.Value.AsDouble += Value;
		break;
	}
	StatSeries.CurrentValue.bIsValid = true;
	++StatSeries.CurrentCount;
}

} // namespace TraceServices
