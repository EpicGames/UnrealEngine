// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/Map.h"
#include "Model/CsvProfilerProviderPrivate.h"
#include "ProfilingDebugging/MiscTrace.h"

namespace TraceServices
{

class IAnalysisSession;
class IFrameProvider;
class IThreadProvider;
class IEditableCounterProvider;
class IEditableCounter;

class FCsvProfilerAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FCsvProfilerAnalyzer(IAnalysisSession& Session, FCsvProfilerProvider& CsvProfilerProvider, IEditableCounterProvider& EditableCounterProvider, const IFrameProvider& FrameProvider, const IThreadProvider& ThreadProvider);
	~FCsvProfilerAnalyzer();
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	virtual void OnAnalysisEnd() override;

private:
	enum : uint16
	{
		RouteId_RegisterCategory,
		RouteId_DefineInlineStat,
		RouteId_DefineDeclaredStat,
		RouteId_BeginStat,
		RouteId_EndStat,
		RouteId_BeginExclusiveStat,
		RouteId_EndExclusiveStat,
		RouteId_CustomStatInt,
		RouteId_CustomStatFloat,
		RouteId_Event,
		RouteId_Metadata,
		RouteId_BeginCapture,
		RouteId_EndCapture,
	};

	enum ECsvOpType
	{
		CsvOpType_Set,
		CsvOpType_Min,
		CsvOpType_Max,
		CsvOpType_Accumulate,
	};

	union FStatSeriesID
	{
		struct
		{
			uint64 IsFName : 1;
			uint64 IsCountStat : 1;
			uint64 CategoryIndex : 11;
			uint64 FNameOrIndex : 51;
		} Fields;
		uint64 Hash;
	};

	struct FStatSeriesValue
	{
		FStatSeriesValue() { Value.AsInt = 0; }
		union
		{
			int64 AsInt;
			double AsDouble;
		} Value;
		bool bIsValid = false;
	};

	struct FStatSeriesDefinition
	{
		const TCHAR* Name = nullptr;
		int32 CategoryIndex = -1;
		int32 ColumnIndex = -1;
	};

	struct FStatSeriesInstance
	{
		uint64 ProviderHandle = uint64(-1);
		uint64 ProviderCountHandle = uint64(-1);
		IEditableCounter* Counter = nullptr;
		uint32 CurrentFrame = 0;
		FStatSeriesValue CurrentValue;
		int64 CurrentCount = 0;
		ECsvStatSeriesType Type = CsvStatSeriesType_CustomStatInt;
		ETraceFrameType FrameType = TraceFrameType_Game;
	};

	struct FTimingMarker
	{
		uint64 StatId = 0;
		uint64 Cycle = 0;
		bool bIsBegin = false;
		bool bIsExclusive = false;
		bool bIsExclusiveInsertedMarker = false;
	};

	struct FThreadState
	{
		TArray<FTimingMarker> MarkerStack;
		TArray<FTimingMarker> ExclusiveMarkerStack;
		ETraceFrameType FrameType = TraceFrameType_Game;
		TArray<FStatSeriesInstance*> StatSeries;
		FString ThreadName;
	};

	FThreadState& GetThreadState(uint32 ThreadId);
	FStatSeriesDefinition* CreateStatSeries(const TCHAR* Name, int32 CategoryIndex);
	void DefineStatSeries(uint64 StatId, const TCHAR* Name, int32 CategoryIndex, bool bIsInline);
	const TCHAR* GetStatSeriesName(const FStatSeriesDefinition* Definition, ECsvStatSeriesType Type, FThreadState& ThreadState, bool bIsCount);
	FStatSeriesInstance& GetStatSeries(uint64 StatId, ECsvStatSeriesType Type, FThreadState& ThreadState);
	void HandleMarkerEvent(const FOnEventContext& Context, bool bIsExclusive, bool bIsBegin);
	void HandleMarker(const FOnEventContext& Context, FThreadState& ThreadState, const FTimingMarker& Marker);
	void HandleCustomStatEvent(const FOnEventContext& Context, bool bIsFloat);
	void HandleEventEvent(const FOnEventContext& Context);
	void Flush(FStatSeriesInstance& StatSeries);
	void FlushIfNewFrame(FStatSeriesInstance& StatSeries, uint32 FrameNumber);
	void FlushAtEndOfCapture(FStatSeriesInstance& StatSeries, uint32 CaptureEndFrame);
	void SetTimerValue(FStatSeriesInstance& StatSeries, uint32 FrameNumber, double ElapsedTime, bool bCount);
	void SetCustomStatValue(FStatSeriesInstance& StatSeries, uint32 FrameNumber, ECsvOpType OpType, int32 Value);
	void SetCustomStatValue(FStatSeriesInstance& StatSeries, uint32 FrameNumber, ECsvOpType OpType, float Value);
	
	IAnalysisSession& Session;
	FCsvProfilerProvider& CsvProfilerProvider;
	IEditableCounterProvider& EditableCounterProvider;
	const IFrameProvider& FrameProvider;
	const IThreadProvider& ThreadProvider;

	TMap<uint32, FThreadState*> ThreadStatesMap;
	TMap<int32, const TCHAR*> CategoryMap;
	TMap<uint64, FStatSeriesDefinition*> StatSeriesMap;
	TMap<TTuple<int32, FString>, FStatSeriesDefinition*> StatSeriesStringMap;
	TArray<FStatSeriesDefinition*> StatSeriesDefinitionArray;
	TArray<FStatSeriesInstance*> StatSeriesInstanceArray;
	uint32 RenderThreadId = 0;
	uint32 RHIThreadId = 0;
	bool bEnableCounts = false;
	uint32 UndefinedStatSeriesCount = 0;

	TArray<uint64> FrameBoundaries[TraceFrameType_Count];
};

} // namespace TraceServices
