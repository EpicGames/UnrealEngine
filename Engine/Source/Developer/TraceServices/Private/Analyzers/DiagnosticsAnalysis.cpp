// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiagnosticsAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"

namespace TraceServices
{

FDiagnosticsAnalyzer::FDiagnosticsAnalyzer(IAnalysisSession& InSession, FDiagnosticsProvider* InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
	check(Provider != nullptr);
}

FDiagnosticsAnalyzer::~FDiagnosticsAnalyzer()
{
}

void FDiagnosticsAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Session, "Diagnostics", "Session");
	Builder.RouteEvent(RouteId_Session2, "Diagnostics", "Session2");
}

bool FDiagnosticsAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FDiagnosticsAnalyzer"));

	const auto& EventData = Context.EventData;

	if (RouteId == RouteId_Session)
	{
		const uint8* Attachment = EventData.GetAttachment();
		if (Attachment == nullptr)
		{
			return false;
		}

		FSessionInfo SessionInfo;

		uint8 AppNameOffset = EventData.GetValue<uint8>("AppNameOffset");
		uint8 CommandLineOffset = EventData.GetValue<uint8>("CommandLineOffset");

		SessionInfo.Platform = FString::ConstructFromPtrSize((const ANSICHAR*)Attachment, AppNameOffset);

		Attachment += AppNameOffset;
		int32 AppNameLength = CommandLineOffset - AppNameOffset;
		SessionInfo.AppName = FString::ConstructFromPtrSize((const ANSICHAR*)Attachment, AppNameLength);

		Attachment += AppNameLength;
		int32 CommandLineLength = EventData.GetAttachmentSize() - CommandLineOffset;
		SessionInfo.CommandLine = FString::ConstructFromPtrSize((const ANSICHAR*)Attachment, CommandLineLength);

		SessionInfo.ConfigurationType = (EBuildConfiguration)EventData.GetValue<uint8>("ConfigurationType");

		SessionInfo.TargetType = (EBuildTargetType)EventData.GetValue<uint8>("TargetType");

		FAnalysisSessionEditScope _(Session);

		if (Provider)
		{
			Provider->SetSessionInfo(SessionInfo);
		}

		Session.AddMetadata(FName("Platform"), SessionInfo.Platform);
		Session.AddMetadata(FName("AppName"), SessionInfo.AppName);
		Session.AddMetadata(FName("CommandLine"), SessionInfo.CommandLine);
		UpdateSessionMetadata(EventData);

		return false;
	}

	if (RouteId == RouteId_Session2)
	{
		FSessionInfo SessionInfo;

		EventData.GetString("Platform", SessionInfo.Platform);
		EventData.GetString("AppName", SessionInfo.AppName);
		EventData.GetString("ProjectName", SessionInfo.ProjectName);
		EventData.GetString("EngineVersion", SessionInfo.EngineVersion);
		EventData.GetString("BuildVersion", SessionInfo.BuildVersion);
		EventData.GetString("Branch", SessionInfo.Branch);
		EventData.GetString("CommandLine", SessionInfo.CommandLine);
		EventData.GetString("VFSPaths", SessionInfo.VFSPaths);
		SessionInfo.Changelist = EventData.GetValue<uint32>("Changelist", 0);
		SessionInfo.ConfigurationType = (EBuildConfiguration)EventData.GetValue<uint8>("ConfigurationType");
		SessionInfo.TargetType = (EBuildTargetType)EventData.GetValue<uint8>("TargetType");

		TArrayView<const uint32> Data = EventData.GetArrayView<uint32>("InstanceId");
		if (Data.Num() > 0)
		{
			check(Data.Num() == 4);
			SessionInfo.InstanceId = FGuid(Data[0], Data[1], Data[2], Data[3]);
		}

		FAnalysisSessionEditScope _(Session);
		Provider->SetSessionInfo(SessionInfo);
		UpdateSessionMetadata(EventData);

		return false;
	}

	return true;
}

void FDiagnosticsAnalyzer::UpdateSessionMetadata(const UE::Trace::IAnalyzer::FEventData& EventData)
{
	const UE::Trace::IAnalyzer::FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
	const uint32 FieldCount = TypeInfo.GetFieldCount();
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const UE::Trace::IAnalyzer::FEventFieldInfo* FieldInfo = TypeInfo.GetFieldInfo(FieldIndex);
		if (!FieldInfo)
		{
			continue;
		}
		switch (FieldInfo->GetType())
		{
			case UE::Trace::IAnalyzer::FEventFieldInfo::EType::Integer:
			{
				FName FieldName(FieldInfo->GetName());
				int64 Value = EventData.GetValue<int64>(FieldInfo->GetName());
				Session.AddMetadata(FieldName, Value);
				break;
			}
			case UE::Trace::IAnalyzer::FEventFieldInfo::EType::Float:
			{
				FName FieldName(FieldInfo->GetName());
				double Value = EventData.GetValue<double>(FieldInfo->GetName());
				Session.AddMetadata(FieldName, Value);
				break;
			}
			case UE::Trace::IAnalyzer::FEventFieldInfo::EType::AnsiString:
			case UE::Trace::IAnalyzer::FEventFieldInfo::EType::WideString:
			{
				FName FieldName(FieldInfo->GetName());
				FString Value;
				EventData.GetString(FieldInfo->GetName(), Value);
				Session.AddMetadata(FieldName, Value);
				break;
			}
		}
	}
}

} // namespace TraceServices
