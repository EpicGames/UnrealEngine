// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "TraceServices/Containers/Allocators.h"
#include "Model/DiagnosticsPrivate.h"
#include "Containers/UnrealString.h"

namespace TraceServices
{

class IAnalysisSession;

class FDiagnosticsAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FDiagnosticsAnalyzer(IAnalysisSession& Session, FDiagnosticsProvider* InProvider);
	~FDiagnosticsAnalyzer();

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	void UpdateSessionMetadata(const UE::Trace::IAnalyzer::FEventData& EventData);

	enum : uint16
	{
		RouteId_Session,
		RouteId_Session2,
	};

	IAnalysisSession& Session;
	FDiagnosticsProvider* Provider;
};

} // namespace TraceServices
