// Copyright Epic Games, Inc. All Rights Reserved.

#include "GpuProfilerTraceAnalysis.h"

#include "CborReader.h"
#include "CborWriter.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

// TraceServices
#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "Model/CountersPrivate.h"
#include "Model/TimingProfilerPrivate.h"

#define GPUPROFILER_LOG_API_L1(Format, ...) UE_LOGF(LogTraceServices, Log, Format, ##__VA_ARGS__)
#define GPUPROFILER_LOG_API_L2(Format, ...) //UE_LOGF(LogTraceServices, Verbose, Format, ##__VA_ARGS__)
#define GPUPROFILER_LOG_API_L3(Format, ...) //UE_LOGF(LogTraceServices, VeryVerbose, Format, ##__VA_ARGS__)

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FGpuProfilerAnalyzer::FGpuProfilerAnalyzer(FAnalysisSession& InSession, FTimingProfilerProvider& InTimingProfilerProvider, IEditableCounterProvider& InEditableCounterProvider)
	: Session(InSession)
	, TimingProfilerProvider(InTimingProfilerProvider)
	, EditableCounterProvider(InEditableCounterProvider)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Init,                 "GpuProfiler", "Init");
	Builder.RouteEvent(RouteId_QueueSpec,            "GpuProfiler", "QueueSpec");
	Builder.RouteEvent(RouteId_EventFrameBoundary,   "GpuProfiler", "EventFrameBoundary");
	Builder.RouteEvent(RouteId_EventBreadcrumbSpec,  "GpuProfiler", "EventBreadcrumbSpec");
	Builder.RouteEvent(RouteId_EventBeginBreadcrumb, "GpuProfiler", "EventBeginBreadcrumb");
	Builder.RouteEvent(RouteId_EventEndBreadcrumb,   "GpuProfiler", "EventEndBreadcrumb");
	Builder.RouteEvent(RouteId_EventBeginWork,       "GpuProfiler", "EventBeginWork");
	Builder.RouteEvent(RouteId_EventEndWork,         "GpuProfiler", "EventEndWork");
	Builder.RouteEvent(RouteId_EventWait,            "GpuProfiler", "EventWait");
	Builder.RouteEvent(RouteId_EventStats,           "GpuProfiler", "EventStats");
	Builder.RouteEvent(RouteId_SignalFence,          "GpuProfiler", "SignalFence");
	Builder.RouteEvent(RouteId_WaitFence,            "GpuProfiler", "WaitFence");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::OnAnalysisEnd()
{
	if (ErrorData.NumInterleavedEvents > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Number of interleaved events: %u, max error %f", ErrorData.NumInterleavedEvents, ErrorData.InterleavedEventsMaxDelta);
	}

	if (ErrorData.NumInterleavedAndReversedEvents > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Number of interleaved and reversed events: %u, max error %f", ErrorData.NumInterleavedAndReversedEvents, ErrorData.InterleavedAndReversedEventsMaxDelta);
	}

	if (ErrorData.NumMismatchedEvents > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Number of mismatched events: %u", ErrorData.NumMismatchedEvents);
	}

	if (ErrorData.NumNegativeDurationEvents > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Number of negative duration events: %u, max error %f", ErrorData.NumNegativeDurationEvents, ErrorData.NegativeDurationEventsMaxDelta);
	}

	if (ErrorData.NumTimerWarnings > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Number of timer/metadata warnings: %u", ErrorData.NumTimerWarnings);
	}

	if (Queues.Num() > 0 || TimerMap.Num() > 0)
	{
		UE_LOGF(LogTraceServices, Log, "[GpuProfiler] Analysis completed (%u queues, %d timers, %d breadcrumb specs, %d breadcrumb names).",
			Queues.Num(), TimerMap.Num(), SpecIdToTimerIdMap.Num(), BreadcrumbMap.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGpuProfilerAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FGpuProfilerAnalyzer"));

	const auto& EventData = Context.EventData;

	switch (RouteId)
	{

	case RouteId_Init:
	{
		Version = (uint32)EventData.GetValue<uint8>("Version");
		GPUPROFILER_LOG_API_L1("[GpuProfiler] Init Version=%u", Version);
		break;
	}

	case RouteId_QueueSpec:
	{
		const uint32 QueueId = EventData.GetValue<uint32>("QueueId");

		const uint8 GPU = (QueueId >> 8) & 0xFF;
		const uint8 Index = (QueueId >> 16) & 0xFF;
		const uint8 Type = QueueId & 0xFF;

		FString Name;
		EventData.GetString("TypeString", Name);

		GPUPROFILER_LOG_API_L2("[GpuProfiler] QueueSpec QueueId=%u, GPU=%u, Index=%u, Type=%d \"%ls\"", QueueId, (uint32)GPU, (uint32)Index, (uint32)Type, *Name);

		const TCHAR* PersistentName = Session.StoreString(Name);

		{
			FAnalysisSessionEditScope _(Session);
			TimingProfilerProvider.AddGpuQueue(QueueId, GPU, Index, Type, PersistentName);
		}

		FQueue& Queue = GetOrAddQueue(QueueId);
		InitCountersDesc(Queue, GPU, Index, *Name);
		break;
	}

	case RouteId_EventFrameBoundary:
	{
		const uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint32 FrameNumber = EventData.GetValue<uint32>("FrameNumber");

		GPUPROFILER_LOG_API_L3("[GpuProfiler] EventFrameBoundary QueueId=%u, FrameNumber=%u", QueueId, FrameNumber);

		FQueue& Queue = GetOrAddQueue(QueueId);
		Queue.FrameNumber = FrameNumber;

		if (Queue.LastTime > 0)
		{
			FAnalysisSessionEditScope _(Session);
			Queue.NumDrawsCounter->SetValue(Queue.LastTime, (int64)Queue.NumDraws);
			Queue.NumPrimitivesCounter->SetValue(Queue.LastTime, (int64)Queue.NumPrimitives);
		}

		Queue.NumDraws = 0;
		Queue.NumPrimitives = 0;
		Queue.LastTime = 0;
		break;
	}

	case RouteId_EventBreadcrumbSpec:
	{
		OnEventBreadcrumbSpec(Context);
		break;
	}

	case RouteId_EventBeginBreadcrumb:
	{
		const uint32 SpecId = EventData.GetValue<uint32>("SpecId");
		const uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint64 GPUTimestampTOP = EventData.GetValue<uint64>("GPUTimestampTOP");
		// Ignore events for which the timestamp could not be determined.
		if (GPUTimestampTOP == 0)
		{
			break;
		}
		const double Time = Context.EventTime.AsSeconds(GPUTimestampTOP);
		TArray<uint8> Metadata(EventData.GetArrayView<uint8>("Metadata"));

		GPUPROFILER_LOG_API_L3("[GpuProfiler] EventBeginBreadcrumb QueueId=%u, Time=%f, SpecId=%u", QueueId, Time, SpecId);

		uint32 TimerId = GetOrAddTimer(SpecId, TEXT("<unknown>"));

		if (Metadata.Num() > 0) // excludes empty metadata
		{
			FAnalysisSessionEditScope _(Session);
			TimerId = TimingProfilerProvider.AddMetadata(TimerId, MoveTemp(Metadata));
		}
#if 0 // for debugging only
		else
		{
			FAnalysisSessionEditScope _(Session);
			const ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider.GetTimerReader();
			const FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerId);
			check(Timer);
			if (Timer->HasValidMetadataSpecId())
			{
				UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Empty metadata for a timing event while its timer %u (%ls) has valid metadata spec id (%u) !!!", TimerId, Timer->Name, Timer->MetadataSpecId);
			}
		}
#endif

		FQueue& Queue = GetOrAddQueue(QueueId);
		BeginEvent(Queue, 0, Time, TimerId);
		break;
	}

	case RouteId_EventEndBreadcrumb:
	{
		const uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint64 GPUTimestampBOP = EventData.GetValue<uint64>("GPUTimestampBOP");
		// Ignore events for which the timestamp could not be determined.
		if (GPUTimestampBOP == 0)
		{
			break;
		}
		const double Time = Context.EventTime.AsSeconds(GPUTimestampBOP);

		GPUPROFILER_LOG_API_L3("[GpuProfiler] EventEndBreadcrumb QueueId=%u, Time=%f", QueueId, Time);

		FQueue& Queue = GetOrAddQueue(QueueId);
		EndEvent(Queue, 0, Time, InvalidTimerId);
		break;
	}

	case RouteId_EventBeginWork:
	{
		const uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint64 GPUTimestampTOP = EventData.GetValue<uint64>("GPUTimestampTOP");
		double Time = Context.EventTime.AsSeconds(GPUTimestampTOP);
		//const uint64 CPUTimestamp = EventData.GetValue<uint64>("CPUTimestamp"); // not used

		GPUPROFILER_LOG_API_L3("[GpuProfiler] EventBeginWork QueueId=%u, GPUTimestampTOP=%f", QueueId, Time);

		if (GpuWorkTimerId == InvalidTimerId)
		{
			const FString GpuWorkTimerName(TEXT("GpuWork"));
			GpuWorkTimerId = GetOrAddTimer(~0u, GpuWorkTimerName);
		}

		FQueue& Queue = GetOrAddQueue(QueueId);
		TArray<uint8> CborData;
		{
			CborData.Reserve(256);
			FMemoryWriter MemoryWriter(CborData, false, true);
			FCborWriter CborWriter(&MemoryWriter, ECborEndianness::StandardCompliant);
			CborWriter.WriteContainerStart(ECborCode::Map, 1); // 1 is the FieldCount
			CborWriter.WriteValue("Frame", 5);
			CborWriter.WriteValue((int64)Queue.FrameNumber);
		}
		uint32 MetadataTimerId;
		{
			FAnalysisSessionEditScope _(Session);
			MetadataTimerId = TimingProfilerProvider.AddMetadata(GpuWorkTimerId, MoveTemp(CborData));
		}

		BeginEvent(Queue, 1, Time, MetadataTimerId);
		break;
	}

	case RouteId_EventEndWork:
	{
		uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint64 GPUTimestampBOP = EventData.GetValue<uint64>("GPUTimestampBOP");
		const double Time = Context.EventTime.AsSeconds(GPUTimestampBOP);

		GPUPROFILER_LOG_API_L3("[GpuProfiler] EventEndWork QueueId=%u, GPUTimestampBOP=%f", QueueId, Time);

		FQueue& Queue = GetOrAddQueue(QueueId);
		EndEvent(Queue, 1, Time, GpuWorkTimerId);
		break;
	}

	case RouteId_EventWait:
	{
		uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint64 StartTimestamp = EventData.GetValue<uint64>("StartTime");
		const double StartTime = Context.EventTime.AsSeconds(StartTimestamp);

		const uint64 EndTimestamp = EventData.GetValue<uint64>("EndTime");
		const double EndTime = Context.EventTime.AsSeconds(EndTimestamp);

		GPUPROFILER_LOG_API_L3("[GpuProfiler] EventWait QueueId=%u, StartTime=%f, EndTime=%f", QueueId, StartTime, EndTime);

		if (GpuWaitTimerId == InvalidTimerId)
		{
			const FString GpuWaitTimerName(TEXT("GpuWait"));
			GpuWaitTimerId = GetOrAddTimer(~0u, GpuWaitTimerName);
		}

		FQueue& Queue = GetOrAddQueue(QueueId);
		TArray<uint8> CborData;
		{
			CborData.Reserve(256);
			FMemoryWriter MemoryWriter(CborData, false, true);
			FCborWriter CborWriter(&MemoryWriter, ECborEndianness::StandardCompliant);
			CborWriter.WriteContainerStart(ECborCode::Map, 1); // 1 is the FieldCount
			CborWriter.WriteValue("Frame", 5);
			CborWriter.WriteValue((int64)Queue.FrameNumber);
		}
		uint32 MetadataTimerId;
		{
			FAnalysisSessionEditScope _(Session);
			MetadataTimerId = TimingProfilerProvider.AddMetadata(GpuWaitTimerId, MoveTemp(CborData));
		}

		BeginEvent(Queue, 1, StartTime, MetadataTimerId);
		EndEvent(Queue, 1, EndTime, GpuWaitTimerId);
		break;
	}

	case RouteId_EventStats:
	{
		const uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		const uint32 NumDraws = EventData.GetValue<uint32>("NumDraws");
		const uint32 NumPrimitives = EventData.GetValue<uint32>("NumPrimitives");

		FQueue& Queue = GetOrAddQueue(QueueId);

		Queue.NumDraws += NumDraws;
		Queue.NumPrimitives += NumPrimitives;
		break;
	}

	case RouteId_SignalFence:
	{
		uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		uint64 Timestamp = EventData.GetValue<uint64>("CPUTimestamp");
		uint64 Value = EventData.GetValue<uint64>("Value");;

		FQueue& Queue = GetOrAddQueue(QueueId);
		FGpuSignalFence Fence;
		Fence.Timestamp = Context.EventTime.AsSeconds(Timestamp);
		Fence.Value = Value;

		FAnalysisSessionEditScope _(Session);
		TimingProfilerProvider.AddGpuSignalFence(QueueId, Fence);
		break;
	}

	case RouteId_WaitFence:
	{
		uint32 QueueId = EventData.GetValue<uint32>("QueueId");
		uint64 Timestamp = EventData.GetValue<uint64>("CPUTimestamp");
		uint32 QueueToWaitForId = EventData.GetValue<uint32>("QueueToWaitForId");
		uint64 Value = EventData.GetValue<uint64>("Value");;

		FGpuWaitFence Fence;
		Fence.Timestamp = Context.EventTime.AsSeconds(Timestamp);
		Fence.Value = Value;
		Fence.QueueToWaitForId = QueueToWaitForId;

		FAnalysisSessionEditScope _(Session);
		TimingProfilerProvider.AddGpuWaitFence(QueueId, Fence);
		break;
	}

	} // switch (RouteId)

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGpuProfilerAnalyzer::FQueue& FGpuProfilerAnalyzer::GetOrAddQueue(uint32 QueueId)
{
	FQueue* FoundQueue = Queues.Find(QueueId);
	if (FoundQueue)
	{
		return *FoundQueue;
	}

	FQueue& NewQueue = Queues.Add(QueueId);
	NewQueue.Id = QueueId;
	InitCounters(NewQueue);
	return NewQueue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::BeginEvent(FQueue& Queue, int32 StackIndex, double BeginEventTime, uint32 BeginEventTimerId)
{
	Queue.Stack[StackIndex].Push({ BeginEventTime, BeginEventTimerId });
	Queue.LastTime = BeginEventTime;

	FAnalysisSessionEditScope _(Session);
	IEditableTimeline<FTimingProfilerEvent>* Timeline = (StackIndex == 1) ?
		TimingProfilerProvider.GetGpuQueueWorkEditableTimeline(Queue.Id) :
		TimingProfilerProvider.GetGpuQueueEditableTimeline(Queue.Id);
	if (ensure(Timeline))
	{
		FTimingProfilerEvent Event;
		Event.TimerIndex = BeginEventTimerId;
		const double LastTimestamp = Timeline->GetLastTimestamp();
		if (BeginEventTime < LastTimestamp)
		{
			++ErrorData.NumInterleavedEvents;
			ErrorData.InterleavedEventsMaxDelta = FMath::Max(ErrorData.InterleavedEventsMaxDelta, LastTimestamp - BeginEventTime);
			if (++ErrorData.NumWarnings < ErrorData.NumMaxWarnings)
			{
				uint32 OriginalBeginEventTimerId = TimingProfilerProvider.GetOriginalTimerIdFromMetadata(BeginEventTimerId);
				UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Queue %u %s BEGIN %d \"%ls\" : Time %f < %f (last) !!!",
					Queue.Id,
					StackIndex == 1 ? "WORK" : "BREADCRUMB",
					int32(BeginEventTimerId), GetTimerName(OriginalBeginEventTimerId),
					BeginEventTime, LastTimestamp);
			}
			BeginEventTime = LastTimestamp;
		}
		Timeline->AppendBeginEvent(BeginEventTime, Event);
		Session.UpdateDurationSeconds(BeginEventTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::EndEvent(FQueue& Queue, int32 StackIndex, double EndEventTime, uint32 EndEventTimerId)
{
	Queue.LastTime = EndEventTime;

	if (Queue.Stack[StackIndex].IsEmpty())
	{
		return;
	}
	FOpenEvent BeginEvent = Queue.Stack[StackIndex].Pop(EAllowShrinking::No);
	uint32 BeginEventTimerId;
	{
		FAnalysisSessionReadScope _(Session);
		BeginEventTimerId = TimingProfilerProvider.GetOriginalTimerIdFromMetadata(BeginEvent.TimerId);
	}
	if (EndEventTimerId != InvalidTimerId &&
		EndEventTimerId != BeginEventTimerId)
	{
		++ErrorData.NumMismatchedEvents;
		if (++ErrorData.NumWarnings < ErrorData.NumMaxWarnings)
		{
			UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Queue %u %s END %d \"%ls\" (%f) != BEGIN %d \"%ls\" (%f) !!!",
				Queue.Id,
				StackIndex == 1 ? "WORK" : "BREADCRUMB",
				int32(EndEventTimerId), GetTimerName(EndEventTimerId), EndEventTime,
				int32(BeginEvent.TimerId), GetTimerName(BeginEventTimerId), BeginEvent.Time);
		}
	}
	if (BeginEvent.Time > EndEventTime)
	{
		++ErrorData.NumNegativeDurationEvents;
		ErrorData.NegativeDurationEventsMaxDelta = FMath::Max(ErrorData.NegativeDurationEventsMaxDelta, BeginEvent.Time - EndEventTime);
		if (++ErrorData.NumWarnings < ErrorData.NumMaxWarnings)
		{
			if (EndEventTimerId == InvalidTimerId)
			{
				UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Queue %u %s END : Time %f < %f (BEGIN %d \"%ls\") !!!",
					Queue.Id,
					StackIndex == 1 ? "WORK" : "BREADCRUMB",
					EndEventTime, BeginEvent.Time,
					int32(BeginEvent.TimerId), GetTimerName(BeginEventTimerId));
			}
			else
			{
				UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Queue %u %s END %d \"%ls\" : Time %f < %f (BEGIN %d \"%ls\") !!!",
					Queue.Id,
					StackIndex == 1 ? "WORK" : "BREADCRUMB",
					int32(EndEventTimerId), GetTimerName(EndEventTimerId),
					EndEventTime, BeginEvent.Time,
					int32(BeginEvent.TimerId), GetTimerName(BeginEventTimerId));
			}
		}
	}

	FAnalysisSessionEditScope _(Session);
	IEditableTimeline<FTimingProfilerEvent>* Timeline = (StackIndex == 1) ?
		TimingProfilerProvider.GetGpuQueueWorkEditableTimeline(Queue.Id) :
		TimingProfilerProvider.GetGpuQueueEditableTimeline(Queue.Id);
	if (ensure(Timeline))
	{
		const double LastTimestamp = Timeline->GetLastTimestamp();
		if (EndEventTime < LastTimestamp)
		{
			++ErrorData.NumInterleavedAndReversedEvents;
			ErrorData.InterleavedAndReversedEventsMaxDelta = FMath::Max(ErrorData.InterleavedAndReversedEventsMaxDelta, LastTimestamp - EndEventTime);
			if (++ErrorData.NumWarnings < ErrorData.NumMaxWarnings)
			{
				if (EndEventTimerId == InvalidTimerId)
				{
					UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Queue %u %s END : Time %f < %f (last) !!!",
						Queue.Id,
						StackIndex == 1 ? "WORK" : "BREADCRUMB",
						EndEventTime, LastTimestamp);
				}
				else
				{
					UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] Queue %u %s END %d \"%ls\" : Time %f < %f (last) !!!",
						Queue.Id,
						StackIndex == 1 ? "WORK" : "BREADCRUMB",
						int32(EndEventTimerId), GetTimerName(EndEventTimerId),
						EndEventTime, LastTimestamp);
				}
			}
			EndEventTime = LastTimestamp;
		}
		Timeline->AppendEndEvent(EndEventTime);
		Session.UpdateDurationSeconds(EndEventTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::OnEventBreadcrumbSpec(const FOnEventContext& Context)
{
	const auto& EventData = Context.EventData;

	uint32 SpecId = EventData.GetValue<uint32>("SpecId");

	FString Name, NameFormat;
	EventData.GetString("StaticName", Name);
	EventData.GetString("NameFormat", NameFormat);
	TArrayView<const uint8> FieldNames = EventData.GetArrayView<uint8>("FieldNames");

	GPUPROFILER_LOG_API_L2("[GpuProfiler] EventBreadcrumbSpec Id=%u Name=\"%ls\" NameFormat=\"%ls\" Fields[]={%u bytes}",
		SpecId, *Name, *NameFormat, FieldNames.Num());

	if (Name.Compare(NameFormat) == 0)
	{
		Name.Empty();
	}

	int Index = 0;
	while (Index < NameFormat.Len() && Index < Name.Len())
	{
		if (Name[Index] != NameFormat[Index])
		{
			break;
		}

		++Index;
	}

	if (Index > 1)
	{
		NameFormat.MidInline(Index);
	}

	if (Name.IsEmpty())
	{
		const FString Separators = TEXT("% (=");
		Index = 0;
		bool bIsInFormatSpecifier = false;
		while (Index < NameFormat.Len())
		{
			int32 SpecIndex = -1;
			if (Separators.FindChar(NameFormat[Index], SpecIndex))
			{
				Name = NameFormat.Left(Index);
				NameFormat.MidInline(Index);
				break;
			}

			++Index;
		}

		if (Name.IsEmpty())
		{
			if (!NameFormat.IsEmpty())
			{
				Name = NameFormat;
				NameFormat.Empty();
			}
			else
			{
				Name = Session.StoreString(TEXT("Unknown"));
			}
		}
	}

	FMetadataSpec Spec;
	if (!NameFormat.IsEmpty())
	{
		Spec.Format = Session.StoreString(NameFormat);
	}

	if (FieldNames.Num() > 0)
	{
		FMemoryReaderView MemoryReader(FieldNames);
		FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
		FCborContext CborContext;

		while (CborReader.ReadNext(CborContext))
		{
			if (CborContext.MajorType() == ECborCode::TextString)
			{
				FString Field = CborContext.AsString();
				Spec.FieldNames.Add(Session.StoreString(Field));
			}
		}
	}

	uint32 TimerId;
	if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
	{
		TimerId = *FoundTimerIdBySpecId;
		SetTimerName(SpecId, TimerId, Name);
	}
	else
	{
		GPUPROFILER_LOG_API_L2("[GpuProfiler] EventBreadcrumbSpec with a new timer (\"%ls\")", *Name);
		TimerId = AddGpuTimer(SpecId, Name);
	}

	if (Spec.FieldNames.Num() > 0 || Spec.Format != nullptr)
	{
		FAnalysisSessionEditScope _(Session);

		uint32 MetadataSpecId = TimingProfilerProvider.AddMetadataSpec(MoveTemp(Spec));

		const ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider.GetTimerReader();
		const FTimingProfilerTimer* PreviousTimer = TimerReader.GetTimer(TimerId);
		if (PreviousTimer && PreviousTimer->HasValidMetadataSpecId())
		{
			TimerId = AddGpuTimer(SpecId, Name);

			if (++ErrorData.NumTimerWarnings < ErrorData.NumMaxWarnings)
			{
				const FMetadataSpec* PreviousMetadataSpec = TimingProfilerProvider.GetMetadataSpec(PreviousTimer->MetadataSpecId);
				check(PreviousMetadataSpec);
				UE_LOGF(LogTraceServices, Warning, "[GpuProfiler] EventBreadcrumbSpec -- de-duplicated timer for SpecId %u -- old: %u (%ls) metadata=\"%ls\" (%d fields) --> new: %u (%ls) metadata=\"%ls\" (%d fields) !!!",
					SpecId,
					PreviousTimer->Id, PreviousTimer->Name, PreviousMetadataSpec->Format, PreviousMetadataSpec->FieldNames.Num(),
					TimerId, *Name, Spec.Format, Spec.FieldNames.Num());
			}
		}

		TimingProfilerProvider.SetMetadataSpec(TimerId, MetadataSpecId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FGpuProfilerAnalyzer::GetOrAddTimer(uint32 SpecId, const FString& Breadcrumb)
{
	if (SpecId != ~0u)
	{
		if (const uint32* FoundTimerIdBySpecId = SpecIdToTimerIdMap.Find(SpecId))
		{
			return *FoundTimerIdBySpecId;
		}
	}

	// Merge by name for the GPU timers is disabled as it is not safe to merge when metdata can be attached late.
	//if (const uint32* FoundTimerId = BreadcrumbMap.Find(Breadcrumb))
	//{
	//	return *FoundTimerId;
	//}

	return AddGpuTimer(SpecId, Breadcrumb);
}
////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FGpuProfilerAnalyzer::AddGpuTimer(uint32 SpecId, const FString& Breadcrumb)
{
	uint32 TimerId;
	const TCHAR* TimerName = nullptr;

	{
		FAnalysisSessionEditScope _(Session);
		TimerId = TimingProfilerProvider.AddGpuTimer(Breadcrumb);
		const ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider.GetTimerReader();
		const FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerId);
		if (Timer)
		{
			TimerName = Timer->Name;
		}
	}

	TimerMap.Add(TimerId, TimerName);
	BreadcrumbMap.Add(Breadcrumb, TimerId);

	if (SpecId != ~0u)
	{
		SpecIdToTimerIdMap.Add(SpecId, TimerId);

		GPUPROFILER_LOG_API_L3("[GpuProfiler] --> AddGpuTimer(SpecId=%u, Name=\"%ls\") --> TimerId=%u", SpecId, TimerName, TimerId);
	}
	else
	{
		GPUPROFILER_LOG_API_L3("[GpuProfiler] --> AddGpuTimer(Name=\"%ls\") --> TimerId=%u", TimerName, TimerId);
	}

	return TimerId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FGpuProfilerAnalyzer::GetTimerName(uint32 TimerId) const
{
	if (TimerId == InvalidTimerId)
	{
		return TEXT("<invalid>");
	}
	const TCHAR* const* TimerNamePtr = TimerMap.Find(TimerId);
	return TimerNamePtr ? *TimerNamePtr : TEXT("<unknown>");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::SetTimerName(uint32 SpecId, uint32 TimerId, FStringView TimerName)
{
	FAnalysisSessionEditScope _(Session);
	TimingProfilerProvider.SetTimerName(TimerId, TimerName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::InitCounters(FQueue& FoundQueue)
{
	FoundQueue.NumDrawsCounter = EditableCounterProvider.CreateEditableCounter();
	FoundQueue.NumDrawsCounter->SetIsFloatingPoint(false);

	FoundQueue.NumPrimitivesCounter = EditableCounterProvider.CreateEditableCounter();
	FoundQueue.NumPrimitivesCounter->SetIsFloatingPoint(false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuProfilerAnalyzer::InitCountersDesc(FQueue& FoundQueue, uint8 Gpu, uint8 Index, const TCHAR* Name)
{
	const FString DrawsCounterName = FString::Printf(TEXT("NumDraws_GPU%u_%s_%u"), Gpu, Name, Index);
	const FString PrimitivesCounterName = FString::Printf(TEXT("NumPrimitives_GPU%u_%s_%u"), Gpu, Name, Index);

	FoundQueue.NumDrawsCounter->SetName(Session.StoreString(DrawsCounterName));
	FoundQueue.NumDrawsCounter->SetDescription(TEXT("The number of draw calls on the specified queue."));

	FoundQueue.NumPrimitivesCounter->SetName(Session.StoreString(PrimitivesCounterName));
	FoundQueue.NumPrimitivesCounter->SetDescription(TEXT("The number of primitives on the specified queue."));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef GPUPROFILER_LOG_API_L3
#undef GPUPROFILER_LOG_API_L2
#undef GPUPROFILER_LOG_API_L1
