// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"

// TraceAnalysis
#include "Trace/Analyzer.h"

namespace TraceServices
{

class FAnalysisSession;
class FTimingProfilerProvider;
class IEditableCounter;
class IEditableCounterProvider;

// The New GPU Profiler
class FGpuProfilerAnalyzer : public UE::Trace::IAnalyzer
{
private:
	enum : uint16
	{
		RouteId_Init,
		RouteId_QueueSpec,
		RouteId_EventFrameBoundary,
		RouteId_EventBreadcrumbSpec,
		RouteId_EventBeginBreadcrumb,
		RouteId_EventEndBreadcrumb,
		RouteId_EventBeginWork,
		RouteId_EventEndWork,
		RouteId_EventWait,
		RouteId_EventStats,
		RouteId_SignalFence,
		RouteId_WaitFence,
	};

	struct FOpenEvent
	{
		double Time;
		uint32 TimerId;
	};

	struct FQueue
	{
		uint32 Id = 0;
		uint32 FrameNumber = 0;
		TArray<FOpenEvent> Stack[2]; // [0] = breadcrumbs, [1] = work
		IEditableCounter* NumDrawsCounter;
		IEditableCounter* NumPrimitivesCounter;
		uint64 NumDraws = 0;
		uint64 NumPrimitives = 0;
		double LastTime = 0.0f;
	};

	struct FErrorData
	{
		static constexpr uint32 NumMaxWarnings = 100;
		static constexpr uint32 NumMaxErrors = 100;

		uint32 NumWarnings = 0;
		uint32 NumTimerWarnings = 0;
		uint32 NumErrors = 0;

		uint32 NumInterleavedEvents = 0;
		uint32 NumInterleavedAndReversedEvents = 0;
		uint32 NumMismatchedEvents = 0;
		uint32 NumNegativeDurationEvents = 0;

		double InterleavedEventsMaxDelta = 0.0f;
		double InterleavedAndReversedEventsMaxDelta = 0.0f;
		double NegativeDurationEventsMaxDelta = 0.0f;
	};

public:
	FGpuProfilerAnalyzer(FAnalysisSession& Session, FTimingProfilerProvider& TimingProfilerProvider, IEditableCounterProvider& EditableCounterProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	FQueue& GetOrAddQueue(uint32 QueueId);
	void BeginEvent(FQueue& Queue, int32 StackIndex, double Time, uint32 TimerId);
	void EndEvent(FQueue& Queue, int32 StackIndex, double Time, uint32 TimerId);
	void OnEventBreadcrumbSpec(const FOnEventContext& Context);

	uint32 GetOrAddTimer(uint32 SpecId, const FString& Breadcrumb);
	uint32 AddGpuTimer(uint32 SpecId, const FString& Breadcrumb);
	const TCHAR* GetTimerName(uint32 TimerId) const;
	void SetTimerName(uint32 SpecId, uint32 TimerId, FStringView TimerName);

	void InitCounters(FQueue& FoundQueue);
	void InitCountersDesc(FQueue& FoundQueue, uint8 Gpu, uint8 Index, const TCHAR* Name);

private:
	FAnalysisSession& Session;
	FTimingProfilerProvider& TimingProfilerProvider;
	IEditableCounterProvider& EditableCounterProvider;

	uint32 Version = 0;
	static constexpr uint32 InvalidTimerId = 0;
	uint32 GpuWorkTimerId = InvalidTimerId;
	uint32 GpuWaitTimerId = InvalidTimerId;
	TMap<uint32, uint32> SpecIdToTimerIdMap; // breadcrumb spec id --> GPU timer id
	TMap<FString, uint32> BreadcrumbMap; // breadcrumb name --> GPU timer id
	TMap<uint32, const TCHAR*> TimerMap; // GPU timer id --> persistent timer name
	TMap<uint32, FQueue> Queues; // QueueId --> FQueue
	FErrorData ErrorData;
};

} // namespace TraceServices
