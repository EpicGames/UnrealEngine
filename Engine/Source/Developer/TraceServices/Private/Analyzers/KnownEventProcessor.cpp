// Copyright Epic Games, Inc. All Rights Reserved.

#include "KnownEventProcessor.h"

#include "Misc/CString.h"

// TraceServices
#include "Common/Utils.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FKnownEventProcessor
////////////////////////////////////////////////////////////////////////////////////////////////////

FKnownEventProcessor::FKnownEventProcessor()
{
	KnownEvents.AddDefaulted((int)EKnownEvent::Count);
	KnownEventMap.AddDefaulted(MaxKnownEventId + 1);

	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FKnownEventProcessor::~FKnownEventProcessor()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::Reset()
{
	KnownEvents[(int)EKnownEvent::Invalid].Id = InvalidEventId;
	KnownEvents[(int)EKnownEvent::Trace_NewEvent].Id = 0;

	for (uint32 EventId = 0; EventId <= MaxKnownEventId; ++EventId)
	{
		KnownEventMap[EventId] = EKnownEvent::Invalid;
	}
	KnownEventMap[0] = EKnownEvent::Trace_NewEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::RegisterCallbacks()
{
	using FOnEventContext = UE::Trace::IAnalyzer::FOnEventContext;
	using FEventData = UE::Trace::IAnalyzer::FEventData;

	//////////////////////////////////////////////////
	// Misc.*

	KnownEvents[(int)EKnownEvent::Misc_RegionBeginWithId].Callback =
		[this](const FOnEventContext& Context)
		{
			const uint64 Cycle = Context.EventData.GetValue<uint64>("CycleAndId");
			UpdateThreadTime(Context, Cycle);
		};

	//////////////////////////////////////////////////
	// CpuProfiler.*

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventBatchV3].Callback =
		[this](const FOnEventContext& Context)
		{
			const FEventData& EventData = Context.EventData;
			TArrayView<const uint8> Data = EventData.GetArrayView<uint8>("Data");
			if (Data.GetData() != nullptr)
			{
				ProcessCpuEventBatchV2V3(Context, Data.GetData(), Data.Num(), 3);
			}
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventBatchV2].Callback =
		[this](const FOnEventContext& Context)
		{
			const FEventData& EventData = Context.EventData;
			TArrayView<const uint8> Data = EventData.GetArrayView<uint8>("Data");
			if (Data.GetData() != nullptr)
			{
				ProcessCpuEventBatchV2V3(Context, Data.GetData(), Data.Num(), 2);
			}
		};

	KnownEvents[(int)EKnownEvent::CpuProfiler_EventBatch].Callback =
	KnownEvents[(int)EKnownEvent::CpuProfiler_EndCapture].Callback =
		[this](const FOnEventContext& Context)
		{
			const FEventData& EventData = Context.EventData;
			TArrayView<const uint8> Data = EventData.GetArrayView<uint8>("Data");
			if (Data.GetData() != nullptr)
			{
				ProcessCpuEventBatchV1(Context, Data.GetData(), Data.Num());
			}
		};

	//////////////////////////////////////////////////
	// GpuProfiler.*

	KnownEvents[(int)EKnownEvent::GpuProfiler_EventBeginBreadcrumb].Callback =
	KnownEvents[(int)EKnownEvent::GpuProfiler_EventBeginWork].Callback =
		[this](const FOnEventContext& Context)
		{
			const uint64 Cycle = Context.EventData.GetValue<uint64>("GPUTimestampTOP");
			UpdateThreadTime(Context, Cycle);
		};
	KnownEvents[(int)EKnownEvent::GpuProfiler_EventEndBreadcrumb].Callback =
	KnownEvents[(int)EKnownEvent::GpuProfiler_EventEndWork].Callback =
		[this](const FOnEventContext& Context)
		{
			const uint64 Cycle = Context.EventData.GetValue<uint64>("GPUTimestampBOP");
			UpdateThreadTime(Context, Cycle);
		};
	KnownEvents[(int)EKnownEvent::GpuProfiler_EventWait].Callback =
		[this](const FOnEventContext& Context)
		{
			const uint64 Cycle = Context.EventData.GetValue<uint64>("EndTime");
			UpdateThreadTime(Context, Cycle);
		};
	KnownEvents[(int)EKnownEvent::GpuProfiler_SignalFence].Callback =
	KnownEvents[(int)EKnownEvent::GpuProfiler_WaitFence].Callback =
		[this](const FOnEventContext& Context)
		{
			const uint64 Cycle = Context.EventData.GetValue<uint64>("CPUTimestamp");
			UpdateThreadTime(Context, Cycle);
		};

	//////////////////////////////////////////////////
	// Known events with a "Cycle" field

	KnownEvents[(int)EKnownEvent::Misc_Bookmark].Callback =
	KnownEvents[(int)EKnownEvent::Misc_ScreenshotHeader].Callback =
	KnownEvents[(int)EKnownEvent::Misc_RegionBegin].Callback =
	KnownEvents[(int)EKnownEvent::Misc_RegionEnd].Callback =
	KnownEvents[(int)EKnownEvent::Misc_RegionEndWithId].Callback =
	KnownEvents[(int)EKnownEvent::Misc_BeginFrame].Callback =
	KnownEvents[(int)EKnownEvent::Misc_EndFrame].Callback =
	KnownEvents[(int)EKnownEvent::Logging_LogMessage].Callback =
	KnownEvents[(int)EKnownEvent::PlatformFile_BeginOpen].Callback =
	KnownEvents[(int)EKnownEvent::PlatformFile_EndOpen].Callback =
	KnownEvents[(int)EKnownEvent::PlatformFile_BeginRead].Callback =
	KnownEvents[(int)EKnownEvent::PlatformFile_EndRead].Callback =
	KnownEvents[(int)EKnownEvent::PlatformFile_BeginClose].Callback =
	KnownEvents[(int)EKnownEvent::PlatformFile_EndClose].Callback =
	KnownEvents[(int)EKnownEvent::CpuProfiler_EndThread].Callback =
	KnownEvents[(int)EKnownEvent::Memory_Marker].Callback =
	KnownEvents[(int)EKnownEvent::LLM_TagValue].Callback =
	KnownEvents[(int)EKnownEvent::SlateTrace_AddWidget].Callback =
		[this](const FOnEventContext& Context)
		{
			const uint64 Cycle = Context.EventData.GetValue<uint64>("Cycle");
			UpdateThreadTime(Context, Cycle);
		};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::OnAnalysisBegin(const UE::Trace::IAnalyzer::FOnAnalysisContext& Context)
{
	Reset();
	RegisterCallbacks();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::OnAnalysisEnd()
{
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::OnNewEvent(const UE::Trace::IAnalyzer::FEventTypeInfo& TypeInfo)
{
	const uint32 EventId = TypeInfo.GetId();
	if (!ensure(EventId <= MaxKnownEventId))
	{
		return;
	}

	const ANSICHAR* LoggerName = TypeInfo.GetLoggerName();
	check(LoggerName != nullptr);

	const ANSICHAR* EventName = TypeInfo.GetName();
	check(EventName != nullptr);

	#define BEGIN_KNOWN_LOGGER(_Name) \
		if (FCStringAnsi::Strcmp(LoggerName, _Name) == 0) \
		{

	#define KNOWN_EVENT(_Name,_Id) \
			if (FCStringAnsi::Strcmp(EventName, _Name) == 0) \
			{ \
				KnownEvents[(int)EKnownEvent::_Id].Id = EventId; \
				KnownEventMap[EventId] = EKnownEvent::_Id; \
				return; \
			}

	#define END_KNOWN_LOGGER() \
			return; \
		}

	BEGIN_KNOWN_LOGGER("$Trace")
		KNOWN_EVENT("NewEvent", Trace_NewEvent)
		//KNOWN_EVENT("NewTrace", Trace_NewTrace)
		//KNOWN_EVENT("ThreadTiming", Trace_ThreadTiming)
		//KNOWN_EVENT("ThreadInfo", Trace_ThreadInfo)
		//KNOWN_EVENT("ThreadGroupBegin", Trace_ThreadGroupBegin)
		//KNOWN_EVENT("ThreadGroupEnd", Trace_ThreadGroupEnd)
		//KNOWN_EVENT("ChannelAnnounce", Trace_ChannelAnnounce_Old) // deprecated
	END_KNOWN_LOGGER()

	//BEGIN_KNOWN_LOGGER("Trace")
		//KNOWN_EVENT("ChannelAnnounce", Trace_ChannelAnnounce)
		//KNOWN_EVENT("ChannelAnnounce", Trace_ChannelToggle)
	//END_KNOWN_LOGGER()

	//BEGIN_KNOWN_LOGGER("Diagnostics")
		//KNOWN_EVENT("Session", Diagnostics_Session) // deprecated
		//KNOWN_EVENT("Session2", Diagnostics_Session2)
	//END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("Misc")
		//KNOWN_EVENT("CreateThread", Misc_CreateThread)
		//KNOWN_EVENT("SetThreadGroup", Misc_SetThreadGroup)
		//KNOWN_EVENT("BeginThreadGroupScope", Misc_BeginThreadGroupScope)
		//KNOWN_EVENT("BookmarkSpec", Misc_BookmarkSpec)
		KNOWN_EVENT("Bookmark", Misc_Bookmark)
		KNOWN_EVENT("ScreenshotHeader", Misc_ScreenshotHeader)
		KNOWN_EVENT("RegionBegin", Misc_RegionBegin)
		KNOWN_EVENT("RegionBeginWithId", Misc_RegionBeginWithId)
		KNOWN_EVENT("RegionEnd", Misc_RegionEnd)
		KNOWN_EVENT("RegionEndWithId", Misc_RegionEndWithId)
		//KNOWN_EVENT("BeginGameFrame", Misc_BeginGameFrame)
		//KNOWN_EVENT("EndGameFrame", Misc_EndGameFrame)
		//KNOWN_EVENT("BeginRenderFrame", Misc_BeginRenderFrame)
		//KNOWN_EVENT("EndRenderFrame", Misc_EndRenderFrame)
		KNOWN_EVENT("BeginFrame", Misc_BeginFrame)
		KNOWN_EVENT("EndFrame", Misc_EndFrame)
	END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("Logging")
		//KNOWN_EVENT("LogCategory", Logging_LogCategory)
		//KNOWN_EVENT("LogMessageSpec", Logging_LogMessageSpec)
		KNOWN_EVENT("LogMessage", Logging_LogMessage)
	END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("PlatformFile")
		KNOWN_EVENT("BeginOpen", PlatformFile_BeginOpen)
		KNOWN_EVENT("EndOpen", PlatformFile_EndOpen)
		KNOWN_EVENT("BeginRead", PlatformFile_BeginRead)
		KNOWN_EVENT("EndRead", PlatformFile_EndRead)
		KNOWN_EVENT("BeginClose", PlatformFile_BeginClose)
		KNOWN_EVENT("EndClose", PlatformFile_EndClose)
	END_KNOWN_LOGGER()

	//BEGIN_KNOWN_LOGGER("LoadTime")
		//KNOWN_EVENT("ClassInfo", LoadTime_ClassInfo)
		//KNOWN_EVENT("NewAsyncPackage", LoadTime_NewAsyncPackage)
	//END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("GpuProfiler")
		//KNOWN_EVENT("EventSpec", GpuProfiler_EventSpec) // deprecated
		//KNOWN_EVENT("Frame", GpuProfiler_Frame) // deprecated
		//KNOWN_EVENT("Frame2", GpuProfiler_Frame2) // deprecated
		//KNOWN_EVENT("Init", GpuProfiler_Init)
		//KNOWN_EVENT("QueueSpec", GpuProfiler_QueueSpec)
		//KNOWN_EVENT("EventFrameBoundary", GpuProfiler_EventFrameBoundary)
		//KNOWN_EVENT("EventBreadcrumbSpec", GpuProfiler_EventBreadcrumbSpec)
		KNOWN_EVENT("EventBeginBreadcrumb", GpuProfiler_EventBeginBreadcrumb)
		KNOWN_EVENT("EventEndBreadcrumb", GpuProfiler_EventEndBreadcrumb)
		KNOWN_EVENT("EventBeginWork", GpuProfiler_EventBeginWork)
		KNOWN_EVENT("EventEndWork", GpuProfiler_EventEndWork)
		KNOWN_EVENT("EventWait", GpuProfiler_EventWait)
		//KNOWN_EVENT("EventStats", GpuProfiler_EventStats)
		KNOWN_EVENT("SignalFence", GpuProfiler_SignalFence)
		KNOWN_EVENT("WaitFence", GpuProfiler_WaitFence)
	END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("CpuProfiler")
		//KNOWN_EVENT("EventSpec", CpuProfiler_EventSpec)
		//KNOWN_EVENT("MetadataSpec", CpuProfiler_MetadataSpec)
		KNOWN_EVENT("Metadata", CpuProfiler_Metadata)
		KNOWN_EVENT("EventBatchV3", CpuProfiler_EventBatchV3)
		KNOWN_EVENT("EventBatchV2", CpuProfiler_EventBatchV2) // deprecated
		KNOWN_EVENT("EventBatch", CpuProfiler_EventBatch) // deprecated
		KNOWN_EVENT("EndCapture", CpuProfiler_EndCapture) // deprecated
		KNOWN_EVENT("EndThread", CpuProfiler_EndThread)
	END_KNOWN_LOGGER()

	//BEGIN_KNOWN_LOGGER("Stats")
		//KNOWN_EVENT("Spec", Stats_Spec)
	//END_KNOWN_LOGGER()

	//BEGIN_KNOWN_LOGGER("Counters")
		//KNOWN_EVENT("Spec", Counters_Spec)
	//END_KNOWN_LOGGER()

	//BEGIN_KNOWN_LOGGER("CsvProfiler")
		//KNOWN_EVENT("RegisterCategory", CsvProfiler_RegisterCategory)
		//KNOWN_EVENT("DefineInlineStat", CsvProfiler_DefineInlineStat)
		//KNOWN_EVENT("DefineDeclaredStat", CsvProfiler_DefineDeclaredStat)
		//KNOWN_EVENT("Metadata", CsvProfiler_Metadata)
		//KNOWN_EVENT("BeginCapture", CsvProfiler_BeginCapture)
	//END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("Memory")
		KNOWN_EVENT("Marker", Memory_Marker)
	END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("LLM")
		KNOWN_EVENT("TagValue", LLM_TagValue)
		//KNOWN_EVENT("TagsSpec", LLM_TagsSpec)
		//KNOWN_EVENT("TrackerSpec", LLM_TrackerSpec)
	END_KNOWN_LOGGER()

	BEGIN_KNOWN_LOGGER("SlateTrace")
		KNOWN_EVENT("AddWidget", SlateTrace_AddWidget)
	END_KNOWN_LOGGER()

	#undef BEGIN_KNOWN_LOGGER
	#undef KNOWN_EVENT
	#undef END_KNOWN_LOGGER

	const uint32 FieldCount = TypeInfo.GetFieldCount();
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo = *(TypeInfo.GetFieldInfo(FieldIndex));
		if (!FieldInfo.IsArray() &&
			FieldInfo.GetType() == UE::Trace::IAnalyzer::FEventFieldInfo::EType::Integer &&
			!FieldInfo.IsSigned() &&
			FieldInfo.GetTypeSize() == 8 &&
			FCStringAnsi::Strcmp(FieldInfo.GetName(), "Cycle") == 0)
		{
			CycleEvents.Add(EventId);
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::OnEvent(uint32 EventId, const UE::Trace::IAnalyzer::FOnEventContext& Context)
{
	if (ensure(EventId <= MaxKnownEventId))
	{
		EKnownEvent KnownEvent = KnownEventMap[EventId];
		check((int)KnownEvent < (int)EKnownEvent::Count);
		const FKnownEvent& Event = KnownEvents[(int)KnownEvent];

		if (Event.Callback)
		{
			Event.Callback(Context);
			return;
		}
	}

	// Events with a uint64 "Cycle" field...
	if (CycleEvents.Contains(EventId))
	{
		const uint64 Cycle = Context.EventData.GetValue<uint64>("Cycle");
		OnUpdateThreadTime(Context.ThreadInfo.GetId(), Context, Cycle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::ProcessCpuEventBatchV1(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize)
{
	// Trace analysis was changed to be able to provide a suitable id. Prior to
	// this users of Trace would send along their own thread ids. For backwards
	// compatibility we'll bias field thread ids to avoid collision with Trace's.
	constexpr uint32 Bias = 0x70000000;
	uint32 ThreadId = Context.EventData.GetValue<uint32>("ThreadId", 0);
	ThreadId |= ThreadId ? Bias : Context.ThreadInfo.GetId();

	FThreadInfo& Thread = GetOrAddThread(ThreadId);

	Thread.Cycle = ProcessCpuEventBatchV1(Context, BufferPtr, BufferSize, Thread.Cycle);

	OnUpdateThreadTime(ThreadId, Context, Thread.Cycle);

	if (ThreadId != Context.ThreadInfo.GetId()) // ThreadId has bias (it's using the "ThreadId" field)
	{
		OnUpdateThreadTime(Context.ThreadInfo.GetId(), Context, Thread.Cycle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FKnownEventProcessor::ProcessCpuEventBatchV1(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize, uint64 ThreadCycle)
{
	const uint8* BufferEnd = BufferPtr + BufferSize;
	while (BufferPtr < BufferEnd)
	{
		uint64 DecodedCycle = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		ThreadCycle += (DecodedCycle >> 1);

		if (DecodedCycle & 1ull)
		{
			uint32 SpecId = (uint32)FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		}
	}
	if (!ensure(BufferPtr == BufferEnd))
	{
		//error
	}

	return ThreadCycle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::ProcessCpuEventBatchV2V3(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize, int32 Version)
{
	const uint32 ThreadId = Context.ThreadInfo.GetId();
	FThreadInfo& Thread = GetOrAddThread(ThreadId);

	Thread.Cycle = ProcessCpuEventBatchV2V3(Context, BufferPtr, BufferSize, Version, Thread.Cycle);

	OnUpdateThreadTime(ThreadId, Context, Thread.Cycle);
}
	
////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FKnownEventProcessor::ProcessCpuEventBatchV2V3(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize, int32 Version, uint64 ThreadCycle)
{
	check(Context.EventTime.GetTimestamp() == 0);
	const uint64 BaseCycle = Context.EventTime.AsCycle64();

	const uint8* BufferEnd = BufferPtr + BufferSize;
	while (BufferPtr < BufferEnd)
	{
		uint64 DecodedCycle = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		uint64 ActualCycle = (DecodedCycle >> 2);

		// ActualCycle larger or equal to ThreadCycle means we have a new base value.
		if (ActualCycle < ThreadCycle)
		{
			ActualCycle += ThreadCycle;
		}

		// If we late connect we will be joining the cycle stream mid-flow and
		// will have missed out on it's base timestamp. Reconstruct it here.
		if (ActualCycle < BaseCycle)
		{
			ActualCycle += BaseCycle;
		}

		if (DecodedCycle & 2ull) // coroutine
		{
			if (DecodedCycle & 1ull) // Restore
			{
				uint64 CoroutineId = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
				uint32 TimerScopeDepth = (uint32)FTraceAnalyzerUtils::Decode7bit(BufferPtr);
			}
			else // Save
			{
				uint32 TimerScopeDepth = (uint32)FTraceAnalyzerUtils::Decode7bit(BufferPtr);
			}
		}
		else // normal CPU events
		{
			if (DecodedCycle & 1ull)
			{
				uint32 SpecId = (uint32)FTraceAnalyzerUtils::Decode7bit(BufferPtr);
#if 0
				if (Version == 3)
				{
					if (SpecId & 1u) // The last bit is set if this is a metadata id.
					{
						uint32 MetadataId = SpecId >> 1;
					}
					else
					{
						SpecId = SpecId >> 1;
					}
				}
				else // Version == 2
				{
					//SpecId
				}
#endif
			}
		}

		check(ActualCycle > 0);
		ThreadCycle = ActualCycle;
	}
	if (!ensure(BufferPtr == BufferEnd))
	{
		// error
	}

	return ThreadCycle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FKnownEventProcessor::FThreadInfo& FKnownEventProcessor::GetOrAddThread(uint32 ThreadId)
{
	FThreadInfo* FoundThread = ThreadMap.Find(ThreadId);
	if (FoundThread)
	{
		return *FoundThread;
	}
	return ThreadMap.Add(ThreadId, { ThreadId });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::UpdateThreadTime(const UE::Trace::IAnalyzer::FOnEventContext& Context, uint64 Cycle)
{
	if (Cycle != 0)
	{
		const uint32 ThreadId = Context.ThreadInfo.GetId();
		OnUpdateThreadTime(ThreadId, Context, Cycle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FKnownEventProcessor::OnUpdateThreadTime(uint32 ThreadId, const UE::Trace::IAnalyzer::FOnEventContext& Context, uint64 Cycle)
{
	if (OnUpdateThreadTimeCallback)
	{
		OnUpdateThreadTimeCallback(ThreadId, Context, Cycle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
