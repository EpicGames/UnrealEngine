// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Templates/Function.h"

// TraceAnalysis
#include "Trace/Analyzer.h"

namespace TraceServices
{

class FKnownEventProcessor
{
public:
	typedef TFunction<void(uint32 ThreadId, const UE::Trace::IAnalyzer::FOnEventContext& Context, uint64 Cycle)> FOnUpdateThreadTimeFunc;

private:
	enum class EKnownEvent : uint32
	{
		Invalid,

		Trace_NewEvent,
		//Trace_NewTrace,
		//Trace_ThreadTiming,
		//Trace_ThreadInfo,
		//Trace_ThreadGroupBegin,
		//Trace_ThreadGroupEnd,
		//Trace_ChannelAnnounce_Old, // "$Trace", deprecated in 4.26

		//Trace_ChannelAnnounce, // "Trace", new in 4.26
		//Trace_ChannelToggle,

		//Diagnostics_Session,
		//Diagnostics_Session2,

		//Misc_CreateThread,
		//Misc_SetThreadGroup,
		//Misc_BeginThreadGroupScope,
		//Misc_BookmarkSpec,
		Misc_Bookmark,
		Misc_ScreenshotHeader,
		//Misc_ScreenshotChunk,
		Misc_RegionBegin,
		Misc_RegionBeginWithId,
		Misc_RegionEnd,
		Misc_RegionEndWithId,
		//Misc_BeginGameFrame, // deprecated
		//Misc_EndGameFrame, // deprecated
		//Misc_BeginRenderFrame, // deprecated
		//Misc_EndRenderFrame, // deprecated
		Misc_BeginFrame,
		Misc_EndFrame,

		//Logging_LogCategory,
		//Logging_LogMessageSpec,
		Logging_LogMessage,

		PlatformFile_BeginOpen,
		PlatformFile_EndOpen,
		PlatformFile_BeginRead,
		PlatformFile_EndRead,
		PlatformFile_BeginClose,
		PlatformFile_EndClose,

		//LoadTime_ClassInfo,
		//LoadTime_NewAsyncPackage,

		//GpuProfiler_EventSpec, // old GpuProfiler, deprecated
		//GpuProfiler_Frame, // old GpuProfiler, deprecated
		//GpuProfiler_Frame2, // old GpuProfiler, deprecated
		//GpuProfiler_Init,
		//GpuProfiler_QueueSpec,
		//GpuProfiler_EventFrameBoundary,
		//GpuProfiler_EventBreadcrumbSpec,
		GpuProfiler_EventBeginBreadcrumb,
		GpuProfiler_EventEndBreadcrumb,
		GpuProfiler_EventBeginWork,
		GpuProfiler_EventEndWork,
		GpuProfiler_EventWait,
		//GpuProfiler_EventStats,
		GpuProfiler_SignalFence,
		GpuProfiler_WaitFence,

		//CpuProfiler_EventSpec,
		//CpuProfiler_MetadataSpec, // added in UE 5.x
		CpuProfiler_Metadata, // added in UE 5.x
		CpuProfiler_EventBatchV3, // added in UE 5.6
		CpuProfiler_EventBatchV2, // backward compatibility; added in UE 5.1, removed in 5.6
		CpuProfiler_EventBatch, // backward compatibility; removed in UE 5.1
		CpuProfiler_EndCapture, // backward compatibility; removed in UE 5.1
		CpuProfiler_EndThread, // "Cycle" field added in UE 5.4

		//Stats_Spec,

		//Counters_Spec,

		//CsvProfiler_RegisterCategory,
		//CsvProfiler_DefineInlineStat,
		//CsvProfiler_DefineDeclaredStat,
		//CsvProfiler_Metadata,
		//CsvProfiler_BeginCapture,

		Memory_Marker,

		LLM_TagValue,
		//LLM_TagsSpec,
		//LLM_TrackerSpec,

		SlateTrace_AddWidget,

		Count
	};

	typedef TFunction<void(const UE::Trace::IAnalyzer::FOnEventContext& Context)> KnownEventCallback;

	struct FKnownEvent
	{
		uint32 Id = InvalidEventId;
		KnownEventCallback Callback = nullptr;
	};

	struct FThreadInfo
	{
		uint32 Id = 0;
		uint32 Reserved = 0; // for padding
		uint64 Cycle = 0;
	};

public:
	FKnownEventProcessor();
	virtual ~FKnownEventProcessor();

	void Reset();

	void SetOnUpdateThreadTime(const FOnUpdateThreadTimeFunc& Callback)
	{
		OnUpdateThreadTimeCallback = Callback;
	}

	void OnAnalysisBegin(const UE::Trace::IAnalyzer::FOnAnalysisContext& Context);
	void OnAnalysisEnd();
	void OnNewEvent(const UE::Trace::IAnalyzer::FEventTypeInfo& TypeInfo);
	void OnEvent(uint32 EventId, const UE::Trace::IAnalyzer::FOnEventContext& Context);

private:
	bool IsKnownEvent(uint32 EventId, EKnownEvent KnownEvent) const
	{
		return EventId == KnownEvents[(int)KnownEvent].Id;
	}

	void RegisterCallbacks();

	void ProcessCpuEventBatchV1(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize);
	uint64 ProcessCpuEventBatchV1(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize, uint64 ThreadCycle);

	void ProcessCpuEventBatchV2V3(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize, int32 Version);
	uint64 ProcessCpuEventBatchV2V3(const UE::Trace::IAnalyzer::FOnEventContext& Context, const uint8* BufferPtr, uint32 BufferSize, int32 Version, uint64 ThreadCycle);

	FThreadInfo& GetOrAddThread(uint32 ThreadId);

	void UpdateThreadTime(const UE::Trace::IAnalyzer::FOnEventContext& Context, uint64 Cycle);
	void OnUpdateThreadTime(uint32 ThreadId, const UE::Trace::IAnalyzer::FOnEventContext& Context, uint64 Cycle);

private:
	static constexpr uint32 InvalidEventId = ~0u;
	static constexpr uint32 MaxKnownEventId = 0xFFFF;

	TArray<FKnownEvent> KnownEvents; // [EKnownEvent::Count]
	TArray<EKnownEvent> KnownEventMap; // index == EventId --> EKnownEvent; [MaxKnownEventId + 1]
	TSet<uint32> CycleEvents; // events with a uint64 "Cycle" field.

	TMap<uint32, FThreadInfo> ThreadMap; // ThreadId --> FThreadInfo

	FOnUpdateThreadTimeFunc OnUpdateThreadTimeCallback;
};

} // namespace TraceServices
