// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadTimeTraceAnalysis.h"

#include "Analyzers/MiscTraceAnalysis.h"
#include "Common/FormatArgs.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "Model/LoadTimeProfilerPrivate.h"
#include "Serialization/LoadTimeTrace.h"

#include <limits>

// Enables backward compatibility with previous versions of "LoadTime" trace.
#define UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY 1

// Macro to log the errors detected during "LoadTime" trace analysis.
#define UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(...) \
{ \
	constexpr uint64 MaxErrorCount = 100; \
	if (++ErrorCount <= MaxErrorCount) \
	{ \
		UE_LOG(LogTraceServices, Error, TEXT("[LoadTime] ") __VA_ARGS__); \
		if (ErrorCount == MaxErrorCount) \
		{ \
			UE_LOGF(LogTraceServices, Error, "[LoadTime] Too many errors! Further LoadTime analysis errors will not be reported anymore."); \
		} \
	} \
}

// Macro to log the warnings detected during "LoadTime" trace analysis.
#define UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(...) \
{ \
	constexpr uint64 MaxWarningCount = 100; \
	if (++WarningCount <= MaxWarningCount) \
	{ \
		UE_LOG(LogTraceServices, Warning, TEXT("[LoadTime] ") __VA_ARGS__); \
		if (WarningCount == MaxWarningCount) \
		{ \
			UE_LOGF(LogTraceServices, Warning, "[LoadTime] Too many warnings! Further LoadTime analysis warnings will not be reported anymore."); \
		} \
	} \
}

// Debug macro to log extra warnings detected during "LoadTime" trace analysis.
#define UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX(...) //UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(__VA_ARGS__)

// Macro to log the important "LoadTime" trace analysis stats.
#define UE_LOAD_TIME_TRACE_ANALYSIS_LOG(Severity, ...) { UE_LOG(LogTraceServices, Severity, TEXT("[LoadTime] ") __VA_ARGS__); }

// Debug macro to log the name and parameters of "LoadTime" trace events analyzed.
#define UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(...) //{ UE_LOG(LogTraceServices, Log, TEXT("[LoadTime] ") __VA_ARGS__); }

// Debug macro to log additional info while analyzing a "LoadTime" trace event.
#define UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(...) //{ UE_LOG(LogTraceServices, Log, TEXT("[LoadTime]   --> ") __VA_ARGS__); }

namespace TraceServices
{

FAsyncLoadingTraceAnalyzer::FAsyncLoadingTraceAnalyzer(IAnalysisSession& InSession, FLoadTimeProfilerProvider& InLoadTimeProfilerProvider)
	: Session(InSession)
	, LoadTimeProfilerProvider(InLoadTimeProfilerProvider)
{
}

FAsyncLoadingTraceAnalyzer::~FAsyncLoadingTraceAnalyzer()
{
	for (const auto& KV : ThreadStatesMap)
	{
		delete KV.Value;
	}
	for (const auto& KV : ActiveAsyncPackagesMap)
	{
		delete KV.Value;
	}
	for (const auto& KV : ActiveRequestsMap)
	{
		delete KV.Value;
	}
	for (const auto& KV : ActiveLinkersMap)
	{
		delete KV.Value;
	}
}

FAsyncLoadingTraceAnalyzer::FThreadState& FAsyncLoadingTraceAnalyzer::GetThreadState(uint32 ThreadId)
{
	FThreadState* ThreadState = ThreadStatesMap.FindRef(ThreadId);
	if (!ThreadState)
	{
		ThreadState = new FThreadState();
		ThreadStatesMap.Add(ThreadId, ThreadState);
		FAnalysisSessionEditScope _(Session);
		ThreadState->CpuTimeline = &LoadTimeProfilerProvider.EditCpuTimeline(ThreadId);
	}
	return *ThreadState;
}

const FClassInfo* FAsyncLoadingTraceAnalyzer::GetClassInfo(uint64 ClassPtr) const
{
	const FClassInfo* const* ClassInfo = ClassInfosMap.Find(ClassPtr);
	if (ClassInfo)
	{
		return *ClassInfo;
	}
	else
	{
		return nullptr;
	}
}

void FAsyncLoadingTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_StartAsyncLoading, "LoadTime", "StartAsyncLoading");
	Builder.RouteEvent(RouteId_SuspendAsyncLoading, "LoadTime", "SuspendAsyncLoading");
	Builder.RouteEvent(RouteId_ResumeAsyncLoading, "LoadTime", "ResumeAsyncLoading");
	Builder.RouteEvent(RouteId_PackageSummary, "LoadTime", "PackageSummary");
	Builder.RouteEvent(RouteId_BeginProcessSummary, "LoadTime", "BeginProcessSummary");
	Builder.RouteEvent(RouteId_EndProcessSummary, "LoadTime", "EndProcessSummary");
	Builder.RouteEvent(RouteId_BeginCreateExport, "LoadTime", "BeginCreateExport");
	Builder.RouteEvent(RouteId_EndCreateExport, "LoadTime", "EndCreateExport");
	Builder.RouteEvent(RouteId_BeginSerializeExport, "LoadTime", "BeginSerializeExport");
	Builder.RouteEvent(RouteId_EndSerializeExport, "LoadTime", "EndSerializeExport");
	Builder.RouteEvent(RouteId_BeginPostLoad, "LoadTime", "BeginPostLoad");
	Builder.RouteEvent(RouteId_EndPostLoad, "LoadTime", "EndPostLoad");
	Builder.RouteEvent(RouteId_BeginPostLoadObject, "LoadTime", "BeginPostLoadObject");
	Builder.RouteEvent(RouteId_EndPostLoadObject, "LoadTime", "EndPostLoadObject");
	Builder.RouteEvent(RouteId_NewAsyncPackage, "LoadTime", "NewAsyncPackage");
	Builder.RouteEvent(RouteId_DestroyAsyncPackage, "LoadTime", "DestroyAsyncPackage");
	Builder.RouteEvent(RouteId_NewLinker, "LoadTime", "NewLinker");
	Builder.RouteEvent(RouteId_DestroyLinker, "LoadTime", "DestroyLinker");
	Builder.RouteEvent(RouteId_BeginRequest, "LoadTime", "BeginRequest");
	Builder.RouteEvent(RouteId_EndRequest, "LoadTime", "EndRequest");
	Builder.RouteEvent(RouteId_BeginRequestGroup, "LoadTime", "BeginRequestGroup");
	Builder.RouteEvent(RouteId_EndRequestGroup, "LoadTime", "EndRequestGroup");
	Builder.RouteEvent(RouteId_AsyncPackageRequestAssociation, "LoadTime", "AsyncPackageRequestAssociation");
	Builder.RouteEvent(RouteId_AsyncPackageLinkerAssociation, "LoadTime", "AsyncPackageLinkerAssociation");
	Builder.RouteEvent(RouteId_AsyncPackageImportDependency, "LoadTime", "AsyncPackageImportDependency");
	Builder.RouteEvent(RouteId_ClassInfo, "LoadTime", "ClassInfo");
	Builder.RouteEvent(RouteId_BatchIssued, "IoDispatcher", "BatchIssued");
	Builder.RouteEvent(RouteId_BatchResolved, "IoDispatcher", "BatchResolved");

#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	Builder.RouteEvent(RouteId_BeginObjectScope, "LoadTime", "BeginObjectScope");
	Builder.RouteEvent(RouteId_EndObjectScope, "LoadTime", "EndObjectScope");
	Builder.RouteEvent(RouteId_BeginPostLoadExport, "LoadTime", "BeginPostLoadExport");
	Builder.RouteEvent(RouteId_EndPostLoadExport, "LoadTime", "EndPostLoadExport");
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
}

void FAsyncLoadingTraceAnalyzer::OnAnalysisEnd()
{
	if (ErrorCount > 0)
	{
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG(Error, TEXT("%llu errors, %llu warnings"), ErrorCount, WarningCount);
	}
	else if (WarningCount > 0)
	{
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG(Warning, TEXT("%llu errors, %llu warnings"), ErrorCount, WarningCount);
	}

	UE_LOAD_TIME_TRACE_ANALYSIS_LOG(Log, TEXT("Analysis completed (%d threads, %d active async packages, %d active requests + %d fake requests, %d active linkers, %d active batches, %d exports, %d classes)."),
		ThreadStatesMap.Num(),
		ActiveAsyncPackagesMap.Num(),
		ActiveRequestsMap.Num(),
		FakeRequestsStack.Num(),
		ActiveLinkersMap.Num(),
		ActiveBatchesMap.Num(),
		ExportsMap.Num(),
		ClassInfosMap.Num());
}

bool FAsyncLoadingTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FAsyncLoadingTraceAnalyzer"));

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_StartAsyncLoading:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("StartAsyncLoading Time=%f"), Time);
		FAnalysisSessionEditScope _(Session);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_SuspendAsyncLoading:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("SuspendAsyncLoading Time=%f"), Time);
		FAnalysisSessionEditScope _(Session);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_ResumeAsyncLoading:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("ResumeAsyncLoading Time=%f"), Time);
		FAnalysisSessionEditScope _(Session);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_PackageSummary:
	{
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("PackageSummary AsyncPackage=%llu"), AsyncPackagePtr);

		FAsyncPackageState* AsyncPackage = ActiveAsyncPackagesMap.FindRef(AsyncPackagePtr);
		if (!AsyncPackage)
		{
#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
			if (AsyncPackagePtr == 0)
			{
				AsyncPackagePtr = EventData.GetValue<uint64>("Linker");
			}
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
			FLinkerLoadState* Linker = ActiveLinkersMap.FindRef(AsyncPackagePtr);
			if (Linker)
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("%llu (%p) is a Linker; Linker->AsyncPackage=%p"), AsyncPackagePtr, Linker, Linker->AsyncPackage);
				AsyncPackage = Linker->AsyncPackage;
			}
		}
		if (AsyncPackage)
		{
			FPackageInfo* PackageInfo = AsyncPackage->PackageInfo;
			if (PackageInfo)
			{
				FAnalysisSessionEditScope _(Session);
				FString PackageName = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
				if (PackageName.Len())
				{
					PackageInfo->Name = Session.StoreString(*PackageName);
				}
				FPackageSummaryInfo& Summary = PackageInfo->Summary;
				Summary.TotalHeaderSize = EventData.GetValue<uint32>("TotalHeaderSize");
				Summary.ImportCount = EventData.GetValue<uint32>("ImportCount");
				Summary.ExportCount = EventData.GetValue<uint32>("ExportCount");
				Summary.Priority = EventData.GetValue<int32>("Priority", 0);  // added in UE 5.4
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("  TotalHeaderSize=%u ImportCount=%u ExportCount=%u Priority=%d"), Summary.TotalHeaderSize, Summary.ImportCount, Summary.ExportCount, Summary.Priority);
			}
			else
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown PackageInfo!"));
			}
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
		}
		break;
	}

	case RouteId_BeginProcessSummary:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginProcessSummary Time=%f AsyncPackage=%llu"), Time, AsyncPackagePtr);

		FPackageInfo* PackageInfo = nullptr;
		FAsyncPackageState* AsyncPackage = ActiveAsyncPackagesMap.FindRef(AsyncPackagePtr);
		if (AsyncPackage)
		{
			PackageInfo = AsyncPackage->PackageInfo;
			if (!PackageInfo)
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown PackageInfo!"));
			}
		}
		else
		{
			FLinkerLoadState* Linker = ActiveLinkersMap.FindRef(AsyncPackagePtr);
			if (Linker)
			{
				if (Linker->AsyncPackage)
				{
					PackageInfo = Linker->AsyncPackage->PackageInfo;
					Linker->PackageInfo = PackageInfo;
					if (!PackageInfo)
					{
						UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown PackageInfo!"));
					}
				}
				else
				{
					{
						FAnalysisSessionEditScope _(Session);
						PackageInfo = &LoadTimeProfilerProvider.CreatePackage();
					}
					FAsyncPackageState* FakeAsyncPackageState = new FAsyncPackageState();
					FakeAsyncPackageState->PackageInfo = PackageInfo;
					FakeAsyncPackageState->Linker = Linker;
					Linker->PackageInfo = PackageInfo;
					Linker->AsyncPackage = FakeAsyncPackageState;
					Linker->bHasFakeAsyncPackageState = true;
					UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("Linker %llu (%p) : set AsyncPackage = new (fake) FAsyncPackageState %p"), AsyncPackagePtr, Linker, FakeAsyncPackageState);
				}
			}
			else
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
			}
		}

		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		FAnalysisSessionEditScope _(Session);
		ThreadState.EnterScope(Time, PackageInfo);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_EndProcessSummary:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndProcessSummary Time=%f"), Time);

		if (ThreadState.CpuScopeStackDepth <= 0)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndProcessSummary without Begin event!"));
			break;
		}

		if (ThreadState.GetCurrentScopeEventType() != LoadTimeProfilerObjectEventType_None)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndProcessSummary does not match the Begin event type!"));
			break;
		}

		FAnalysisSessionEditScope _(Session);
		ThreadState.LeaveScope(Time);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_BeginCreateExport:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginCreateExport Time=%f AsyncPackage=%llu"), Time, AsyncPackagePtr);

		FLinkerLoadState* Linker = nullptr;
		FAsyncPackageState* AsyncPackage = ActiveAsyncPackagesMap.FindRef(AsyncPackagePtr);
		if (!AsyncPackage)
		{
#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
			if (AsyncPackagePtr == 0)
			{
				AsyncPackagePtr = EventData.GetValue<uint64>("Linker");
			}
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
			Linker = ActiveLinkersMap.FindRef(AsyncPackagePtr);
			if (Linker)
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("%llu (%p) is a Linker; Linker->AsyncPackage=%p"), AsyncPackagePtr, Linker, Linker->AsyncPackage);
				AsyncPackage = Linker->AsyncPackage;
			}
		}

		FAnalysisSessionEditScope _(Session);
		FPackageExportInfo& Export = LoadTimeProfilerProvider.CreateExport();
#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
		Export.SerialSize = EventData.GetValue<uint64>("SerialSize");
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
		if (AsyncPackage)
		{
			FPackageInfo* PackageInfo = AsyncPackage->PackageInfo;
			if (PackageInfo)
			{
				PackageInfo->Exports.Add(&Export);
				Export.Package = PackageInfo;
			}
			else
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
			}
		}
#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
		else if (Linker)
		{
			if (!Linker->PackageInfo)
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown PackageInfo!"));
				if (!UnknownPackageInfo)
				{
					UnknownPackageInfo = &LoadTimeProfilerProvider.CreatePackage();
					UnknownPackageInfo->Name = Session.StoreString(TEXT("[unknown]"));
				}
				Linker->PackageInfo = UnknownPackageInfo;
			}
			Linker->PackageInfo->Exports.Add(&Export);
			Export.Package = Linker->PackageInfo;
		}
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
		}
		ThreadState.EnterScope(Time, &Export, LoadTimeProfilerObjectEventType_Create);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_EndCreateExport:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 ObjectPtr = EventData.GetValue<uint64>("Object");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndCreateExport Time=%f Object=%llu"), Time, ObjectPtr);

		if (ThreadState.CpuScopeStackDepth <= 0)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndCreateExport without Begin event!"));
			break;
		}

		if (ThreadState.GetCurrentScopeEventType() != LoadTimeProfilerObjectEventType_Create)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndCreateExport does not match the Begin event type!"));
			break;
		}

		FAnalysisSessionEditScope _(Session);
		FPackageExportInfo* Export = ThreadState.GetCurrentExportScope();
		if (Export)
		{
			ExportsMap.Add(ObjectPtr, Export);
			const FClassInfo* ObjectClass = GetClassInfo(EventData.GetValue<uint64>("Class"));
			Export->Class = ObjectClass;
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown export scope!"));
		}
		ThreadState.LeaveScope(Time);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_BeginSerializeExport:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 ObjectPtr = EventData.GetValue<uint64>("Object");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginSerializeExport Time=%f Object=%llu"), Time, ObjectPtr);

		FPackageExportInfo* Export = ExportsMap.FindRef(ObjectPtr);

		FAnalysisSessionEditScope _(Session);
		if (Export)
		{
			uint64 SerialSize = EventData.GetValue<uint64>("SerialSize");
			if (SerialSize)
			{
				Export->SerialSize = SerialSize;
			}
			if (Export->Package)
			{
				const_cast<FPackageInfo*>(Export->Package)->TotalExportsSerialSize += SerialSize;
			}
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX(TEXT("Unknown export Object!"));
		}
		ThreadState.EnterScope(Time, Export, LoadTimeProfilerObjectEventType_Serialize);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_EndSerializeExport:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndSerializeExport Time=%f"), Time);

		if (ThreadState.CpuScopeStackDepth <= 0)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndSerializeExport without Begin event!"));
			break;
		}

		if (ThreadState.GetCurrentScopeEventType() != LoadTimeProfilerObjectEventType_Serialize)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndSerializeExport does not match the Begin event type!"));
			break;
		}

		FAnalysisSessionEditScope _(Session);
		ThreadState.LeaveScope(Time);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_BeginPostLoad:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginPostLoad ThreadId=%u --> PostLoadScopeDepth=%lli"), ThreadId, ThreadState.PostLoadScopeDepth + 1);

		++ThreadState.PostLoadScopeDepth;
		break;
	}

	case RouteId_EndPostLoad:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndPostLoad ThreadId=%u --> PostLoadScopeDepth=%lli"), ThreadId, ThreadState.PostLoadScopeDepth - 1);

		if (ThreadState.PostLoadScopeDepth > 0)
		{
			--ThreadState.PostLoadScopeDepth;
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Invalid PostLoad scope!"));
		}
		break;
	}

#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	case RouteId_BeginPostLoadExport:
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	case RouteId_BeginPostLoadObject:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 ObjectPtr = EventData.GetValue<uint64>("Object");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginPostLoadObject ThreadId=%u Time=%f Object=%llu"), ThreadId, Time, ObjectPtr);

		if (RouteId == RouteId_BeginPostLoadObject && !ThreadState.PostLoadScopeDepth)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX(TEXT("Not in a PostLoad scope!"));
			break;
		}

		FPackageExportInfo* Export = ExportsMap.FindRef(ObjectPtr);
		if (!Export)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX(TEXT("Unknown export Object!"));
		}

		FAnalysisSessionEditScope _(Session);
		ThreadState.EnterScope(Time, Export, LoadTimeProfilerObjectEventType_PostLoad);
		Session.UpdateDurationSeconds(Time);
		break;
	}

#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	case RouteId_EndPostLoadExport:
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	case RouteId_EndPostLoadObject:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndPostLoadObject ThreadId=%u Time=%f"), ThreadId, Time);

		if (RouteId == RouteId_EndPostLoadObject && !ThreadState.PostLoadScopeDepth)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX(TEXT("Not in a PostLoad scope!"));
			break;
		}

		if (ThreadState.CpuScopeStackDepth <= 0)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndPostLoadObject without Begin event!"));
			break;
		}

		if (ThreadState.GetCurrentScopeEventType() != LoadTimeProfilerObjectEventType_PostLoad)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndPostLoadObject does not match the Begin event type!"));
			break;
		}

		FAnalysisSessionEditScope _(Session);
		ThreadState.LeaveScope(Time);
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_BeginRequest:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 RequestId = EventData.GetValue<uint64>("RequestId");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginRequest Time=%f RequestId=%llu"), Time, RequestId);

		{
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Time);
		}

		FRequestState* RequestState = new FRequestState();
		RequestState->WallTimeStartCycle = Cycle;
		RequestState->WallTimeEndCycle = 0;
		RequestState->ThreadId = ThreadId;

		if (RequestId == 0)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("new (fake) FRequestState %p"), RequestState);
			FakeRequestsStack.Push(RequestState);
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("new FRequestState %p"), RequestState);
			ActiveRequestsMap.Add(RequestId, RequestState);
		}

		TSharedPtr<FRequestGroupState> RequestGroup = ThreadState.RequestGroupStack.Num() ? ThreadState.RequestGroupStack[0] : nullptr;
		if (!RequestGroup)
		{
			RequestGroup = MakeShared<FRequestGroupState>();
			RequestGroup->Name = TEXT("[ungrouped]");
		}
		RequestGroup->Requests.Add(RequestState);
		++RequestGroup->ActiveRequestsCount;
		RequestState->Group = RequestGroup;
		break;
	}

	case RouteId_EndRequest:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 RequestId = EventData.GetValue<uint64>("RequestId");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndRequest Time=%f RequestId=%llu"), Time, RequestId);

		{
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Time);
		}

		FRequestState* RequestState = nullptr;
		if (RequestId == 0)
		{
			if (!FakeRequestsStack.IsEmpty())
			{
				RequestState = FakeRequestsStack.Pop();
			}
		}
		else
		{
			ActiveRequestsMap.RemoveAndCopyValue(RequestId, RequestState);
		}
		if (RequestState)
		{
			RequestState->WallTimeEndCycle = Cycle;
			RequestState->Group->LatestEndCycle = FMath::Max(RequestState->Group->LatestEndCycle, RequestState->WallTimeEndCycle);
			--RequestState->Group->ActiveRequestsCount;
			if (RequestState->Group->LoadRequest && RequestState->Group->ActiveRequestsCount == 0)
			{
				FAnalysisSessionEditScope _(Session);
				double EndTime = Context.EventTime.AsSeconds(RequestState->Group->LatestEndCycle);
				RequestState->Group->LoadRequest->EndTime = EndTime;
				Session.UpdateDurationSeconds(EndTime);
			}

			for (FAsyncPackageState* State : RequestState->AsyncPackages)
			{
				if (State->Request == RequestState)
				{
					State->Request = nullptr;
				}
			}

			UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("delete FRequestState %p"), RequestState);
			delete RequestState;
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown Request!"));
		}
		break;
	}

	case RouteId_BeginRequestGroup:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		FString FormatString;
		const uint8* FormatArgs;
		if (EventData.GetString("Format", FormatString))
		{
			FormatArgs = EventData.GetArrayView<uint8>("FormatArgs").GetData();
		}
		else
		{
			const TCHAR* FormatStringPtr = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			FormatArgs = EventData.GetAttachment() + (FCString::Strlen(FormatStringPtr) + 1) * sizeof(TCHAR);
			FormatString = FormatStringPtr;
		}
		FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, *FormatString, FormatArgs);
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginRequestGroup ThreadId=%u Name=\"%s\""), ThreadId, FormatBuffer);

		TSharedRef<FRequestGroupState> GroupState = MakeShared<FRequestGroupState>();

		FAnalysisSessionEditScope _(Session);
		GroupState->Name = Session.StoreString(FormatBuffer);
		ThreadState.RequestGroupStack.Push(GroupState);
		break;
	}

	case RouteId_EndRequestGroup:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndRequestGroup ThreadId=%u"), ThreadId);

		if (ThreadState.RequestGroupStack.Num())
		{
			ThreadState.RequestGroupStack.Pop(EAllowShrinking::No);
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Invalid request group!"));
		}
		break;
	}

	case RouteId_NewAsyncPackage:
	{
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("NewAsyncPackage AsyncPackage=%llu"), AsyncPackagePtr);
		FAsyncPackageState* AsyncPackageState = new FAsyncPackageState();
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("new FAsyncPackageState %p"), AsyncPackageState);
		{
			FAnalysisSessionEditScope _(Session);
			AsyncPackageState->PackageInfo = &LoadTimeProfilerProvider.CreatePackage();
#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
			if (EventData.GetAttachmentSize())
			{
				const TCHAR* PackageName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
				AsyncPackageState->PackageInfo->Name = Session.StoreString(PackageName);
			}
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
		}
		ActiveAsyncPackagesMap.Add(AsyncPackagePtr, AsyncPackageState);
		break;
	}

	case RouteId_DestroyAsyncPackage:
	{
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("DestroyAsyncPackage AsyncPackage=%llu"), AsyncPackagePtr);
		FAsyncPackageState* AsyncPackageState;
		if (ActiveAsyncPackagesMap.RemoveAndCopyValue(AsyncPackagePtr, AsyncPackageState))
		{
			if (AsyncPackageState->Linker)
			{
				AsyncPackageState->Linker->AsyncPackage = nullptr;
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("Linker (%p) : reset AsyncPackage"), AsyncPackageState->Linker);
			}

			if (AsyncPackageState->Request)
			{
				AsyncPackageState->Request->AsyncPackages.RemoveSingleSwap(AsyncPackageState);
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("Request (%p) : remove AsyncPackage"), AsyncPackageState->Request);
			}

			for (FAsyncPackageState* ImportedAsyncPackage : AsyncPackageState->ImportedAsyncPackages)
			{
				ImportedAsyncPackage->ImportedByAsyncPackages.Remove(AsyncPackageState);
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("Imported AsyncPackage (%p) : remove ImportedBy AsyncPackage"), ImportedAsyncPackage);
			}

			for (FAsyncPackageState* ImportedByAsyncPackage : AsyncPackageState->ImportedByAsyncPackages)
			{
				ImportedByAsyncPackage->ImportedAsyncPackages.Remove(AsyncPackageState);
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("ImportedBy AsyncPackage (%p) : remove Imported AsyncPackage"), ImportedByAsyncPackage);
			}

			UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("delete FAsyncPackageState %p"), AsyncPackageState);
			delete AsyncPackageState;
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
		}
		break;
	}

	case RouteId_NewLinker:
	{
		uint64 LinkerPtr = EventData.GetValue<uint64>("Linker");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("NewLinker Linker=%llu"), LinkerPtr);
		FLinkerLoadState* LinkerState = new FLinkerLoadState();
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("new FLinkerLoadState %p"), LinkerState);
		ActiveLinkersMap.Add(LinkerPtr, LinkerState);
		break;
	}

	case RouteId_DestroyLinker:
	{
		uint64 LinkerPtr = EventData.GetValue<uint64>("Linker");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("DestroyLinker Linker=%llu"), LinkerPtr);
		FLinkerLoadState* LinkerState;
		if (ActiveLinkersMap.RemoveAndCopyValue(LinkerPtr, LinkerState))
		{
			FAsyncPackageState* AsyncPackageState = LinkerState->AsyncPackage;
			if (AsyncPackageState)
			{
				AsyncPackageState->Linker = nullptr;
				if (LinkerState->bHasFakeAsyncPackageState)
				{
					UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("delete (fake) FAsyncPackageState %p"), AsyncPackageState);
					ensure(AsyncPackageState->ImportedAsyncPackages.IsEmpty());
					ensure(AsyncPackageState->ImportedByAsyncPackages.IsEmpty());
					delete AsyncPackageState;
				}
			}
			UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("delete FLinkerLoadState %p"), LinkerState);
			delete LinkerState;
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown Linker!"));
		}
		break;
	}

	case RouteId_AsyncPackageImportDependency:
	{
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		uint64 ImportedAsyncPackagePtr = EventData.GetValue<uint64>("ImportedAsyncPackage");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("AsyncPackageImportDependency AsyncPackage=%llu ImportedAsyncPackage=%llu"), AsyncPackagePtr, ImportedAsyncPackagePtr);

		FAsyncPackageState* AsyncPackage = ActiveAsyncPackagesMap.FindRef(AsyncPackagePtr);
		FAsyncPackageState* ImportedAsyncPackage = nullptr;
		if (AsyncPackage)
		{
			ImportedAsyncPackage = ActiveAsyncPackagesMap.FindRef(ImportedAsyncPackagePtr);
		}
		else
		{
			FLinkerLoadState* LinkerState = ActiveLinkersMap.FindRef(AsyncPackagePtr);
			if (LinkerState)
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("%llu (%p) is a Linker; Linker->AsyncPackage=%p"), AsyncPackagePtr, LinkerState, LinkerState->AsyncPackage);
				AsyncPackage = LinkerState->AsyncPackage;

				FLinkerLoadState* ImportedLinkerState = ActiveLinkersMap.FindRef(ImportedAsyncPackagePtr);
				if (ImportedLinkerState)
				{
					UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("%llu (%p) is a Linker; ImportedLinker->AsyncPackage=%p"), ImportedAsyncPackagePtr, ImportedLinkerState, ImportedLinkerState->AsyncPackage);
					ImportedAsyncPackage = ImportedLinkerState->AsyncPackage;
				}
			}
		}
		if (AsyncPackage && ImportedAsyncPackage)
		{
			if (!AsyncPackage->ImportedAsyncPackages.Contains(ImportedAsyncPackage))
			{
				AsyncPackage->ImportedAsyncPackages.Add(ImportedAsyncPackage);
				ImportedAsyncPackage->ImportedByAsyncPackages.Add(AsyncPackage);
				FAnalysisSessionEditScope _(Session);
				AsyncPackage->PackageInfo->ImportedPackages.Add(ImportedAsyncPackage->PackageInfo);
				if (AsyncPackage->Request)
				{
					PackageRequestAssociation(Context, ImportedAsyncPackage, AsyncPackage->Request);
				}
			}
		}
		break;
	}

	case RouteId_AsyncPackageRequestAssociation:
	{
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		uint64 RequestId = EventData.GetValue<uint64>("RequestId");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("AsyncPackageRequestAssociation AsyncPackage=%llu RequestId=%llu"), AsyncPackagePtr, RequestId);

		FAsyncPackageState* AsyncPackageState = ActiveAsyncPackagesMap.FindRef(AsyncPackagePtr);
		if (!AsyncPackageState)
		{
			FLinkerLoadState* Linker = ActiveLinkersMap.FindRef(AsyncPackagePtr);
			if (Linker)
			{
				UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("%llu (%p) is a Linker; Linker->AsyncPackage=%p"), AsyncPackagePtr, Linker, Linker->AsyncPackage);
				AsyncPackageState = Linker->AsyncPackage;
			}
		}
		if (!AsyncPackageState)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
			break;
		}

		FRequestState* RequestState = nullptr;
		if (RequestId == 0)
		{
			if (!FakeRequestsStack.IsEmpty())
			{
				RequestState = FakeRequestsStack.Top();
			}
		}
		else
		{
			RequestState = ActiveRequestsMap.FindRef(RequestId);
		}
		if (!RequestState)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown Request!"));
			break;
		}

		FAnalysisSessionEditScope _(Session);
		PackageRequestAssociation(Context, AsyncPackageState, RequestState);
		break;
	}

	case RouteId_AsyncPackageLinkerAssociation:
	{
		uint64 AsyncPackagePtr = EventData.GetValue<uint64>("AsyncPackage");
		uint64 LinkerPtr = EventData.GetValue<uint64>("Linker");
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("AsyncPackageLinkerAssociation AsyncPackage=%llu Linker=%llu"), AsyncPackagePtr, LinkerPtr);

		FAsyncPackageState* AsyncPackageState = ActiveAsyncPackagesMap.FindRef(AsyncPackagePtr);
		if (!AsyncPackageState)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown AsyncPackage!"));
			break;
		}

		FLinkerLoadState* LinkerState = ActiveLinkersMap.FindRef(LinkerPtr);
		if (!LinkerState)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown Linker!"));
			break;
		}

		if (LinkerState->PackageInfo)
		{
			FAnalysisSessionEditScope _(Session);
			AsyncPackageState->PackageInfo->Name = LinkerState->PackageInfo->Name;
			AsyncPackageState->PackageInfo->Summary = LinkerState->PackageInfo->Summary;
		}
		LinkerState->AsyncPackage = AsyncPackageState;
		AsyncPackageState->Linker = LinkerState;
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("AsyncPackage %llu (%p) <-> Linker %llu (%p)"), AsyncPackagePtr, AsyncPackageState, LinkerPtr, LinkerState);
		break;
	}

	case RouteId_ClassInfo:
	{
		FAnalysisSessionEditScope _(Session);
		uint64 ClassPtr = EventData.GetValue<uint64>("Class");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		const FClassInfo& ClassInfo = LoadTimeProfilerProvider.AddClassInfo(*Name);
		ClassInfosMap.Add(ClassPtr, &ClassInfo);
		break;
	}

	case RouteId_BatchIssued: // IoDispatcher
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 BatchId = EventData.GetValue<uint64>("BatchId");

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("IoDispatcher.BatchIssued Time=%f BatchId=%llu"), Time, BatchId);

		uint64 BatchHandle;
		{
			FAnalysisSessionEditScope _(Session);
			BatchHandle = LoadTimeProfilerProvider.BeginIoDispatcherBatch(BatchId, Time);
			Session.UpdateDurationSeconds(Time);
		}
		ActiveBatchesMap.Add(BatchId, BatchHandle);
		break;
	}

	case RouteId_BatchResolved: // IoDispatcher
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		uint64 BatchId = EventData.GetValue<uint64>("BatchId");
		uint64 TotalSize = EventData.GetValue<uint64>("TotalSize");

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("IoDispatcher.BatchResolved Time=%f BatchId=%llu TotalSize=%llu"), Time, BatchId, TotalSize);

		uint64* FindBatchHandle = ActiveBatchesMap.Find(BatchId);
		if (FindBatchHandle)
		{
			FAnalysisSessionEditScope _(Session);
			LoadTimeProfilerProvider.EndIoDispatcherBatch(*FindBatchHandle, Time, TotalSize);
			Session.UpdateDurationSeconds(Time);
		}
		else
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING(TEXT("Unknown Batch!"));
		}
		break;
	}

#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	case RouteId_BeginObjectScope:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);

		uint64 ObjectPtr = EventData.GetValue<uint64>("Object");
		ELoadTimeProfilerObjectEventType EventType = static_cast<ELoadTimeProfilerObjectEventType>(EventData.GetValue<uint8>("EventType"));

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("BeginObjectScope Time=%f Object=%llu EventType=%u"), Time, ObjectPtr, uint32(EventType));

		FPackageExportInfo* Export = ExportsMap.FindRef(ObjectPtr);
		if (!Export)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX(TEXT("Unknown export Object!"));
		}

#if 0 // debug
		EventType = static_cast<ELoadTimeProfilerObjectEventType>(99);
#endif

		FAnalysisSessionEditScope _(Session);
		ThreadState.EnterScope(Time, Export, EventType);
		Session.UpdateDurationSeconds(Time);
		break;
	}
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY

#if UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	case RouteId_EndObjectScope:
	{
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		FThreadState& ThreadState = GetThreadState(ThreadId);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);

		UE_LOAD_TIME_TRACE_ANALYSIS_LOG1(TEXT("EndObjectScope Time=%f"), Time);

		if (ThreadState.CpuScopeStackDepth <= 0)
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndObjectScope without Begin event!"));
			break;
		}

#if 0 // debug
		if (ThreadState.GetCurrentExportScopeEventType() != static_cast<ELoadTimeProfilerObjectEventType>(99)))
		{
			UE_LOAD_TIME_TRACE_ANALYSIS_ERROR(TEXT("EndObjectScope does not match the Begin event type!"));
			break;
		}
#endif

		FAnalysisSessionEditScope _(Session);
		ThreadState.LeaveScope(Time);
		Session.UpdateDurationSeconds(Time);
		break;
	}
#endif // UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
	}

	return true;
}

void FAsyncLoadingTraceAnalyzer::FThreadState::EnterScope(double Time, const FPackageInfo* PackageInfo)
{
	FScopeStackEntry& StackEntry = CpuScopeStack[CpuScopeStackDepth++];
	StackEntry.Event.Export = nullptr;
	StackEntry.Event.Package = PackageInfo;
	StackEntry.Event.EventType = LoadTimeProfilerObjectEventType_None;
	CurrentEvent = StackEntry.Event;
	CpuTimeline->AppendBeginEvent(Time, StackEntry.Event);
}

void FAsyncLoadingTraceAnalyzer::FThreadState::EnterScope(double Time, const FPackageExportInfo* ExportInfo, ELoadTimeProfilerObjectEventType EventType)
{
	FScopeStackEntry& StackEntry = CpuScopeStack[CpuScopeStackDepth++];
	StackEntry.Event.Export = ExportInfo;
	StackEntry.Event.Package = ExportInfo ? ExportInfo->Package : nullptr;
	StackEntry.Event.EventType = EventType;
	CurrentEvent = StackEntry.Event;
	CpuTimeline->AppendBeginEvent(Time, StackEntry.Event);
}

void FAsyncLoadingTraceAnalyzer::FThreadState::LeaveScope(double Time)
{
	check(CpuScopeStackDepth > 0);
	FScopeStackEntry& StackEntry = CpuScopeStack[--CpuScopeStackDepth];
	CpuTimeline->AppendEndEvent(Time);
	if (CpuScopeStackDepth > 0)
	{
		CurrentEvent = CpuScopeStack[CpuScopeStackDepth - 1].Event;
	}
	else
	{
		CurrentEvent = FLoadTimeProfilerCpuEvent();
	}
}

ELoadTimeProfilerObjectEventType FAsyncLoadingTraceAnalyzer::FThreadState::GetCurrentScopeEventType()
{
	if (CpuScopeStackDepth > 0)
	{
		FScopeStackEntry& StackEntry = CpuScopeStack[CpuScopeStackDepth - 1];
		return StackEntry.Event.EventType;
	}
	else
	{
		return LoadTimeProfilerObjectEventType_None;
	}
}

FPackageExportInfo* FAsyncLoadingTraceAnalyzer::FThreadState::GetCurrentExportScope()
{
	if (CpuScopeStackDepth > 0)
	{
		FScopeStackEntry& StackEntry = CpuScopeStack[CpuScopeStackDepth - 1];
		return const_cast<FPackageExportInfo*>(StackEntry.Event.Export);
	}
	else
	{
		return nullptr;
	}
}

void FAsyncLoadingTraceAnalyzer::PackageRequestAssociation(const FOnEventContext& Context, FAsyncPackageState* AsyncPackageState, FRequestState* RequestState)
{
	if (!AsyncPackageState->Request)
	{
		UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("PackageRequestAssociation(AsyncPackageState=%p, RequestState=%p)"), AsyncPackageState, RequestState);
		RequestState->AsyncPackages.Add(AsyncPackageState);
		AsyncPackageState->Request = RequestState;
		FLoadRequest* LoadRequest = RequestState->Group->LoadRequest;
		if (!LoadRequest)
		{
			LoadRequest = &LoadTimeProfilerProvider.CreateRequest();
			LoadRequest->StartTime = Context.EventTime.AsSeconds(RequestState->WallTimeStartCycle);
			LoadRequest->EndTime = std::numeric_limits<double>::infinity();
			LoadRequest->Name = Session.StoreString(*RequestState->Group->Name);
			LoadRequest->ThreadId = RequestState->ThreadId;
			RequestState->Group->LoadRequest = LoadRequest;
			UE_LOAD_TIME_TRACE_ANALYSIS_LOG2(TEXT("Provider.CreateRequest \"%s\""), LoadRequest->Name);
		}
		if (AsyncPackageState->PackageInfo)
		{
			LoadRequest->Packages.Add(AsyncPackageState->PackageInfo);
			AsyncPackageState->PackageInfo->RequestId = LoadRequest->Id;
		}
		for (FAsyncPackageState* ImportedAsyncPackage : AsyncPackageState->ImportedAsyncPackages)
		{
			PackageRequestAssociation(Context, ImportedAsyncPackage, RequestState);
		}
	}
}

} // namespace TraceServices

#undef UE_LOAD_TIME_TRACE_ANALYSIS_BACKWARD_COMPATIBILITY
#undef UE_LOAD_TIME_TRACE_ANALYSIS_ERROR
#undef UE_LOAD_TIME_TRACE_ANALYSIS_WARNING
#undef UE_LOAD_TIME_TRACE_ANALYSIS_WARNING_EX
#undef UE_LOAD_TIME_TRACE_ANALYSIS_LOG
#undef UE_LOAD_TIME_TRACE_ANALYSIS_LOG1
#undef UE_LOAD_TIME_TRACE_ANALYSIS_LOG2
