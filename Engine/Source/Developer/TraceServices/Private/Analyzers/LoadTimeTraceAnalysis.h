// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Trace/Analyzer.h"
#include "AnalysisServicePrivate.h"
#include "Model/LoadTimeProfilerPrivate.h"

namespace TraceServices
{

struct FClassInfo;

inline bool operator!=(const FLoadTimeProfilerCpuEvent& Lhs, const FLoadTimeProfilerCpuEvent& Rhs)
{
	return Lhs.Package != Rhs.Package ||
		Lhs.Export != Rhs.Export ||
		Lhs.EventType != Rhs.EventType;
}

class FAsyncLoadingTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FAsyncLoadingTraceAnalyzer(IAnalysisSession& Session, FLoadTimeProfilerProvider& LoadTimeProfilerProvider);
	virtual ~FAsyncLoadingTraceAnalyzer();

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	struct FRequestState;
	struct FAsyncPackageState;
	struct FLinkerLoadState;

	struct FRequestGroupState
	{
		FString Name;
		TArray<FRequestState*> Requests;
		FLoadRequest* LoadRequest = nullptr;
		uint64 LatestEndCycle = 0;
		uint64 ActiveRequestsCount = 0;
	};

	struct FRequestState
	{
		uint64 WallTimeStartCycle;
		uint64 WallTimeEndCycle;
		uint32 ThreadId;
		TSharedPtr<FRequestGroupState> Group;
		TArray<FAsyncPackageState*> AsyncPackages;
	};

	struct FAsyncPackageState
	{
		FPackageInfo* PackageInfo = nullptr;
		FRequestState* Request = nullptr;
		FLinkerLoadState* Linker = nullptr;
		TSet<FAsyncPackageState*> ImportedAsyncPackages;
		TSet<FAsyncPackageState*> ImportedByAsyncPackages;
	};

	struct FLinkerLoadState
	{
		FPackageInfo* PackageInfo = nullptr;
		FAsyncPackageState* AsyncPackage = nullptr;
		bool bHasFakeAsyncPackageState = false;
	};

	struct FScopeStackEntry
	{
		FLoadTimeProfilerCpuEvent Event;
		bool EnteredEvent;
	};

	struct FThreadState
	{
		FScopeStackEntry CpuScopeStack[256];
		uint64 CpuScopeStackDepth = 0;
		int64 PostLoadScopeDepth = 0;
		FLoadTimeProfilerCpuEvent CurrentEvent;
		TArray<TSharedPtr<FRequestGroupState>> RequestGroupStack;

		FLoadTimeProfilerProvider::CpuTimelineInternal* CpuTimeline;

		void EnterScope(double Time, const FPackageInfo* PackageInfo);
		void EnterScope(double Time, const FPackageExportInfo* ExportInfo, ELoadTimeProfilerObjectEventType EventType);
		void LeaveScope(double Time);
		ELoadTimeProfilerObjectEventType GetCurrentScopeEventType();
		FPackageExportInfo* GetCurrentExportScope();
	};

	void PackageRequestAssociation(const FOnEventContext& Context, FAsyncPackageState* AsyncPackageState, FRequestState* RequestState);
	FThreadState& GetThreadState(uint32 ThreadId);
	const FClassInfo* GetClassInfo(uint64 ClassPtr) const;

	enum : uint16
	{
		RouteId_StartAsyncLoading,
		RouteId_SuspendAsyncLoading,
		RouteId_ResumeAsyncLoading,
		RouteId_NewAsyncPackage,
		RouteId_DestroyAsyncPackage,
		RouteId_NewLinker,
		RouteId_DestroyLinker,
		RouteId_BeginRequest,
		RouteId_EndRequest,
		RouteId_BeginRequestGroup,
		RouteId_EndRequestGroup,
		RouteId_PackageSummary,
		RouteId_AsyncPackageRequestAssociation,
		RouteId_AsyncPackageLinkerAssociation,
		RouteId_AsyncPackageImportDependency,
		RouteId_BeginProcessSummary,
		RouteId_EndProcessSummary,
		RouteId_BeginCreateExport,
		RouteId_EndCreateExport,
		RouteId_BeginSerializeExport,
		RouteId_EndSerializeExport,
		RouteId_BeginPostLoad,
		RouteId_EndPostLoad,
		RouteId_BeginPostLoadObject,
		RouteId_EndPostLoadObject,
		RouteId_ClassInfo,
		RouteId_BatchIssued,
		RouteId_BatchResolved,

		// Backwards compatibility
		RouteId_BeginObjectScope,
		RouteId_EndObjectScope,
		RouteId_BeginPostLoadExport,
		RouteId_EndPostLoadExport,
	};

	enum
	{
		FormatBufferSize = 65536
	};
	TCHAR FormatBuffer[FormatBufferSize];
	TCHAR TempBuffer[FormatBufferSize];

	IAnalysisSession& Session;
	FLoadTimeProfilerProvider& LoadTimeProfilerProvider;

	template<typename ValueType>
	struct FPointerMapKeyFuncs
	{
		typedef uint64 KeyType;
		typedef uint64 KeyInitType;
		typedef const TPairInitializer<uint64, ValueType>& ElementInitType;

		enum { bAllowDuplicateKeys = false };

		static FORCEINLINE bool Matches(uint64 A, uint64 B)
		{
			return A == B;
		}

		static FORCEINLINE uint32 GetKeyHash(uint64 Key)
		{
			return PointerHash((const void*)Key);
		}

		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element.Key;
		}
	};

	template<typename ValueType>
	using TPointerMap = TMap<uint64, ValueType, FDefaultSetAllocator, FPointerMapKeyFuncs<ValueType>>;

	TPointerMap<FAsyncPackageState*> ActiveAsyncPackagesMap;
	TPointerMap<FLinkerLoadState*> ActiveLinkersMap;
	TPointerMap<FPackageExportInfo*> ExportsMap;
	TMap<uint64, FRequestState*> ActiveRequestsMap;
	TArray<FRequestState*> FakeRequestsStack;
	TPointerMap<uint64> ActiveBatchesMap;
	TMap<uint32, FThreadState*> ThreadStatesMap;
	TPointerMap<const FClassInfo*> ClassInfosMap;
	FPackageInfo* UnknownPackageInfo = nullptr;
	uint64 ErrorCount = 0;
	uint64 WarningCount = 0;
};

} // namespace TraceServices
