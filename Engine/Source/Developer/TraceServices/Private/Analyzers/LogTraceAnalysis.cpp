// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogTraceAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogTrace.h"
#include "Model/LogPrivate.h"

namespace TraceServices
{

FLogTraceAnalyzer::FLogTraceAnalyzer(IAnalysisSession& InSession, FLogProvider& InLogProvider)
	: Session(InSession)
	, LogProvider(InLogProvider)
{

}

void FLogTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_LogCategory, "Logging", "LogCategory");
	Builder.RouteEvent(RouteId_LogMessageSpec, "Logging", "LogMessageSpec");
	Builder.RouteEvent(RouteId_LogMessage, "Logging", "LogMessage");
}

void FLogTraceAnalyzer::OnAnalysisEnd()
{
	FAnalysisSessionEditScope _(Session);

	const uint64 InsertCount = LogProvider.GetInsertCount();
	if (InsertCount > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[Logs] Number of inserts (out of order messages): %llu", InsertCount);
	}

	const uint64 UnknownSpecTotalCount = LogProvider.GetUnknownSpecTotalCount();
	if (UnknownSpecTotalCount > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[Logs] Number of log messages received with unknown spec (LogPoint): %llu (%llu still unresolved)",
			UnknownSpecTotalCount, LogProvider.GetUnresolvedSpecCount());
	}

	UE_LOGF(LogTraceServices, Log, "[Logs] Analysis completed (%llu messages, %llu specs, %llu categories).",
		LogProvider.GetMessageCount(),
		LogProvider.GetMessageSpecCount(),
		LogProvider.GetCategoryCount());
}

bool FLogTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FLogTraceAnalyzer"));

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_LogCategory:
	{
		uint64 CategoryPointer = EventData.GetValue<uint64>("CategoryPointer");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		const TCHAR* CategoryName = Session.StoreString(*Name);
		ELogVerbosity::Type CategoryDefaultVerbosity = static_cast<ELogVerbosity::Type>(EventData.GetValue<uint8>("DefaultVerbosity"));

		FAnalysisSessionEditScope _(Session);
		FLogCategoryInfo& Category = LogProvider.GetCategory(CategoryPointer);
		Category.Name = CategoryName;
		Category.DefaultVerbosity = CategoryDefaultVerbosity;
		//LogProvider.UpdateCategory(CategoryPointer);
		break;
	}
	case RouteId_LogMessageSpec:
	{
		uint64 LogPoint = EventData.GetValue<uint64>("LogPoint");

		uint64 CategoryPointer = EventData.GetValue<uint64>("CategoryPointer");

		int32 Line = EventData.GetValue<int32>("Line");
		ELogVerbosity::Type Verbosity = static_cast<ELogVerbosity::Type>(EventData.GetValue<uint8>("Verbosity"));

		const TCHAR* File = nullptr;
		const TCHAR* FormatString = nullptr;
		FString Str;
		if (EventData.GetString("FileName", Str))
		{
			File = Session.StoreString(*Str);
			if (EventData.GetString("FormatString", Str))
			{
				FormatString = Session.StoreString(*Str);
			}
		}
		else // backward compatibility
		{
			const ANSICHAR* AnsiFile = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
			if (AnsiFile)
			{
				File = Session.StoreString(ANSI_TO_TCHAR(AnsiFile));
				FormatString = Session.StoreString(reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + TCString<ANSICHAR>::Strlen(AnsiFile) + 1));
			}
		}

		FAnalysisSessionEditScope _(Session);
		FLogMessageSpec& Spec = LogProvider.GetMessageSpec(LogPoint);
		Spec.Category = &LogProvider.GetCategory(CategoryPointer);
		Spec.Line = Line;
		Spec.Verbosity = Verbosity;
		Spec.File = File;
		Spec.FormatString = FormatString;
		LogProvider.UpdateMessageSpec(LogPoint);
		break;
	}
	case RouteId_LogMessage:
	{
		uint64 LogPoint = EventData.GetValue<uint64>("LogPoint");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		TArrayView<const uint8> FormatArgsView = FTraceAnalyzerUtils::LegacyAttachmentArray("FormatArgs", Context);

		FAnalysisSessionEditScope _(Session);
		LogProvider.AppendMessage(LogPoint, Time, FormatArgsView);
		break;
	}
	}

	return true;
}

} // namespace TraceServices
