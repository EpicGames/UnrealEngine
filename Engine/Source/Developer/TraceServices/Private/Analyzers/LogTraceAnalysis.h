// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices
{

class IAnalysisSession;
class FLogProvider;

class FLogTraceAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FLogTraceAnalyzer(IAnalysisSession& Session, FLogProvider& LogProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_LogCategory,
		RouteId_LogMessageSpec,
		RouteId_LogMessage,
	};

	IAnalysisSession& Session;
	FLogProvider& LogProvider;
};

} // namespace TraceServices
