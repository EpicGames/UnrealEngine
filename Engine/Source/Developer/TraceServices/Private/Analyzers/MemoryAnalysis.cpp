// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"

namespace TraceServices
{

FMemoryAnalyzer::FMemoryAnalyzer(IAnalysisSession& InSession, FMemoryProvider* InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
	check(Provider != nullptr);
}

FMemoryAnalyzer::~FMemoryAnalyzer()
{
}

void FMemoryAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_TagsSpec, "LLM", "TagsSpec");
	Builder.RouteEvent(RouteId_TrackerSpec, "LLM", "TrackerSpec");
	Builder.RouteEvent(RouteId_TagSetSpec, "LLM", "TagSetSpec"); // added in UE 5.6
	Builder.RouteEvent(RouteId_TagValue, "LLM", "TagValue");
}

void FMemoryAnalyzer::OnAnalysisEnd()
{
	if (Provider)
	{
		FProviderEditScopeLock _(*Provider);
		Provider->OnAnalysisCompleted();
	}
}

bool FMemoryAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	if (!Provider)
	{
		return false;
	}

	LLM_SCOPE_BYNAME(TEXT("Insights/FMemoryAnalyzer"));

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_TagsSpec:
	{
		const int64 TagId = EventData.GetValue<int64>("TagId");
		const int64 ParentId = EventData.GetValue<int64>("ParentId");
		const uint8 TagSetId = EventData.GetValue<uint8>("TagSetId"); // added in UE 5.6
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);

		FProviderEditScopeLock _(*Provider);
		Provider->AddTagSpec(TagId, Name, ParentId, (FMemoryTagSetId)TagSetId);
	}
	break;
	case RouteId_TrackerSpec:
	{
		uint8 TrackerId = EventData.GetValue<uint8>("TrackerId");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("Name", Context);

		FProviderEditScopeLock _(*Provider);
		Provider->AddTrackerSpec((FMemoryTrackerId)TrackerId, Name);
	}
	break;
	case RouteId_TagSetSpec: // added in UE 5.6
	{
		const uint8 TagSetId = EventData.GetValue<uint8>("TagSetId");
		FString Name;
		EventData.GetString("Name", Name);

		FProviderEditScopeLock _(*Provider);
		Provider->AddTagSetSpec((FMemoryTagSetId)TagSetId, Name);
	}
	break;
	case RouteId_TagValue:
	{
		const uint8 TrackerId = EventData.GetValue<uint8>("TrackerId");
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double Time = Context.EventTime.AsSeconds(Cycle);
		const TArrayReader<int64>& Tags = EventData.GetArray<int64>("Tags"); // was traced as (void*)[]
		const TArrayReader<int64>& Samples = EventData.GetArray<int64>("Values");

		const uint32 TagsCount = Tags.Num();
		const int64* TagsData = Tags.GetData();

		TArray<int64> TagsCopy;
		if (TagsData == nullptr)
		{
			// For backward compatibility with 32bit platforms.
			const TArrayReader<uint32>& Tags32 = EventData.GetArray<uint32>("Tags");
			check(Tags32.Num() == TagsCount);
			const uint32* TagsData32 = Tags32.GetData();
			if (!TagsData32)
			{
				break;
			}
			TagsCopy.Reserve(TagsCount);
			for (uint32 TagIndex = 0; TagIndex < TagsCount; ++TagIndex)
			{
				TagsCopy.Push(TagsData32[TagIndex]);
			}
			TagsData = TagsCopy.GetData();
		}

		check(Samples.Num() == TagsCount);
		const int64* SamplesData = Samples.GetData();
		check(SamplesData != nullptr);
		TArray<FMemoryTagSample> Values;
		Values.Reserve(TagsCount);
		for (uint32 TagIndex = 0; TagIndex < TagsCount; ++TagIndex)
		{
			Values.Push(FMemoryTagSample{ SamplesData[TagIndex] });
		}

		Sample++;

		{
			FProviderEditScopeLock _(*Provider);
			Provider->AddTagSnapshot(TrackerId, Time, TagsData, Values.GetData(), TagsCount);
		}

		{
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Time);
		}
	}
	break;
	}
	return true;
}

} // namespace TraceServices
