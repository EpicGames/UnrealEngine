// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "TraceServices/Containers/Allocators.h"
#include "Model/MemoryPrivate.h"
#include "Containers/UnrealString.h"
#include "Common/PagedArray.h"

namespace TraceServices
{

class IAnalysisSession;

class FMemoryAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FMemoryAnalyzer(IAnalysisSession& Session, FMemoryProvider* InProvider);
	~FMemoryAnalyzer();
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_TagsSpec,
		RouteId_TagSetSpec,
		RouteId_TrackerSpec,
		RouteId_TagValue,
	};

	IAnalysisSession& Session;
	FMemoryProvider* Provider;
	uint64 Sample = 0;
};

} // namespace TraceServices
