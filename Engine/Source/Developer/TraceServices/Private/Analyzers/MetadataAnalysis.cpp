// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetadataAnalysis.h"

#include "Common/ProviderLock.h"
#include "Common/Utils.h"
#include "Containers/StringConv.h"
#include "CoreMinimal.h"
#include "Model/DefinitionProvider.h"
#include "Model/MetadataProvider.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/Strings.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename ArrayType>
static void WriteData(ArrayType& OutData, const UE::Trace::IAnalyzer::FEventData& EventData, uint16 MetadataTypeId, const FMetadataProvider* MetadataProvider, IAnalysisSession& Session);

////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataAnalysis::FMetadataAnalysis(IAnalysisSession& InSession, FMetadataProvider* InProvider)
	: Session(InSession)
	, MetadataProvider(InProvider)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataAnalysis::~FMetadataAnalysis()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataAnalysis::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	// Some operations with the metadata stack
	Builder.RouteEvent(RouteId_ClearScope, "MetadataStack", "ClearScope", true);
	Builder.RouteEvent(RouteId_SaveStack, "MetadataStack", "SaveStack");
	Builder.RouteEvent(RouteId_RestoreStack, "MetadataStack", "RestoreStack", true);

	// Subscribe to all scopes on the "Metadata" logger. These are the actual meta data scopes.
	Builder.RouteLoggerEvents(RouteId_Metascope, "Metadata", true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataAnalysis::OnAnalysisEnd()
{
	FProviderEditScopeLock _(*MetadataProvider);
	MetadataProvider->OnAnalysisCompleted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMetadataAnalysis::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	FProviderEditScopeLock _(*MetadataProvider);
	switch (RouteId)
	{
	case RouteId_ClearScope:
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			if (Style == EStyle::EnterScope)
			{
				MetadataProvider->BeginClearStackScope(ThreadId);
			}
			else if (ensure(Style == EStyle::LeaveScope))
			{
				MetadataProvider->EndClearStackScope(ThreadId);
			}
		}
		break;

	case RouteId_SaveStack:
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			const uint32 RuntimeId = Context.EventData.GetValue<uint32>("Id");
			MetadataProvider->SaveStack(ThreadId, RuntimeId);
		}
		break;

	case RouteId_RestoreStack:
		{
			const uint32 ThreadId = Context.ThreadInfo.GetId();
			if (Style == EStyle::EnterScope)
			{
				const uint32 RuntimeId = Context.EventData.GetValue<uint32>("Id");
				MetadataProvider->BeginRestoreSavedStackScope(ThreadId, RuntimeId);
			}
			else if (ensure(Style == EStyle::LeaveScope))
			{
				MetadataProvider->EndRestoreSavedStackScope(ThreadId);
			}
		}
		break;

	case RouteId_Metascope:
		{
			const FEventData& EventData = Context.EventData;
			const FEventTypeInfo& EventInfo = Context.EventData.GetTypeInfo();

			// Check if we have already encountered and registered this meta data type. If not, build a schema
			// for this event type and register it with the provider.
			const uint16 MetadataTypeId = GetOrRegisterType(EventInfo);

			const uint32 ThreadId = Context.ThreadInfo.GetId();

			// Push the actual data of the scope to the provider or pop if we are leaving the scope.
			if (Style == EStyle::EnterScope)
			{
				TArray<uint8, TInlineAllocator<32>> Data;
				WriteData(Data, EventData, MetadataTypeId, MetadataProvider, Session);

				MetadataProvider->PushScopedMetadata(ThreadId, MetadataTypeId, Data.GetData(), Data.Num());
			}
			else if (ensure(Style == EStyle::LeaveScope))
			{
				MetadataProvider->PopScopedMetadata(ThreadId, MetadataTypeId);
			}
		}
		break;

	default:
		checkf(false, TEXT("Invalid route."));
		break;
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint16 FMetadataAnalysis::GetOrRegisterType(const FEventTypeInfo& EventInfo)
{
	if (const uint16* CachedTypeId = EncounteredMetadataTypes.Find(EventInfo.GetId()))
	{
		return *CachedTypeId;
	}

	uint16 MetadataTypeId = 0;
	FMetadataSchema Schema(static_cast<uint8>(EventInfo.GetFieldCount()));
	FMetadataSchema::FBuilder Builder = Schema.Builder();
	for (uint32 FieldIdx = 0; FieldIdx < EventInfo.GetFieldCount(); ++FieldIdx)
	{
		const FEventFieldInfo* FieldInfo = EventInfo.GetFieldInfo(FieldIdx);
		const TCHAR* Name = Session.StoreString(ANSI_TO_TCHAR(FieldInfo->GetName()));
		const uint8 Size = FieldInfo->GetSize();

		switch (FieldInfo->GetType())
		{
		case FEventFieldInfo::EType::Integer:
			{
				if (FieldInfo->IsSigned())
				{
					constexpr auto FieldType = FMetadataSchema::EFieldType::SignedInteger;
					if (Size == 1) Builder.AddField<int8>(Name, FieldType);
					else if (Size == 2) Builder.AddField<int16>(Name, FieldType);
					else if (Size == 4) Builder.AddField<int32>(Name, FieldType);
					else if (Size == 8) Builder.AddField<int64>(Name, FieldType);
					else { checkNoEntry(); }
				}
				else
				{
					constexpr auto FieldType = FMetadataSchema::EFieldType::Integer;
					if (Size == 1) Builder.AddField<uint8>(Name, FieldType);
					else if (Size == 2) Builder.AddField<uint16>(Name, FieldType);
					else if (Size == 4) Builder.AddField<uint32>(Name, FieldType);
					else if (Size == 8) Builder.AddField<uint64>(Name, FieldType);
					else { checkNoEntry(); }
				}
			}
			break;

		case FEventFieldInfo::EType::Float:
			{
				constexpr auto FieldType = FMetadataSchema::EFieldType::FloatingPoint;
				if (Size == 4) Builder.AddField<float>(Name, FieldType);
				else if (Size == 8) Builder.AddField<double>(Name, FieldType);
			}
			break;

		case FEventFieldInfo::EType::WideString:
		case FEventFieldInfo::EType::AnsiString:
			{
				// Note that we don't have an ANSI string store yet, so we store any ANSI
				// string as WIDE for metadata.
				constexpr auto FieldType = FMetadataSchema::EFieldType::WideStringPtr;
				Builder.AddField<TCHAR*>(Name, FieldType);
			}
			break;

		case FEventFieldInfo::EType::Reference8:
		case FEventFieldInfo::EType::Reference16:
		case FEventFieldInfo::EType::Reference32:
		case FEventFieldInfo::EType::Reference64:
			{
				constexpr auto FieldType = FMetadataSchema::EFieldType::Reference;
				if (Size == 1) Builder.AddField<UE::Trace::FEventRef8>(Name, FieldType);
				else if (Size == 2) Builder.AddField<UE::Trace::FEventRef16>(Name, FieldType);
				else if (Size == 4) Builder.AddField<UE::Trace::FEventRef32>(Name, FieldType);
				else if (Size == 8) Builder.AddField<UE::Trace::FEventRef64>(Name, FieldType);
				else { checkNoEntry(); }
			}
			break;

		default:
			break;
		}
	}
	Builder.Finish();
	MetadataTypeId = MetadataProvider->RegisterMetadataType(ANSI_TO_TCHAR(EventInfo.GetName()), Schema);
	EncounteredMetadataTypes.Emplace(EventInfo.GetId(), MetadataTypeId);
	return MetadataTypeId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename ArrayType>
static void WriteData(ArrayType& OutData, const UE::Trace::IAnalyzer::FEventData& EventData, uint16 MetadataTypeId, const FMetadataProvider* MetadataProvider, IAnalysisSession& Session)
{
	using namespace UE::Trace;
	const IAnalyzer::FEventTypeInfo& EventInfo = EventData.GetTypeInfo();
	const FMetadataSchema* Schema = MetadataProvider->GetRegisteredMetadataSchema(MetadataTypeId);
	check(Schema);
	FMetadataSchema::FWriter Writer = Schema->Writer();

	for (uint32 FieldIdx = 0; FieldIdx < EventInfo.GetFieldCount(); ++FieldIdx)
	{
		const IAnalyzer::FEventFieldInfo* FieldInfo = EventInfo.GetFieldInfo(FieldIdx);
		switch (FieldInfo->GetType())
		{
		case IAnalyzer::FEventFieldInfo::EType::Reference8:
		case IAnalyzer::FEventFieldInfo::EType::Reference16:
		case IAnalyzer::FEventFieldInfo::EType::Reference32:
		case IAnalyzer::FEventFieldInfo::EType::Reference64:
			{
				switch (FieldInfo->GetSize())
				{
				case 1:
					{
						auto RefValue = EventData.GetReferenceValue<uint8>(FieldIdx);
						Writer.WriteField(static_cast<uint8>(FieldIdx), &RefValue, sizeof(UE::Trace::FEventRef8), OutData);
					}
					break;
				case 2:
					{
						auto RefValue = EventData.GetReferenceValue<uint16>(FieldIdx);
						Writer.WriteField(static_cast<uint8>(FieldIdx), &RefValue, sizeof(UE::Trace::FEventRef16), OutData);
					}
					break;
				case 4:
					{
						auto RefValue = EventData.GetReferenceValue<uint32>(FieldIdx);
						Writer.WriteField(static_cast<uint8>(FieldIdx), &RefValue, sizeof(UE::Trace::FEventRef32), OutData);
					}
					break;
				case 8:
					{
						auto RefValue = EventData.GetReferenceValue<uint64>(FieldIdx);
						Writer.WriteField(static_cast<uint8>(FieldIdx), &RefValue, sizeof(UE::Trace::FEventRef64), OutData);
					}
					break;
				default:
					checkNoEntry();
				}
			}
			break;

		case IAnalyzer::FEventFieldInfo::EType::WideString:
			{
				FWideStringView View;
				EventData.GetString(FieldInfo->GetName(), View);
				const TCHAR* Str = Session.StoreString(View);
				Writer.WriteField(static_cast<uint8>(FieldIdx), Str, sizeof(TCHAR*), OutData);
			}
			break;

		case IAnalyzer::FEventFieldInfo::EType::AnsiString:
			{
				FString String;
				EventData.GetString(FieldInfo->GetName(), String);
				const TCHAR* Str = Session.StoreString(String);
				Writer.WriteField(static_cast<uint8>(FieldIdx), Str, sizeof(TCHAR*), OutData);
			}
			break;

		default:
			{
				const IAnalyzer::FEventFieldHandle FieldHandle = EventInfo.GetFieldHandleUnchecked(FieldIdx);
				const void* Value = EventData.GetValueRaw(FieldHandle);
				const uint8 Size = FieldInfo->GetSize();
				Writer.WriteField(static_cast<uint8>(FieldIdx), Value, Size, OutData);
			}
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
