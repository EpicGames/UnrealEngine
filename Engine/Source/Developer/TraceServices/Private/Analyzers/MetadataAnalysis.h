// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/Map.h"

namespace TraceServices
{

class IAnalysisSession;
class FMetadataProvider;

class FMetadataAnalysis : public UE::Trace::IAnalyzer
{
public:
	FMetadataAnalysis(IAnalysisSession& Session, FMetadataProvider* InProvider);
	virtual ~FMetadataAnalysis() override;

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	uint16 GetOrRegisterType(const FEventTypeInfo& EventInfo);

	enum EMetadataRouteIds
	{
		RouteId_ClearScope,
		RouteId_SaveStack,
		RouteId_RestoreStack,
		RouteId_Metascope,
	};

	IAnalysisSession& Session;
	FMetadataProvider* MetadataProvider;
	TMap<uint32,uint16> EncounteredMetadataTypes;
};

} // namespace TraceServices
