// Copyright Epic Games, Inc. All Rights Reserved.

#include "MiscTraceAnalysis.h"

#include "Common/ProviderLock.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/MessageLog.h"
#include "Model/BookmarksPrivate.h"
#include "Model/Channel.h"
#include "Model/DefinitionProvider.h"
#include "Model/FramesPrivate.h"
#include "Model/LogPrivate.h"
#include "Model/RegionsPrivate.h"
#include "Model/ScreenshotProviderPrivate.h"
#include "Model/ThreadsPrivate.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Strings.h"

namespace TraceServices
{

FMiscTraceAnalyzer::FMiscTraceAnalyzer(IAnalysisSession& InSession,
									   FThreadProvider& InThreadProvider,
									   FLogProvider& InLogProvider,
									   FFrameProvider& InFrameProvider,
									   FChannelProvider& InChannelProvider,
									   FScreenshotProvider& InScreenshotProvider,
									   FRegionProvider& InRegionProvider)
	: Session(InSession)
	, ThreadProvider(InThreadProvider)
	, LogProvider(InLogProvider)
	, FrameProvider(InFrameProvider)
	, ChannelProvider(InChannelProvider)
	, ScreenshotProvider(InScreenshotProvider)
	, RegionProvider(InRegionProvider)
{
	// Todo: update this to use provider locking instead of session locking
	// FProviderEditScopeLock LogProviderLock (LogProvider);
	FAnalysisSessionEditScope _(Session);
	ScreenshotLogCategoryId = LogProvider.RegisterCategory();
	FLogCategoryInfo& ScreenshotLogCategory = LogProvider.GetCategory(ScreenshotLogCategoryId);
	ScreenshotLogCategory.Name = TEXT("Screenshot");
}

void FMiscTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	// Threads
	Builder.RouteEvent(RouteId_RegisterGameThread, "Misc", "RegisterGameThread");
	Builder.RouteEvent(RouteId_CreateThread, "Misc", "CreateThread");
	Builder.RouteEvent(RouteId_SetThreadGroup, "Misc", "SetThreadGroup");
	Builder.RouteEvent(RouteId_BeginThreadGroupScope, "Misc", "BeginThreadGroupScope");
	Builder.RouteEvent(RouteId_EndThreadGroupScope, "Misc", "EndThreadGroupScope");

	// Frames
	Builder.RouteEvent(RouteId_BeginFrame, "Misc", "BeginFrame");
	Builder.RouteEvent(RouteId_EndFrame, "Misc", "EndFrame");
	Builder.RouteEvent(RouteId_BeginGameFrame, "Misc", "BeginGameFrame");
	Builder.RouteEvent(RouteId_EndGameFrame, "Misc", "EndGameFrame");
	Builder.RouteEvent(RouteId_BeginRenderFrame, "Misc", "BeginRenderFrame");
	Builder.RouteEvent(RouteId_EndRenderFrame, "Misc", "EndRenderFrame");

	// Channels
	Builder.RouteEvent(RouteId_ChannelAnnounce, "Trace", "ChannelAnnounce");
	Builder.RouteEvent(RouteId_ChannelToggle, "Trace", "ChannelToggle");

	// Screenshots
	Builder.RouteEvent(RouteId_ScreenshotHeader, "Misc", "ScreenshotHeader");
	Builder.RouteEvent(RouteId_ScreenshotChunk, "Misc", "ScreenshotChunk");

	// Timing Regions
	Builder.RouteEvent(RouteId_RegionBegin, "Misc", "RegionBegin");
	Builder.RouteEvent(RouteId_RegionBeginWithId, "Misc", "RegionBeginWithId");
	Builder.RouteEvent(RouteId_RegionEnd, "Misc", "RegionEnd");
	Builder.RouteEvent(RouteId_RegionEndWithId, "Misc", "RegionEndWithId");
}

void FMiscTraceAnalyzer::OnAnalysisEnd()
{
	FProviderEditScopeLock RegionProviderScopedLock(static_cast<IEditableProvider&>(RegionProvider));
	RegionProvider.OnAnalysisSessionEnded();
}

void FMiscTraceAnalyzer::OnThreadInfo(const FThreadInfo& ThreadInfo)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FMiscTraceAnalyzer"));

	uint32 ThreadId = ThreadInfo.GetId();
	FString Name = ThreadInfo.GetName();

	FAnalysisSessionEditScope _(Session);

	ThreadProvider.AddThread(ThreadId, *Name, EThreadPriority(ThreadInfo.GetSortHint()));

	const ANSICHAR* GroupNameA = ThreadInfo.GetGroupName();
	if (*GroupNameA)
	{
		const TCHAR* GroupName = Session.StoreString(ANSI_TO_TCHAR(GroupNameA));
		ThreadProvider.SetThreadGroup(ThreadId, GroupName);
	}
}

bool FMiscTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FMiscTraceAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;

	if (Session.GetBaseDateTime() == 0.0)
	{
		Session.SetBaseDateTime(Context.EventTime.GetBaseDateTimestamp());
	}

	switch (RouteId)
	{
	case RouteId_BeginFrame:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint8 FrameType = EventData.GetValue<uint8>("FrameType");
		check(FrameType < TraceFrameType_Count);
		FrameProvider.BeginFrame(ETraceFrameType(FrameType), Context.EventTime.AsSeconds(Cycle));
		break;
	}

	case RouteId_EndFrame:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint8 FrameType = EventData.GetValue<uint8>("FrameType");
		check(FrameType < TraceFrameType_Count);
		FrameProvider.EndFrame(ETraceFrameType(FrameType), Context.EventTime.AsSeconds(Cycle));
		break;
	}

	case RouteId_BeginGameFrame:
	case RouteId_EndGameFrame:
	case RouteId_BeginRenderFrame:
	case RouteId_EndRenderFrame:
	{
		ETraceFrameType FrameType;
		if (RouteId == RouteId_BeginGameFrame || RouteId == RouteId_EndGameFrame)
		{
			FrameType = TraceFrameType_Game;
		}
		else
		{
			FrameType = TraceFrameType_Rendering;
		}
		const uint8* BufferPtr = EventData.GetAttachment();
		uint64 CycleDiff = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		uint64 Cycle = LastFrameCycle[FrameType] + CycleDiff;
		LastFrameCycle[FrameType] = Cycle;
		if (RouteId == RouteId_BeginGameFrame || RouteId == RouteId_BeginRenderFrame)
		{
			FrameProvider.BeginFrame(FrameType, Context.EventTime.AsSeconds(Cycle));
		}
		else
		{
			FrameProvider.EndFrame(FrameType, Context.EventTime.AsSeconds(Cycle));
		}
		break;
	}

	case RouteId_ChannelAnnounce:
		OnChannelAnnounce(Context);
		break;

	case RouteId_ChannelToggle:
		OnChannelToggle(Context);
		break;

	case RouteId_ScreenshotHeader:
	{
		uint32 Id = EventData.GetValue<uint32>("Id");
		TSharedPtr<FScreenshot> Screenshot = ScreenshotProvider.AddScreenshot(Id);
		Screenshot->Id = Id;

		EventData.GetString("Name", Screenshot->Name);

		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		Screenshot->Timestamp = Context.EventTime.AsSeconds(Cycle);

		Screenshot->Width = EventData.GetValue<uint32>("Width");
		Screenshot->Height = EventData.GetValue<uint32>("Height");
		Screenshot->ChunkNum = EventData.GetValue<uint32>("TotalChunkNum");
		Screenshot->Size = EventData.GetValue<uint32>("Size");
		Screenshot->Data.Reserve(Screenshot->Size);

		FLogMessageSpec& LogMessageSpec = LogProvider.GetMessageSpec(Cycle);
		LogMessageSpec.Category = &LogProvider.GetCategory(ScreenshotLogCategoryId);
		LogMessageSpec.Line = Id;
		LogMessageSpec.File = nullptr;
		LogMessageSpec.FormatString = nullptr;
		LogMessageSpec.Verbosity = ELogVerbosity::Log;

		LogProvider.AppendMessage(Cycle, Screenshot->Timestamp, Screenshot->Name);

		break;
	}

	case RouteId_ScreenshotChunk:
	{
		uint32 Id = EventData.GetValue<uint32>("Id");
		uint16 ChunkNum = EventData.GetValue<uint16>("ChunkNum");
		uint16 Size = EventData.GetValue<uint16>("Size");
		TArrayView<const uint8> Data = EventData.GetArrayView<uint8>("Data");

		ScreenshotProvider.AddScreenshotChunk(Id, ChunkNum, Size, Data);

		break;
	}

	// Begin retired events
	//
	case RouteId_RegisterGameThread:
	{
		const uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		ThreadProvider.AddGameThread(ThreadId);
		break;
	}

	case RouteId_CreateThread:
	{
		const uint32 CreatedThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context, "CreatedThreadId");
		const EThreadPriority Priority = static_cast<EThreadPriority>(EventData.GetValue<uint32>("Priority"));
		const TCHAR* CreatedThreadName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
		ThreadProvider.AddThread(CreatedThreadId, CreatedThreadName, Priority);
		const uint32 CurrentThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context, "CurrentThreadId");
		FThreadState* ThreadState = GetThreadState(CurrentThreadId);
		if (ThreadState->ThreadGroupStack.Num())
		{
			ThreadProvider.SetThreadGroup(CreatedThreadId, ThreadState->ThreadGroupStack.Top());
		}
		break;
	}

	case RouteId_SetThreadGroup:
	{
		const TCHAR* GroupName = Session.StoreString(ANSI_TO_TCHAR(reinterpret_cast<const char*>(EventData.GetAttachment())));
		const uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		ThreadProvider.SetThreadGroup(ThreadId, GroupName);
		break;
	}

	case RouteId_BeginThreadGroupScope:
	{
		const TCHAR* GroupName = Session.StoreString(ANSI_TO_TCHAR(reinterpret_cast<const char*>(EventData.GetAttachment())));
		const uint32 CurrentThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context, "CurrentThreadId");
		FThreadState* ThreadState = GetThreadState(CurrentThreadId);
		ThreadState->ThreadGroupStack.Push(GroupName);
		break;
	}

	case RouteId_EndThreadGroupScope:
	{
		const uint32 CurrentThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context, "CurrentThreadId");
		FThreadState* ThreadState = GetThreadState(CurrentThreadId);
		ThreadState->ThreadGroupStack.Pop();
		break;
	}
	//
	// End retired events

	case RouteId_RegionBegin:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		FString Name;
		check(EventData.GetString("RegionName", Name));
			
		// Category has been added after the initial implementation, so it's treated as optional
		FString Category;
		EventData.GetString("Category", Category);
		double Time = Context.EventTime.AsSeconds(Cycle);
		{
			FProviderEditScopeLock RegionProviderScopedLock(RegionProvider);
			RegionProvider.AppendRegionBegin(*Name, Time, Category.IsEmpty() ? nullptr : *Category);
		}
			Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_RegionBeginWithId:
	{
		uint64 CycleAndId = EventData.GetValue<uint64>("CycleAndId");
		FString Name;
		check(EventData.GetString("RegionName", Name));
			
		// Category has been added after the initial implementation, so it's treated as optional
		FString Category;
		EventData.GetString("Category", Category);
		double Time = Context.EventTime.AsSeconds(CycleAndId);
		{
			FProviderEditScopeLock RegionProviderScopedLock(RegionProvider);
			RegionProvider.AppendRegionBeginWithId(*Name, CycleAndId, Time, Category.IsEmpty() ? nullptr : *Category);
		}
		
		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_RegionEnd:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		FString Name;
		EventData.GetString("RegionName", Name);
		double Time = Context.EventTime.AsSeconds(Cycle);
		{
			FProviderEditScopeLock RegionProviderScopedLock(RegionProvider);
			RegionProvider.AppendRegionEnd(*Name, Time);
		}

		Session.UpdateDurationSeconds(Time);
		break;
	}

	case RouteId_RegionEndWithId:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 RegionId = EventData.GetValue<uint64>("RegionId", 0);
		double Time = Context.EventTime.AsSeconds(Cycle);
		{
			FProviderEditScopeLock RegionProviderScopedLock(RegionProvider);
			RegionProvider.AppendRegionEndWithId(RegionId, Time);
		}

		Session.UpdateDurationSeconds(Time);
		break;
	}
	}

	return true;
}

void FMiscTraceAnalyzer::OnChannelAnnounce(const FOnEventContext& Context)
{
	FString ChannelName = FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("Name", Context);
	uint32 ChannelId = Context.EventData.GetValue<uint32>("Id");
	bool bEnabled = Context.EventData.GetValue<bool>("IsEnabled");
	bool bReadOnly = Context.EventData.GetValue<bool>("ReadOnly", false);

	ChannelProvider.AnnounceChannel(*ChannelName, ChannelId, bReadOnly);
	ChannelProvider.UpdateChannel(ChannelId, bEnabled);
}

void FMiscTraceAnalyzer::OnChannelToggle(const FOnEventContext& Context)
{
	uint32 ChannelId = Context.EventData.GetValue<uint32>("Id");
	bool bEnabled = Context.EventData.GetValue<bool>("IsEnabled");
	ChannelProvider.UpdateChannel(ChannelId, bEnabled);
}

FMiscTraceAnalyzer::FThreadState* FMiscTraceAnalyzer::GetThreadState(uint32 ThreadId)
{
	if (!ThreadStateMap.Contains(ThreadId))
	{
		TSharedRef<FThreadState> ThreadState = MakeShared<FThreadState>();
		ThreadStateMap.Add(ThreadId, ThreadState);
	}
	return &ThreadStateMap[ThreadId].Get();
}

} // namespace TraceServices
