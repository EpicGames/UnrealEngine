// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Templates/SharedPointer.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Common/PagedArray.h"

namespace TraceServices
{

class IAnalysisSession;
class FThreadProvider;
class FLogProvider;
class FFrameProvider;
class FChannelProvider;
class FScreenshotProvider;
class FRegionProvider;

class FMiscTraceAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FMiscTraceAnalyzer(IAnalysisSession& Session,
					   FThreadProvider& ThreadProvider,
					   FLogProvider& LogProvider,
					   FFrameProvider& FrameProvider, 
					   FChannelProvider& ChannelProvider,
					   FScreenshotProvider& ScreenshotProvider,
					   FRegionProvider& RegionProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual void OnThreadInfo(const FThreadInfo& ThreadInfo) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_RegisterGameThread,
		RouteId_CreateThread,
		RouteId_SetThreadGroup,
		RouteId_BeginThreadGroupScope,
		RouteId_EndThreadGroupScope,
		RouteId_BeginFrame,
		RouteId_EndFrame,
		RouteId_BeginGameFrame,
		RouteId_EndGameFrame,
		RouteId_BeginRenderFrame,
		RouteId_EndRenderFrame,
		RouteId_ChannelAnnounce,
		RouteId_ChannelToggle,
		RouteId_ScreenshotHeader,
		RouteId_ScreenshotChunk,
		RouteId_RegionBegin,
		RouteId_RegionBeginWithId,
		RouteId_RegionEnd,
		RouteId_RegionEndWithId
	};

	struct FThreadState
	{
		TArray<const TCHAR*> ThreadGroupStack;
	};

	FThreadState* GetThreadState(uint32 ThreadId);
	void OnChannelAnnounce(const FOnEventContext& Context);
	void OnChannelToggle(const FOnEventContext& Context);

	IAnalysisSession& Session;
	FThreadProvider& ThreadProvider;
	FLogProvider& LogProvider;
	FFrameProvider& FrameProvider;
	FChannelProvider& ChannelProvider;
	FScreenshotProvider& ScreenshotProvider;
	FRegionProvider& RegionProvider;
	
	TMap<uint32, TSharedRef<FThreadState>> ThreadStateMap;
	uint64 LastFrameCycle[TraceFrameType_Count] = { 0, 0 };
	uint64 ScreenshotLogCategoryId = uint64(-1);
};


} // namespace TraceServices
