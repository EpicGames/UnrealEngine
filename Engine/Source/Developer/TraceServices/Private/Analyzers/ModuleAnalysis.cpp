// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModuleAnalysis.h"

#include "HAL/LowLevelMemTracker.h"
#include "Model/ModuleProvider.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////

enum Routes
{
	RouteId_ModuleInit,
	RouteId_ModuleLoad,
	RouteId_ModuleUnload,
};

////////////////////////////////////////////////////////////////////////////////

FModuleAnalyzer::FModuleAnalyzer(IAnalysisSession& InSession)
	: Session(InSession)
	, Provider(nullptr)
	, ModuleBaseShift(0)
{
}

////////////////////////////////////////////////////////////////////////////////

void FModuleAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;
	Builder.RouteEvent(RouteId_ModuleInit, "Diagnostics", "ModuleInit");
	Builder.RouteEvent(RouteId_ModuleLoad, "Diagnostics", "ModuleLoad");
	Builder.RouteEvent(RouteId_ModuleUnload, "Diagnostics", "ModuleUnload");
}

////////////////////////////////////////////////////////////////////////////////

void FModuleAnalyzer::OnAnalysisEnd()
{
	if (Provider)
	{
		Provider->OnAnalysisComplete();
	}
}

////////////////////////////////////////////////////////////////////////////////

bool FModuleAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FModuleAnalyzer"));

	switch (RouteId)
	{
		case RouteId_ModuleInit:
			{
				ModuleBaseShift = IntCastChecked<uint8>(Context.EventData.GetValue("ModuleBaseShift", 16));
				FAnsiStringView SymbolFormat;
				if (Context.EventData.GetString("SymbolFormat", SymbolFormat))
				{
					check(Provider == nullptr); // Should only get one init message
					TSharedPtr<IModuleAnalysisProvider> ModuleProvider = CreateModuleProvider(Session, SymbolFormat);
					if (ModuleProvider)
					{
						Session.AddProvider(GetModuleProviderName(), ModuleProvider, ModuleProvider);
						Provider = ModuleProvider.Get();
					}
				}
			}
			break;

		case RouteId_ModuleLoad:
			if (Provider != nullptr)
			{
				FStringView ModuleName;
				if (Context.EventData.GetString("Name", ModuleName))
				{
					const uint64 Base = GetBaseAddress(Context.EventData);
					const uint32 Size = Context.EventData.GetValue<uint32>("Size");
					const TArrayReader<uint8>& ImageId = Context.EventData.GetArray<uint8>("ImageId");
					Provider->OnModuleLoad(ModuleName, Base, Size, ImageId.GetData(), ImageId.Num());
				}
			}
			break;

		case RouteId_ModuleUnload:
			if (Provider != nullptr)
			{
				const uint64 Base = GetBaseAddress(Context.EventData);
				Provider->OnModuleUnload(Base);
			}
			break;
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////

uint64 FModuleAnalyzer::GetBaseAddress(const FEventData& EventData) const
{
	if (ModuleBaseShift == 0)
	{
		// With 0 base shift assume newer trace event is used with 64-bit base address
		return EventData.GetValue<uint64>("Base");
	}
	else
	{
		// Unshift base address according to capture alignment (e.g. Windows 4k aligned)
		uint64 Base = uint64(EventData.GetValue<uint32>("Base"));
		return Base << ModuleBaseShift;
	}
}

////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
