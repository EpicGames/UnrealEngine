// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Trace/Analyzer.h"

namespace TraceServices
{

class IAnalysisSession;
class IModuleAnalysisProvider;

class FModuleAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FModuleAnalyzer(IAnalysisSession& Session);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	uint64 GetBaseAddress(const FEventData& EventData) const;

private:
	IAnalysisSession& Session;
	IModuleAnalysisProvider* Provider;
	uint8 ModuleBaseShift;
};

} // namespace TraceServices
