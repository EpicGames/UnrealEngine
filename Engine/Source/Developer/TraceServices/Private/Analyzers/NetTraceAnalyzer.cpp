// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetTraceAnalyzer.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "TraceServices/Model/Frames.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"
#include "TraceServices/Model/Threads.h"


DEFINE_LOG_CATEGORY_STATIC(LogNetTrace, Log, All);

namespace TraceServices
{

enum ENetTraceAnalyzerVersion
{
	ENetTraceAnalyzerVersion_Initial = 1,
	ENetTraceAnalyzerVersion_BunchChannelIndex = 2,
	ENetTraceAnalyzerVersion_BunchChannelInfo = 3,
	ENetTraceAnalyzerVersion_FixedBunchSizeEncoding = 4,
	ENetTraceAnalyzerVersion_DebugNameIndexIs32Bits = 5,
};


FNetTraceAnalyzer::FNetTraceAnalyzer(IAnalysisSession& InSession, FNetProfilerProvider& InNetProfilerProvider)
	: Session(InSession)
	, NetProfilerProvider(InNetProfilerProvider)
	, FrameProvider(ReadFrameProvider(InSession))
	, NetTraceVersion(0)
	, NetTraceReporterVersion(0)
{
}

void FNetTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_InitEvent, "NetTrace", "InitEvent");
	Builder.RouteEvent(RouteId_InstanceDestroyedEvent, "NetTrace", "InstanceDestroyedEvent");
	Builder.RouteEvent(RouteId_NameEvent, "NetTrace", "NameEvent");
	Builder.RouteEvent(RouteId_PacketContentEvent, "NetTrace", "PacketContentEvent");
	Builder.RouteEvent(RouteId_PacketEvent, "NetTrace", "PacketEvent");
	Builder.RouteEvent(RouteId_PacketDroppedEvent, "NetTrace", "PacketDroppedEvent");
	Builder.RouteEvent(RouteId_ConnectionCreatedEvent, "NetTrace", "ConnectionCreatedEvent");
	Builder.RouteEvent(RouteId_ConnectionUpdatedEvent, "NetTrace", "ConnectionUpdatedEvent");
	// Add some default event types that we use for generic type events to make it easier to extend
	// ConnectionAdded/Removed connections state /name etc?
	Builder.RouteEvent(RouteId_ConnectionClosedEvent, "NetTrace", "ConnectionClosedEvent");
	Builder.RouteEvent(RouteId_PacketStatsCounterEvent, "NetTrace", "PacketStatsCounterEvent");
	Builder.RouteEvent(RouteId_FrameStatsCounterEvent, "NetTrace", "FrameStatsCounterEvent");
	Builder.RouteEvent(RouteId_ObjectCreatedEvent, "NetTrace", "ObjectCreatedEvent");
	Builder.RouteEvent(RouteId_ObjectExistsEvent, "NetTrace", "ObjectExistsEvent");
	Builder.RouteEvent(RouteId_ObjectDestroyedEvent, "NetTrace", "ObjectDestroyedEvent");
	Builder.RouteEvent(RouteId_ConnectionStateUpdatedEvent, "NetTrace", "ConnectionStateUpdatedEvent");
	Builder.RouteEvent(RouteId_InstanceUpdatedEvent, "NetTrace", "InstanceUpdatedEvent");

	// Default names
	{
		FAnalysisSessionEditScope _(Session);
		BunchHeaderNameIndex = NetProfilerProvider.AddNetProfilerName(TEXT("BunchHeader"));
		PendingNameIndex = NetProfilerProvider.AddNetProfilerName(TEXT("Pending"));
	}
}

void FNetTraceAnalyzer::OnAnalysisEnd()
{
}

uint32 FNetTraceAnalyzer::GetTracedEventTypeIndex(FNetProfilerNameIndexType NameIndex, uint8 Level)
{
	const uint64 TracedEventTypeKey = (((uint64)NameIndex) << 8U) | (uint64)Level;

	if (const uint32* NetProfilerEventTypeIndex = TraceEventTypeToNetProfilerEventTypeIndexMap.Find(TracedEventTypeKey))
	{
		return *NetProfilerEventTypeIndex;
	}
	else
	{
		// Add new EventType
		uint32 NewEventTypeIndex = NetProfilerProvider.AddNetProfilerEventType(NameIndex, Level);
		TraceEventTypeToNetProfilerEventTypeIndexMap.Add(TracedEventTypeKey, NewEventTypeIndex);

		return NewEventTypeIndex;
	}
}

bool FNetTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FNetTraceAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	// sometimes when a trace starts, the cache buffer contains some events which get written before the init event can be written,
	// so we need to discard those. NameEvents are part of the ImportantEvents, so are also expected to arrive before InitEvent
	if ((NetTraceVersion == 0) && (RouteId != RouteId_InitEvent) && (RouteId != RouteId_NameEvent))
	{
		return true;
	}

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_InitEvent:
		{
			const uint64 TimestampCycles = EventData.GetValue<uint64>("Timestamp");
			StartTimeStamp = Context.EventTime.AsSeconds(TimestampCycles);
			LastTimeStamp = StartTimeStamp;

			// we always trace the version so that we make sure that we are backwards compatible with older trace stream
			NetTraceVersion = EventData.GetValue<uint32>("NetTraceVersion");
			NetTraceReporterVersion = EventData.GetValue<uint32>("NetTraceReporterVersion");

			NetProfilerProvider.SetNetTraceVersion(NetTraceVersion);
		}
		break;

		case RouteId_InstanceDestroyedEvent:
		{
			const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
			DestroyActiveGameInstanceState(GameInstanceId);
		}
		break;

		case RouteId_NameEvent:
		{
			FNetProfilerNameIndexType TraceNameId = EventData.GetValue<FNetProfilerNameIndexType>("NameId");
			if (TracedNameIdToNetProfilerNameIdMap.Contains(TraceNameId))
			{
				// need to update the name
				check(false);
			}
			else
			{
				FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<UTF8CHAR>("Name", Context);
				TracedNameIdToNetProfilerNameIdMap.Add(TraceNameId, NetProfilerProvider.AddNetProfilerName(*Name));
			}
		}
		break;

		case RouteId_PacketContentEvent:
		{
			HandlePacketContentEvent(Context, EventData);
		}
		break;

		case RouteId_PacketEvent:
		{
			HandlePacketEvent(Context, EventData);
		}
		break;

		case RouteId_PacketDroppedEvent:
		{
			HandlePacketDroppedEvent(Context, EventData);
		}
		break;

		case RouteId_ConnectionCreatedEvent:
		{
			HandleConnectionCreatedEvent(Context, EventData);
		}
		break;

		case RouteId_PacketStatsCounterEvent:
		{
			HandlePacketStatsCounterEvent(Context, EventData);
		}
		break;

		case RouteId_FrameStatsCounterEvent:
		{
			HandleFrameStatsCounterEvent(Context, EventData);
		}
		break;

		case RouteId_ConnectionStateUpdatedEvent:
		{
			HandleConnectionStateUpdatedEvent(Context, EventData);
		}
		break;

		case RouteId_ConnectionUpdatedEvent:
		{
			HandleConnectionUpdatedEvent(Context, EventData);
		}
		break;

		case RouteId_ConnectionClosedEvent:
		{
			HandleConnectionClosedEvent(Context, EventData);
		}
		break;

		case RouteId_ObjectCreatedEvent:
		{
			HandleObjectCreatedEvent(Context, EventData);
		}
		break;

		case RouteId_ObjectExistsEvent:
		{
			HandleObjectExistsEvent(Context, EventData);
		}
		break;

		case RouteId_ObjectDestroyedEvent:
		{
			HandleObjectDestroyedEvent(Context, EventData);
		}
		break;

		case RouteId_InstanceUpdatedEvent:
		{
			HandleGameInstanceUpdatedEvent(Context, EventData);
		}
		break;
	}

	return true;
}

void FNetTraceAnalyzer::HandlePacketContentEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint8 PacketType =  EventData.GetValue<uint8>("PacketType");

	//UE_LOGF(LogNetTrace, Display, "FNetTraceAnalyzer::HandlePacketContentEvent: GameInstanceId: %u, ConnectionId: %u, %ls", (uint32)GameInstanceId, (uint32)ConnectionId, PacketType ? TEXT("Incoming") : TEXT("Outgoing"));

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);
	FNetTraceConnectionState* ConnectionState = GetActiveConnectionState(GameInstanceId, ConnectionId);
	if (!ConnectionState)
	{
		return;
	}

	const ENetProfilerConnectionMode ConnectionMode = ENetProfilerConnectionMode(PacketType);

	TArray<FNetProfilerContentEvent>& Events = (ConnectionState->BunchEvents)[ConnectionMode];
	TArray<FBunchInfo>& BunchInfos = (ConnectionState->BunchInfos)[ConnectionMode];

	// Decode batched events
	TArrayView<const uint8> DataView = FTraceAnalyzerUtils::LegacyAttachmentArray("Data", Context);
	uint64 BufferSize = DataView.Num();
	const uint8* BufferPtr = DataView.GetData();
	const uint8* BufferEnd = BufferPtr + BufferSize;
	uint64 LastOffset = 0;

	uint64 CurrentBunchOffset = 0U;

	while (BufferPtr < BufferEnd)
	{
		// Decode data
		const EContentEventType DecodedEventType = EContentEventType(*BufferPtr++);
		switch (DecodedEventType)
		{
			case EContentEventType::Object:
			case EContentEventType::NameId:
			{
				FNetProfilerContentEvent& Event = Events.Emplace_GetRef();

				const uint8 DecodedNestingLevel = *BufferPtr++;

				const uint64 DecodedNameOrObjectId = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
				const uint64 DecodedEventStartPos = FTraceAnalyzerUtils::Decode7bit(BufferPtr) + LastOffset;
				LastOffset = DecodedEventStartPos;
				const uint64 DecodedEventEndPos = FTraceAnalyzerUtils::Decode7bit(BufferPtr) + DecodedEventStartPos;

				// Fill in event data
				Event.StartPos = DecodedEventStartPos + CurrentBunchOffset;
				Event.EndPos = DecodedEventEndPos + CurrentBunchOffset;
				Event.Level = DecodedNestingLevel;

				Event.ObjectInstanceIndex = 0;
				Event.NameIndex = 0;
				Event.BunchInfo.Value = 0;

				checkSlow(Event.EndPos > Event.StartPos);

				if (DecodedEventType == EContentEventType::Object)
				{
					// Object index, need to lookup name indirectly
					if (const FNetTraceActiveObjectState* ActiveObjectState = GameInstanceState->ActiveObjects.Find(DecodedNameOrObjectId))
					{
						Event.NameIndex = ActiveObjectState->NameIndex;
						Event.ObjectInstanceIndex = ActiveObjectState->ObjectIndex;
					}
					else if (DecodedNameOrObjectId != 0)
					{
						// Sometime we report data for objects that are still pending creation, which we will update as soon as we have more data.
						FNetProfilerObjectInstance& ObjectInstance = NetProfilerProvider.CreateObject(GameInstanceState->GameInstanceIndex);

						// Fill in the object data we currently have
						ObjectInstance.LifeTime.Begin = GetLastTimestamp();
						ObjectInstance.NameIndex = PendingNameIndex;
						ObjectInstance.NetObjectId = DecodedNameOrObjectId;
						ObjectInstance.TypeId = 0;

						// Add to active objects
						GameInstanceState->ActiveObjects.Add(DecodedNameOrObjectId, { ObjectInstance.ObjectIndex, ObjectInstance.NameIndex });

						Event.NameIndex = ObjectInstance.NameIndex;
						Event.ObjectInstanceIndex = ObjectInstance.ObjectIndex;
					}
				}
				else if (DecodedEventType == EContentEventType::NameId)
				{
					if (const FNetProfilerNameIndexType* NetProfilerNameIndex = TracedNameIdToNetProfilerNameIdMap.Find(IntCastChecked<FNetProfilerNameIndexType>(DecodedNameOrObjectId)))
					{
						Event.NameIndex = *NetProfilerNameIndex;
					}
					else
					{
						UE_LOGF(LogNetTrace, Warning, "PacketContentEvent GameInstanceId: %u, ConnectionId: %u %ls, Missing NameIndex: %llu", (uint32)GameInstanceId, (uint32)ConnectionId, ConnectionMode ? TEXT("Incoming") : TEXT("Outgoing"), DecodedNameOrObjectId);
					}
				}

				// EventTypeIndex does not match NameIndex as we might see the same name on different levels
				Event.EventTypeIndex = GetTracedEventTypeIndex(Event.NameIndex, static_cast<uint8>(Event.Level));
			}
			break;

			case EContentEventType::BunchEvent:
			{
				const FNetProfilerNameIndexType DecodedNameId = IntCastChecked<FNetProfilerNameIndexType>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));

				uint32 DecodedBunchBits = 0U;
				if (NetTraceVersion >= ENetTraceAnalyzerVersion_FixedBunchSizeEncoding)
				{
					DecodedBunchBits = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));
				}
				else
				{
					const uint64 DecodedEventStartPos = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
					const uint64 DecodedEventEndPos = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
					DecodedBunchBits = IntCastChecked<uint32>((uint32)DecodedEventEndPos + (uint32)DecodedEventStartPos);
				}

				const FNetProfilerNameIndexType* NetProfilerNameIndex = DecodedNameId ? TracedNameIdToNetProfilerNameIdMap.Find(DecodedNameId) : nullptr;

				FBunchInfo BunchInfo;

				BunchInfo.BunchInfo.Value = 0;
				BunchInfo.HeaderBits = 0U;
				BunchInfo.BunchBits = DecodedBunchBits;
				BunchInfo.FirstBunchEventIndex = Events.Num();
				BunchInfo.NameIndex = NetProfilerNameIndex ? *NetProfilerNameIndex : 0U;

				BunchInfos.Add(BunchInfo);

				// Must reset LastOffset after reading bunch data
				LastOffset = 0U;
			}
			break;

			case EContentEventType::BunchHeaderEvent:
			{
				const uint32 DecodedEventCount = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));
				const uint32 DecodedHeaderBits = IntCastChecked<uint32>(FTraceAnalyzerUtils::Decode7bit(BufferPtr));

				FBunchInfo& BunchInfo = BunchInfos.Last();

				BunchInfo.EventCount = DecodedEventCount;
				BunchInfo.FirstBunchEventIndex = Events.Num() - DecodedEventCount;

				// A bunch with header bits set is an actual bunch
				if (DecodedHeaderBits)
				{
					if (NetTraceVersion >= ENetTraceAnalyzerVersion_BunchChannelIndex)
					{
						const uint64 DecodedBunchInfo = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
						if (NetTraceVersion >= ENetTraceAnalyzerVersion_BunchChannelInfo)
						{
							BunchInfo.BunchInfo.Value = DecodedBunchInfo;
						}
						else
						{
							BunchInfo.BunchInfo.Value = uint64(0);
							BunchInfo.BunchInfo.ChannelIndex = DecodedBunchInfo;
						}

						if (BunchInfo.NameIndex)
						{
							GameInstanceState->ChannelNames.FindOrAdd(BunchInfo.BunchInfo.ChannelIndex) = BunchInfo.NameIndex;
						}
						else
						{
							const FNetProfilerNameIndexType* ExistingChannelNameIndex = GameInstanceState->ChannelNames.Find(BunchInfo.BunchInfo.ChannelIndex);
							BunchInfo.NameIndex = ExistingChannelNameIndex ? *ExistingChannelNameIndex : 0U;
						}

						BunchInfo.BunchInfo.bIsValid = 1U;
					}

					BunchInfo.HeaderBits = DecodedHeaderBits;
					CurrentBunchOffset = 0U;
				}
				else
				{
					// Merged bunch, set offset for events
					CurrentBunchOffset = BunchInfo.BunchBits;
				}
			}
			break;
		};

	}
	check(BufferPtr == BufferEnd);
}

void FNetTraceAnalyzer::AddEvent(TPagedArray<FNetProfilerContentEvent>& Events, const FNetProfilerContentEvent& InEvent, uint32 Offset, uint32 LevelOffset)
{
	FNetProfilerContentEvent& Event = Events.PushBack();

	Event.EventTypeIndex = GetTracedEventTypeIndex(InEvent.NameIndex, IntCastChecked<uint8>(InEvent.Level + LevelOffset));
	Event.NameIndex = InEvent.NameIndex;
	Event.ObjectInstanceIndex = InEvent.ObjectInstanceIndex;
	Event.StartPos = InEvent.StartPos + Offset;
	Event.EndPos = InEvent.EndPos + Offset;
	Event.Level = InEvent.Level + LevelOffset;
	Event.BunchInfo = InEvent.BunchInfo;
}

void FNetTraceAnalyzer::AddEvent(TPagedArray<FNetProfilerContentEvent>& Events, uint32 StartPos, uint32 EndPos, uint32 Level, FNetProfilerNameIndexType NameIndex, FNetProfilerBunchInfo BunchInfo)
{
	FNetProfilerContentEvent& Event = Events.PushBack();

	Event.EventTypeIndex = GetTracedEventTypeIndex(NameIndex, IntCastChecked<uint8>(Level));
	Event.NameIndex = NameIndex;
	Event.ObjectInstanceIndex = 0;
	Event.StartPos = StartPos;
	Event.EndPos = EndPos;
	Event.Level = Level;
	Event.BunchInfo = BunchInfo;
}

void FNetTraceAnalyzer::FlushPacketEvents(FNetTraceConnectionState& ConnectionState, FNetProfilerConnectionData& ConnectionData, const ENetProfilerConnectionMode ConnectionMode)
{
	TPagedArray<FNetProfilerContentEvent>& Events = ConnectionData.ContentEvents;

	TArray<FNetProfilerContentEvent>& BunchEvents = ConnectionState.BunchEvents[ConnectionMode];

	int32 CurrentBunchEventIndex = 0;

	// Track bunch offsets
	uint32 NextBunchOffset = 0U;

	int32 NonBunchEventCount = ConnectionState.BunchInfos[ConnectionMode].Num() ? ConnectionState.BunchInfos[ConnectionMode][0].FirstBunchEventIndex : BunchEvents.Num();

	// Inject any events reported before the first bunch
	while (CurrentBunchEventIndex < NonBunchEventCount)
	{
		const FNetProfilerContentEvent& BunchEvent = BunchEvents[CurrentBunchEventIndex];

		AddEvent(Events, BunchEvent, 0U, 0U);

		NextBunchOffset = FMath::Max(static_cast<uint32>(BunchEvent.EndPos), NextBunchOffset);
		++CurrentBunchEventIndex;
		++ConnectionData.ContentEventChangeCount;
	}

	uint32 EventsToAdd = 0U;
	for (const FBunchInfo& Bunch : ConnectionState.BunchInfos[ConnectionMode])
	{
		uint32 BunchOffset = NextBunchOffset + Bunch.HeaderBits;
		EventsToAdd += Bunch.EventCount;

		// Report events for committed bunches
		if (Bunch.HeaderBits)
		{
			// Bunch event
			AddEvent(Events, NextBunchOffset, NextBunchOffset + Bunch.HeaderBits + Bunch.BunchBits, 0, Bunch.NameIndex, Bunch.BunchInfo);

			// Bunch header event
			AddEvent(Events, NextBunchOffset, NextBunchOffset + Bunch.HeaderBits, 1, BunchHeaderNameIndex, FNetProfilerBunchInfo::MakeBunchInfo(0));

			// Add events belonging to bunch, including the ones from merged bunches
			for (uint32 EventIt = 0; EventIt < EventsToAdd; ++EventIt)
			{
				const FNetProfilerContentEvent& BunchEvent = BunchEvents[CurrentBunchEventIndex];

				AddEvent(Events, BunchEvent, BunchOffset, 1U);
				++CurrentBunchEventIndex;
			}

			// Accumulate offset
			NextBunchOffset += Bunch.BunchBits + Bunch.HeaderBits;

			// Reset event count
			EventsToAdd = 0U;
		}

		++ConnectionData.ContentEventChangeCount;
	}

	ConnectionState.BunchEvents[ConnectionMode].Reset();
	ConnectionState.BunchInfos[ConnectionMode].Reset();
}

void FNetTraceAnalyzer::HandlePacketEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint64 TimestampCycles = EventData.GetValue<uint64>("Timestamp");
	const uint32 PacketBits = EventData.GetValue<uint32>("PacketBits");
	const uint32 SequenceNumber = EventData.GetValue<uint32>("SequenceNumber");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");
	const uint8 PacketType = EventData.GetValue<uint8>("PacketType");

	const ENetProfilerConnectionMode ConnectionMode = ENetProfilerConnectionMode(PacketType);

	// Update LastTimestamp, later on we will be able to get timestamps piggybacked from other analyzers
	LastTimeStamp = Context.EventTime.AsSeconds(TimestampCycles);

	// Get the NetProfilerFrameIndex for the current engine frame/timestamp
	const uint32 NetProfilerFrameIndex = GetCurrentNetProfilerFrameIndexAndFlushFrameStatsCountersIfNeeded(
		GameInstanceId,
		FrameProvider.GetFrameNumberForTimestamp(ETraceFrameType::TraceFrameType_Game, LastTimeStamp)
	);

	FNetTraceConnectionState* ConnectionState = GetActiveConnectionState(GameInstanceId, ConnectionId);
	if (!ConnectionState)
	{
		return;
	}

	// Add the packet
	FNetProfilerConnectionData& ConnectionData = NetProfilerProvider.EditConnectionData(ConnectionState->ConnectionIndex, ConnectionMode);

	if (ConnectionMode == ENetProfilerConnectionMode::Incoming)
	{
		if (ConnectionData.Packets.Num() > 0)
		{
			uint32 ExpectedSequenceNumber = ConnectionData.Packets.Last().SequenceNumber + 1U;
			while (ExpectedSequenceNumber < SequenceNumber)
			{
				// Inject packets to visualize missing packets
				FNetProfilerPacket& Packet = ConnectionData.Packets.PushBack();
				Packet.SequenceNumber = ExpectedSequenceNumber;

				// Fake it
				Packet.StartEventIndex = ConnectionState->CurrentPacketStartIndex[ConnectionMode];
				Packet.EventCount = 0;
				Packet.TimeStamp = GetLastTimestamp();
				Packet.DeliveryStatus = ENetProfilerDeliveryStatus::Dropped;
				Packet.ConnectionState = ConnectionState->ConnectionState;
				Packet.ContentSizeInBits = 0;
				Packet.TotalPacketSizeInBytes = (Packet.ContentSizeInBits + 7u) >> 3u;

				++ExpectedSequenceNumber;
				++ConnectionData.PacketChangeCount;
			}
		}
	}

	FNetProfilerPacket& Packet = ConnectionData.Packets.PushBack();
	++ConnectionData.PacketChangeCount;

	// Flush packet events
	FlushPacketEvents(*ConnectionState, ConnectionData, ConnectionMode);

	Packet.NetProfilerFrameIndex = NetProfilerFrameIndex;

	// Fill in packet data a packet must have at least 1 event?
	Packet.StartEventIndex = ConnectionState->CurrentPacketStartIndex[ConnectionMode];
	Packet.EventCount = static_cast<uint32>(ConnectionData.ContentEvents.Num()) - Packet.StartEventIndex;
	Packet.TimeStamp = GetLastTimestamp();
	Packet.SequenceNumber = SequenceNumber;
	Packet.DeliveryStatus = ENetProfilerDeliveryStatus::Unknown;
	Packet.ConnectionState = ConnectionState->ConnectionState;

	Packet.ContentSizeInBits = PacketBits;
	Packet.TotalPacketSizeInBytes = (Packet.ContentSizeInBits + 7u) >> 3u;
	Packet.DeliveryStatus = ENetProfilerDeliveryStatus::Delivered;

	// Flush PacketStats
	Packet.StartStatsIndex = static_cast<uint32>(ConnectionData.PacketStats.Num());
	Packet.StatsCount = ConnectionState->PacketStats.Num();
	for (const FNetProfilerStats& Stat : ConnectionState->PacketStats)
	{
		ConnectionData.PacketStats.EmplaceBack(Stat);
	}

	ConnectionState->PacketStats.Reset();
	++ConnectionData.PacketStatsChangeCount;


	// Mark the beginning of a new packet
	ConnectionState->CurrentPacketStartIndex[ConnectionMode] = static_cast<uint32>(ConnectionData.ContentEvents.Num());
	ConnectionState->CurrentPacketBitOffset[ConnectionMode] = 0U;
	ConnectionState->CurrentPacketStatsStartIndex[ConnectionMode] = static_cast<uint32>(ConnectionData.PacketStats.Num());

	//UE_LOGF(LogNetTrace, Log, "PacketEvent GameInstanceId: %u, ConnectionId: %u, %ls, Seq: %u PacketBits: %u", (uint32)GameInstanceId, (uint32)ConnectionId, ConnectionMode ? TEXT("Incoming") : TEXT("Outgoing"), SequenceNumber, Packet.ContentSizeInBits);
}

void FNetTraceAnalyzer::FlushFrameStatsCounters(FNetTraceAnalyzer::FNetTraceGameInstanceState& GameInstanceState)
{
	if (FNetProfilerGameInstanceInternal* GameInstance = NetProfilerProvider.EditGameInstance(GameInstanceState.GameInstanceIndex))
	{
		const bool bIsNewFrame = (GameInstance->Frames->Num() == 0) || (GameInstance->Frames->Last().EngineFrameNumber != GameInstanceState.CurrentEngineFrameIndex);

		if (bIsNewFrame)
		{
			FNetProfilerFrame& Frame = GameInstance->Frames->EmplaceBack();
			Frame.EngineFrameNumber = GameInstanceState.CurrentEngineFrameIndex;
			Frame.StartStatsIndex = static_cast<uint32>(GameInstance->FrameStats->Num());
			Frame.StatsCount = GameInstanceState.FrameStatsCounters.Num();
			Frame.TimeStamp = LastTimeStamp;
		}
		else
		{
			FNetProfilerFrame& Frame = GameInstance->Frames->Last();
			Frame.StatsCount += GameInstanceState.FrameStatsCounters.Num();
		}

		for (const FNetProfilerStats& Stat : GameInstanceState.FrameStatsCounters)
		{
			GameInstance->FrameStats->EmplaceBack(Stat);
		}

		GameInstanceState.FrameStatsCounters.Reset();
		GameInstanceState.CurrentNetProfilerFrameIndex = static_cast<uint32>(GameInstance->Frames->Num());

		// Mark frames dirty
		++GameInstance->FramesChangeCount;
	}
}

uint32 FNetTraceAnalyzer::GetCurrentNetProfilerFrameIndexAndFlushFrameStatsCountersIfNeeded(uint32 GameInstanceId, uint32 EngineFrameIndex)
{
	TSharedRef<FNetTraceAnalyzer::FNetTraceGameInstanceState> GameInstanceStateRef = GetOrCreateActiveGameInstanceState(GameInstanceId);
	FNetTraceAnalyzer::FNetTraceGameInstanceState& GameInstanceState = GameInstanceStateRef.Get();
	if (EngineFrameIndex > GameInstanceState.CurrentEngineFrameIndex)
	{
		FlushFrameStatsCounters(GameInstanceState);
		GameInstanceState.CurrentEngineFrameIndex = EngineFrameIndex;
	}

	return GameInstanceState.CurrentNetProfilerFrameIndex;
}

void FNetTraceAnalyzer::HandlePacketDroppedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint64 TimestampCycles = EventData.GetValue<uint64>("Timestamp");
	const uint32 SequenceNumber = EventData.GetValue<uint32>("SequenceNumber");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");
	const uint8 PacketType = EventData.GetValue<uint8>("PacketType");

	// Update LastTimestamp, later on we will be able to get timestamps piggybacked from other analyzers
	LastTimeStamp = Context.EventTime.AsSeconds(TimestampCycles);

	FNetTraceConnectionState* ConnectionState = GetActiveConnectionState(GameInstanceId, ConnectionId);
	if (!ConnectionState)
	{
		return;
	}

	FNetProfilerConnectionData& ConnectionData = NetProfilerProvider.EditConnectionData(ConnectionState->ConnectionIndex, ENetProfilerConnectionMode(PacketType));

	// Update packet delivery status
	NetProfilerProvider.EditPacketDeliveryStatus(ConnectionState->ConnectionIndex, ENetProfilerConnectionMode(PacketType), SequenceNumber, ENetProfilerDeliveryStatus::Dropped);
}

void FNetTraceAnalyzer::HandleConnectionCreatedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);

	//Sometimes it's possible for CreateConnection to be called multiple times for a ConnectionId if the capture was made using the TailBuffer (can cause multiple OnConnection calls)
	if (!GameInstanceState->ActiveConnections.Contains(ConnectionId))
	{
		// Add to both active connections and to persistent connections
		FNetProfilerConnectionInternal& Connection = NetProfilerProvider.CreateConnection(GameInstanceState->GameInstanceIndex);
		TSharedRef<FNetTraceConnectionState> ConnectionState = MakeShared<FNetTraceConnectionState>();
		GameInstanceState->ActiveConnections.Add(ConnectionId, ConnectionState);

		// Fill in Connection data
		Connection.Connection.ConnectionId = ConnectionId;
		Connection.Connection.LifeTime.Begin = GetLastTimestamp();
		ConnectionState->ConnectionIndex = Connection.Connection.ConnectionIndex;
		ConnectionState->CurrentPacketStartIndex[ENetProfilerConnectionMode::Outgoing] = 0U;
		ConnectionState->CurrentPacketStartIndex[ENetProfilerConnectionMode::Incoming] = 0U;

		ConnectionState->CurrentPacketBitOffset[ENetProfilerConnectionMode::Outgoing] = 0U;
		ConnectionState->CurrentPacketBitOffset[ENetProfilerConnectionMode::Incoming] = 0U;

		ConnectionState->CurrentPacketStatsStartIndex[ENetProfilerConnectionMode::Outgoing] = 0U;
		ConnectionState->CurrentPacketStatsStartIndex[ENetProfilerConnectionMode::Incoming] = 0U;
	}
}

uint32 FNetTraceAnalyzer::GetOrCreateNetProfilerStatsCounterTypeIndex(FNetProfilerNameIndexType NameId, ENetProfilerStatsCounterType StatsType)
{
	if (const uint32* ExistingNetProfilerStatsCounterTypeIndex = TraceNetStatsCounterIdToNetProfilerStatsCounterTypeIndexMap.Find(NameId))
	{
		return *ExistingNetProfilerStatsCounterTypeIndex;
	}
	else
	{
		// Add new counter type
		const FNetProfilerNameIndexType* NetProfilerNameIndex = TracedNameIdToNetProfilerNameIdMap.Find(NameId);
		const FNetProfilerNameIndexType NameIndex = NetProfilerNameIndex ? *NetProfilerNameIndex : 0u;

		uint32 NetProfilerStatsCounterTypeIndex = NetProfilerProvider.AddNetProfilerStatsCounterType(NameIndex, StatsType);
		TraceNetStatsCounterIdToNetProfilerStatsCounterTypeIndexMap.Add(NameId, NetProfilerStatsCounterTypeIndex);

		return NetProfilerStatsCounterTypeIndex;
	}
}

void FNetTraceAnalyzer::HandlePacketStatsCounterEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint32 StatsCounterValue = EventData.GetValue<uint32>("StatsValue");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");
	const FNetProfilerNameIndexType NameId = EventData.GetValue<FNetProfilerNameIndexType>("NameId");

	FNetProfilerStats Stats;
	Stats.StatsCounterTypeIndex = GetOrCreateNetProfilerStatsCounterTypeIndex(NameId, ENetProfilerStatsCounterType::Packet);
	Stats.StatsValue = StatsCounterValue;

	// Accumulate stats for current packet
	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);
	if (TSharedRef<FNetTraceConnectionState>* ConnectionState = GameInstanceState->ActiveConnections.Find(ConnectionId))
	{
		(*ConnectionState)->PacketStats.Emplace(Stats);
	}
}

void FNetTraceAnalyzer::HandleFrameStatsCounterEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint64 TimestampCycles = EventData.GetValue<uint64>("Timestamp");
	const uint32 StatsCounterValue = EventData.GetValue<uint32>("StatsValue");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const FNetProfilerNameIndexType NameId = EventData.GetValue<FNetProfilerNameIndexType>("NameId");

	// Update LastTimestamp, later on we will be able to get timestamps piggybacked from other analyzers
	LastTimeStamp = Context.EventTime.AsSeconds(TimestampCycles);

	// Get the NetProfilerFrameIndex for the current engine frame/timestamp
	const uint32 NetProfilerFrameIndex = GetCurrentNetProfilerFrameIndexAndFlushFrameStatsCountersIfNeeded(
		GameInstanceId,
		FrameProvider.GetFrameNumberForTimestamp(ETraceFrameType::TraceFrameType_Game, LastTimeStamp)
	);

	FNetProfilerStats Stats;
	Stats.StatsCounterTypeIndex = GetOrCreateNetProfilerStatsCounterTypeIndex(NameId, ENetProfilerStatsCounterType::Frame);
	Stats.StatsValue = StatsCounterValue;

	// Accumulate stats for current frame
	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);
	GameInstanceState->FrameStatsCounters.Emplace(Stats);
}

void FNetTraceAnalyzer::HandleConnectionStateUpdatedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");
	const uint8 ConnectionStateValue = EventData.GetValue<uint8>("ConnectionStateValue");

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);

	if (TSharedRef<FNetTraceConnectionState>* ConnectionState = GameInstanceState->ActiveConnections.Find(ConnectionId))
	{
		(*ConnectionState)->ConnectionState = ENetProfilerConnectionState(ConnectionStateValue);
	}
}

void FNetTraceAnalyzer::HandleConnectionUpdatedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");
	FString Name;
	EventData.GetString("Name", Name);
	FString AddressString;
	EventData.GetString("Address", AddressString);

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);

	if (TSharedRef<FNetTraceConnectionState>* ConnectionState = GameInstanceState->ActiveConnections.Find(ConnectionId))
	{
		if (FNetProfilerConnectionInternal* Connection = NetProfilerProvider.EditConnection((*ConnectionState)->ConnectionIndex))
		{
			Connection->Connection.Name = Session.StoreString(Name);
			Connection->Connection.AddressString = Session.StoreString(AddressString);
		}
	}
	else
	{
		// Incomplete trace?  Ignore?
		UE_LOGF(LogNetTrace, Warning, "Connection %d is missing", ConnectionId);
	}

}

void FNetTraceAnalyzer::HandleConnectionClosedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint16 ConnectionId = EventData.GetValue<uint16>("ConnectionId");

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);

	if (TSharedRef<FNetTraceConnectionState>* ConnectionState = GameInstanceState->ActiveConnections.Find(ConnectionId))
	{
		if (FNetProfilerConnectionInternal* Connection = NetProfilerProvider.EditConnection((*ConnectionState)->ConnectionIndex))
		{
			// Update connection state
			Connection->Connection.LifeTime.End =  GetLastTimestamp();
		}
		GameInstanceState->ActiveConnections.Remove(ConnectionId);
	}
	else
	{
		// Incomplete trace?  Ignore?
		UE_LOGF(LogNetTrace, Warning, "Connection %d is missing", ConnectionId);
	}
}

void FNetTraceAnalyzer::HandleObjectCreatedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint64 TypeId = EventData.GetValue<uint64>("TypeId");
	const uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
	//const uint32 OwnerId = EventData.GetValue<uint32>("OwnerId");
	const FNetProfilerNameIndexType NameId = EventData.GetValue<FNetProfilerNameIndexType>("NameId");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);
	const FNetProfilerNameIndexType* NetProfilerNameIndex = TracedNameIdToNetProfilerNameIdMap.Find(NameId);
	const FNetProfilerNameIndexType NameIndex = NetProfilerNameIndex ? *NetProfilerNameIndex : 0u;

	if (FNetTraceActiveObjectState* ActiveObjectInstance = GameInstanceState->ActiveObjects.Find(ObjectId))
	{
		if (FNetProfilerObjectInstance* ExistingInstance = NetProfilerProvider.EditObject(GameInstanceState->GameInstanceIndex, ActiveObjectInstance->ObjectIndex))
		{
			if (ExistingInstance->NameIndex == NameIndex || ExistingInstance->NameIndex == PendingNameIndex)
			{
				// Update existing object instance
				ExistingInstance->LifeTime.Begin = GetLastTimestamp();
				ExistingInstance->NetObjectId = ObjectId;
				ExistingInstance->TypeId = TypeId;

				// Update name in both the persistent instance and the active one
				ExistingInstance->NameIndex = NameIndex;
				ActiveObjectInstance->NameIndex = NameIndex;

				return;
			}

			// End instance and remove it from ActiveObjects
			ExistingInstance->LifeTime.End = GetLastTimestamp();
			GameInstanceState->ActiveObjects.Remove(ObjectId);
		}
	}

	// Add persistent object representation
	FNetProfilerObjectInstance& ObjectInstance = NetProfilerProvider.CreateObject(GameInstanceState->GameInstanceIndex);

	// Fill in object data
	ObjectInstance.LifeTime.Begin = GetLastTimestamp();
	ObjectInstance.NameIndex = NameIndex;
	ObjectInstance.NetObjectId = ObjectId;
	ObjectInstance.TypeId = TypeId;

	// Add to active objects
	GameInstanceState->ActiveObjects.Add(ObjectId, { ObjectInstance.ObjectIndex, ObjectInstance.NameIndex });
}

void FNetTraceAnalyzer::HandleObjectExistsEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint64 TypeId = EventData.GetValue<uint64>("TypeId");
	const uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
	const FNetProfilerNameIndexType NameId = EventData.GetValue<FNetProfilerNameIndexType>("NameId");
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);
	const FNetProfilerNameIndexType* NetProfilerNameIndex = TracedNameIdToNetProfilerNameIdMap.Find(NameId);
	const FNetProfilerNameIndexType NameIndex = NetProfilerNameIndex ? *NetProfilerNameIndex : 0u;

	if (FNetTraceActiveObjectState* ActiveObjectInstance = GameInstanceState->ActiveObjects.Find(ObjectId))
	{
		if (FNetProfilerObjectInstance* ExistingInstance = NetProfilerProvider.EditObject(GameInstanceState->GameInstanceIndex, ActiveObjectInstance->ObjectIndex))
		{
			if (ExistingInstance->NameIndex == NameIndex || ExistingInstance->NameIndex == PendingNameIndex)
			{
				// Update existing object instance
				ExistingInstance->LifeTime.Begin = StartTimeStamp;
				ExistingInstance->NetObjectId = ObjectId;
				ExistingInstance->TypeId = TypeId;

				// Update name in both the persistent instance and the active one
				ExistingInstance->NameIndex = NameIndex;
				ActiveObjectInstance->NameIndex = NameIndex;

				return;
			}

			// End instance and remove it from ActiveObjects
			ExistingInstance->LifeTime.End = GetLastTimestamp();
			GameInstanceState->ActiveObjects.Remove(ObjectId);
		}
	}

	// Add persistent object representation
	FNetProfilerObjectInstance& ObjectInstance = NetProfilerProvider.CreateObject(GameInstanceState->GameInstanceIndex);

	// Fill in object data
	ObjectInstance.LifeTime.Begin = StartTimeStamp;
	ObjectInstance.NameIndex = NameIndex;
	ObjectInstance.NetObjectId = ObjectId;
	ObjectInstance.TypeId = TypeId;

	// Add to active objects
	GameInstanceState->ActiveObjects.Add(ObjectId, { ObjectInstance.ObjectIndex, ObjectInstance.NameIndex });
}

void FNetTraceAnalyzer::HandleObjectDestroyedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	// Remove from active instances and mark the end timestamp in the persistent instance list
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);

	FNetTraceActiveObjectState DestroyedObjectState;
	if (GameInstanceState->ActiveObjects.RemoveAndCopyValue(ObjectId, DestroyedObjectState))
	{
		if (FNetProfilerObjectInstance* ObjectInstance = NetProfilerProvider.EditObject(GameInstanceState->GameInstanceIndex, DestroyedObjectState.ObjectIndex))
		{
			// Update object data
			ObjectInstance->LifeTime.End = GetLastTimestamp();
		}
	}
}

TSharedRef<FNetTraceAnalyzer::FNetTraceGameInstanceState> FNetTraceAnalyzer::GetOrCreateActiveGameInstanceState(uint32 GameInstanceId)
{
	if (TSharedRef<FNetTraceAnalyzer::FNetTraceGameInstanceState>* FoundState = ActiveGameInstances.Find(GameInstanceId))
	{
		return *FoundState;
	}
	else
	{
		// Persistent GameInstance
		FNetProfilerGameInstanceInternal& GameInstance = NetProfilerProvider.CreateGameInstance();
		GameInstance.Instance.GameInstanceId = GameInstanceId;
		GameInstance.Instance.LifeTime.Begin = GetLastTimestamp();

		// Active GameInstanceState
		TSharedRef<FNetTraceGameInstanceState> GameInstanceState = MakeShared<FNetTraceGameInstanceState>();
		ActiveGameInstances.Add(GameInstanceId, GameInstanceState);
		GameInstanceState->GameInstanceIndex = GameInstance.Instance.GameInstanceIndex;

		const uint32 FrameCount = static_cast<uint32>(FrameProvider.GetFrameCount(TraceFrameType_Game));
		GameInstanceState->CurrentEngineFrameIndex = (FrameCount > 0) ? FrameCount - 1 : 0;

		return GameInstanceState;
	}
}

void FNetTraceAnalyzer::DestroyActiveGameInstanceState(uint32 GameInstanceId)
{
	if (TSharedRef<FNetTraceAnalyzer::FNetTraceGameInstanceState>* FoundState = ActiveGameInstances.Find(GameInstanceId))
	{
		// Mark as closed
		if (FNetProfilerGameInstanceInternal* GameInstance = NetProfilerProvider.EditGameInstance((*FoundState)->GameInstanceIndex))
		{
			GameInstance->Instance.LifeTime.End = GetLastTimestamp();
			NetProfilerProvider.MarkGameInstancesDirty();
		}
		ActiveGameInstances.Remove(GameInstanceId);
	}
}

FNetTraceAnalyzer::FNetTraceConnectionState* FNetTraceAnalyzer::GetActiveConnectionState(uint32 GameInstanceId, uint32 ConnectionId)
{
	if (TSharedRef<FNetTraceAnalyzer::FNetTraceGameInstanceState>* FoundState = ActiveGameInstances.Find(GameInstanceId))
	{
		if (TSharedRef<FNetTraceConnectionState>* ConnectionState =  (*FoundState)->ActiveConnections.Find(ConnectionId))
		{
			return &(*ConnectionState).Get();
		}
	}

	return nullptr;
}

void FNetTraceAnalyzer::HandleGameInstanceUpdatedEvent(const FOnEventContext& Context, const FEventData& EventData)
{
	const uint8 GameInstanceId = EventData.GetValue<uint8>("GameInstanceId");
	const bool bIsServer = EventData.GetValue<bool>("bIsServer");
	FString InstanceName;
	EventData.GetString("Name", InstanceName);

	TSharedRef<FNetTraceGameInstanceState> GameInstanceState = GetOrCreateActiveGameInstanceState(GameInstanceId);

	if (FNetProfilerGameInstanceInternal* InternalGameInstance = NetProfilerProvider.EditGameInstance(GameInstanceState->GameInstanceIndex))
	{
		InternalGameInstance->Instance.bIsServer = bIsServer;
		InternalGameInstance->Instance.InstanceName = Session.StoreString(InstanceName);
		NetProfilerProvider.MarkGameInstancesDirty();
	}
}

} // namespace TraceServices
