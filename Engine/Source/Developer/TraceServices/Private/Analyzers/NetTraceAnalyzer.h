// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#include "Model/NetProfilerProvider.h"
#include "Trace/Analyzer.h"

namespace TraceServices
{

class IAnalysisSession;
class IFrameProvider;

class FNetTraceAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FNetTraceAnalyzer(IAnalysisSession& Session, FNetProfilerProvider& NetProfilerProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	virtual void OnAnalysisEnd() override;

private:

	enum : uint16
	{
		RouteId_InitEvent,
		RouteId_InstanceDestroyedEvent,
		RouteId_NameEvent,
		RouteId_PacketContentEvent,
		RouteId_PacketEvent,
		RouteId_PacketDroppedEvent,
		RouteId_ConnectionCreatedEvent,
		RouteId_ConnectionUpdatedEvent,
		RouteId_ConnectionClosedEvent,
		RouteId_PacketStatsCounterEvent,
		RouteId_FrameStatsCounterEvent,
		RouteId_ObjectCreatedEvent,
		RouteId_ObjectExistsEvent,
		RouteId_ObjectDestroyedEvent,
		RouteId_ConnectionStateUpdatedEvent,
		RouteId_InstanceUpdatedEvent,
	};

	// This must be kept in sync with Event types in NetTrace.h
	enum class EContentEventType : uint8
	{
		Object = 0,
		NameId = 1,
		BunchEvent = 2,
		BunchHeaderEvent = 3,
	};

	IAnalysisSession& Session;
	FNetProfilerProvider& NetProfilerProvider;
	const IFrameProvider& FrameProvider;
	uint32 NetTraceVersion;
	uint32 NetTraceReporterVersion;

	// Default names
	FNetProfilerNameIndexType BunchHeaderNameIndex;
	FNetProfilerNameIndexType PendingNameIndex;

	// Shared for trace
	TMap<FNetProfilerNameIndexType, FNetProfilerNameIndexType> TracedNameIdToNetProfilerNameIdMap;

	TMap<uint64, uint32> TraceEventTypeToNetProfilerEventTypeIndexMap;

	TMap<FNetProfilerNameIndexType, uint32> TraceNetStatsCounterIdToNetProfilerStatsCounterTypeIndexMap;

	struct FBunchInfo
	{
		FNetProfilerBunchInfo BunchInfo;
		uint32 BunchBits;
		uint32 HeaderBits;
		int32 FirstBunchEventIndex;
		uint32 EventCount;
		FNetProfilerNameIndexType NameIndex;
	};

	struct FNetTraceConnectionState
	{
		// Index into persistent connection array
		uint32 ConnectionIndex;

		// Current packet data
		uint32 CurrentPacketStartIndex[ENetProfilerConnectionMode::Count];
		uint32 CurrentPacketBitOffset[ENetProfilerConnectionMode::Count];
		uint32 CurrentPacketStatsStartIndex[ENetProfilerConnectionMode::Count];

		// StatsCounters reported during the current packet
		TArray<FNetProfilerStats> PacketStats;

		// Current bunch data
		TArray<FNetProfilerContentEvent> BunchEvents[ENetProfilerConnectionMode::Count];

		TArray<FBunchInfo> BunchInfos[ENetProfilerConnectionMode::Count];

		ENetProfilerConnectionState ConnectionState;
	};

	struct FNetTraceActiveObjectState
	{
		uint32 ObjectIndex;
		FNetProfilerNameIndexType NameIndex;
	};

	// Map from instance id to index in the persistent instance array
	struct FNetTraceGameInstanceState
	{
		TMap<uint32, TSharedRef<FNetTraceConnectionState>> ActiveConnections;
		TMap<uint64, FNetTraceActiveObjectState> ActiveObjects;
		TMap<int32, FNetProfilerNameIndexType> ChannelNames;

		// StatsCounters reported during the current frame
		TArray<FNetProfilerStats> FrameStatsCounters;
		uint32 CurrentEngineFrameIndex = 0;
		uint32 CurrentNetProfilerFrameIndex = 0;

		uint32 GameInstanceIndex;
	};

private:

	uint32 GetTracedEventTypeIndex(FNetProfilerNameIndexType NameIndex, uint8 Level);
	uint32 GetOrCreateNetProfilerStatsCounterTypeIndex(FNetProfilerNameIndexType NameId, ENetProfilerStatsCounterType StatsType);

	TSharedRef<FNetTraceGameInstanceState> GetOrCreateActiveGameInstanceState(uint32 GameInstanceId);
	void DestroyActiveGameInstanceState(uint32 GameInstanceId);

	FNetTraceConnectionState* GetActiveConnectionState(uint32 GameInstanceId, uint32 ConnectionId);
	FNetProfilerTimeStamp GetLastTimestamp() const { return LastTimeStamp; }

	void FlushPacketEvents(FNetTraceConnectionState& ConnectionState, FNetProfilerConnectionData& ConnectionData, const ENetProfilerConnectionMode ConnectionMode);

	void FlushFrameStatsCounters(FNetTraceGameInstanceState& GameInstanceState);
	uint32 GetCurrentNetProfilerFrameIndexAndFlushFrameStatsCountersIfNeeded(uint32 GameInstanceId, uint32 EngineFrameIndex);

	void HandlePacketEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandlePacketContentEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandlePacketDroppedEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleConnectionStateUpdatedEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleConnectionCreatedEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleConnectionUpdatedEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleConnectionClosedEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleFrameStatsCounterEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandlePacketStatsCounterEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleObjectCreatedEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleObjectExistsEvent(const FOnEventContext& Context, const FEventData& EventData);
	void HandleObjectDestroyedEvent(const FOnEventContext& Context, const FEventData& EventData);

	void HandleGameInstanceUpdatedEvent(const FOnEventContext& Context, const FEventData& EventData);

	void AddEvent(TPagedArray<FNetProfilerContentEvent>& Events, const FNetProfilerContentEvent& Event, uint32 Offset, uint32 LevelOffset);
	void AddEvent(TPagedArray<FNetProfilerContentEvent>& Events, uint32 StartPos, uint32 EndPos, uint32 Level, FNetProfilerNameIndexType NameIndex, FNetProfilerBunchInfo Info);

private:

	TMap<uint32, TSharedRef<FNetTraceGameInstanceState>> ActiveGameInstances;
	FNetProfilerTimeStamp StartTimeStamp;
	FNetProfilerTimeStamp LastTimeStamp;
};

} // namespace TraceServices
