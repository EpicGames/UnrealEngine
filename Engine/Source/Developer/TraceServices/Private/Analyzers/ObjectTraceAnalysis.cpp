// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTraceAnalysis.h"

#include "Containers/StringConv.h"
#include "HAL/LowLevelMemTracker.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "Common/Utils.h"
#include "Model/ObjectProviderPrivate.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Strings.h"
#include "TraceServices/Model/ObjectProvider.h"

#define UE_OBJECT_ANALYZER_EVENT(_EventName) const ANSICHAR* EventName = _EventName;
#define UE_OBJECT_ANALYZER_LOG_API_L0(Format, ...) UE_LOGF(LogTraceServices, Log, "[Obj] Tid=%u %s " Format, Context.ThreadInfo.GetId(), EventName, ##__VA_ARGS__);
#define UE_OBJECT_ANALYZER_LOG_API_L1(Format, ...) //UE_LOGF(LogTraceServices, Log, "[Obj] Tid=%u %s " Format, Context.ThreadInfo.GetId(), EventName, ##__VA_ARGS__);

namespace TraceServices
{

FObjectAnalyzer::FObjectAnalyzer(IAnalysisSession& InSession, IEditableObjectProvider& InEditableObjectProvider)
	: Session(InSession)
	, EditableProvider(InEditableObjectProvider)
{
}

void FObjectAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

#define ROUTE_EVENT(Event) Builder.RouteEvent(RouteId_##Event, "CoreUObject", #Event)
	ROUTE_EVENT(BeginSnapshot);
	ROUTE_EVENT(EndSnapshot);
	ROUTE_EVENT(ObjectSpec);
	ROUTE_EVENT(VerseExtraSpec);
	ROUTE_EVENT(ResourceSizeExtraSpec);
	ROUTE_EVENT(TotalResourceSizeExtraSpec);
	ROUTE_EVENT(FieldExtraSpec);
	ROUTE_EVENT(StructExtraSpec);
	ROUTE_EVENT(ClassExtraSpec);
	ROUTE_EVENT(FunctionExtraSpec);
	ROUTE_EVENT(PackageExtraSpec);
	ROUTE_EVENT(ObjectRef);
#undef ROUTE_EVENT
}

void FObjectAnalyzer::OnAnalysisEnd()
{
}

bool FObjectAnalyzer::CheckValidSnapshotForEvent(const ANSICHAR* EventName)
{
	if (CurrentSnapshotId == InvalidSnapshotId)
	{
		if (++NumWarnings < MaxWarningMessages)
		{
			UE_LOGF(LogTraceServices, Warning, "[Obj] Ignoring %s event received outside a Begin-End Snapshot scope.", EventName);
		}
		return false;
	}
	return true;
}

void FObjectAnalyzer::LogErrorForEventWithInvalidObjectId(const ANSICHAR* EventName, uint32 Id)
{
	if (++NumErrors < MaxErrorMessages)
	{
		UE_LOGF(LogTraceServices, Error, "[Obj] %s: Failed to edit object with Id=%u", EventName, Id);
	}
}

static const TCHAR* ReadString(const UE::Trace::IAnalyzer::FEventData& EventData, const IDefinitionProvider* DefinitionProvider, const ANSICHAR* FieldName)
{
	const int32 FieldIndex = EventData.GetTypeInfo().GetFieldIndex(FieldName);
	if (FieldIndex >= 0)
	{
		UE::Trace::FEventRef32 StringRef = EventData.GetReferenceValue<uint32>(FieldIndex);
		const FStringDefinition* StringDef = DefinitionProvider->Get<FStringDefinition>(StringRef);
		if (StringDef)
		{
			return StringDef->Display;
		}
	}
	return nullptr;
}

bool FObjectAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/ObjectAnalyzer"));

	const auto& EventData = Context.EventData;

	switch (RouteId)
	{

	case RouteId_BeginSnapshot:
	{
		UE_OBJECT_ANALYZER_EVENT("BeginSnapshot");

		if (CurrentSnapshotId != InvalidSnapshotId)
		{
			++NumWarnings;
			UE_LOGF(LogTraceServices, Warning, "[Obj] Ignoring duplicate BeginSnapshot event.");
			break;
		}

		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double Time = Context.EventTime.AsSeconds(Cycle);

		UE_OBJECT_ANALYZER_LOG_API_L0("Time=%f", Time);

		if (Time < SnapshotLastTime)
		{
			++NumWarnings;
			UE_LOGF(LogTraceServices, Warning, "[Obj] Invalid timestamp for BeginSnapshot event (%f < %f; delta=%f)!", Time, SnapshotLastTime, SnapshotLastTime - Time);
		}
		SnapshotLastTime = Time;

		uint32 ObjectArrayNum = EventData.GetValue<uint32>("ObjectArrayNum");
		uint32 ObjectArrayNumMinusAvailable = EventData.GetValue<uint32>("ObjectArrayNumMinusAvailable");
		uint32 ObjectArrayNumPermanent = EventData.GetValue<uint32>("ObjectArrayNumPermanent");
		{
			FProviderEditScopeLock _(EditableProvider);
			IObjectEditableSnapshot* Snapshot = EditableProvider.BeginSnapshot(Time);
			if (ensure(Snapshot))
			{
				CurrentSnapshotId = Snapshot->GetId();
				Snapshot->SetTracedObjectArrayNum(ObjectArrayNum);
				Snapshot->SetTracedObjectArrayNumMinusAvailable(ObjectArrayNumMinusAvailable);
				Snapshot->SetTracedObjectArrayNumPermanent(ObjectArrayNumPermanent);
			}
		}

		const ANSICHAR* KnownClasses[] =
		{
			"UObject", "UField", "UStruct", "UClass", "UFunction", "UPackage",
		};
		constexpr int32 NumKnownClasses = UE_ARRAY_COUNT(KnownClasses);
		for (int32 EventIndex = 0; EventIndex < NumKnownClasses; ++EventIndex)
		{
			ANSICHAR FieldName[32];
			FCStringAnsi::Strcpy(FieldName, "SizeOf");
			FCStringAnsi::Strcat(FieldName, KnownClasses[EventIndex]);
			const uint32 SizeOf = EventData.GetValue<uint32>(FieldName);
			UE_LOGF(LogTraceServices, Log, "[Obj] sizeof(%s)=%u", KnownClasses[EventIndex], SizeOf);
		}

		break;
	}

	case RouteId_EndSnapshot:
	{
		UE_OBJECT_ANALYZER_EVENT("EndSnapshot");

		if (CurrentSnapshotId == InvalidSnapshotId)
		{
			++NumWarnings;
			UE_LOGF(LogTraceServices, Warning, "[Obj] Ignoring EndSnapshot event received without a matching BeginSnapshot event.");
			break;
		}

		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double Time = Context.EventTime.AsSeconds(Cycle);

		UE_OBJECT_ANALYZER_LOG_API_L0("Time=%f (Duration=%f)", Time, Time - SnapshotLastTime);

		if (Time < SnapshotLastTime)
		{
			++NumWarnings;
			UE_LOGF(LogTraceServices, Warning, "[Obj] Invalid timestamp for EndSnapshot event (%f < %f; delta=%f)!", Time, SnapshotLastTime, SnapshotLastTime - Time);
		}
		SnapshotLastTime = Time;

		{
			FProviderEditScopeLock _(EditableProvider);
			IObjectEditableSnapshot* Snapshot = EditableProvider.EndSnapshot(Time);
			if (Snapshot)
			{
				UE_LOGF(LogTraceServices, Log, "[Obj] Completed snapshot %u (%u / %u objects, %u refs).", Snapshot->GetId(), Snapshot->GetObjectCount(), Snapshot->GetObjectArrayNum(), Snapshot->GetNumReferences());
				check(Snapshot->GetId() == CurrentSnapshotId);

				// debug
				//Snapshot->SaveAs(*FString::Printf(TEXT("D:/uobjects_%u.csv"), Snapshot->GetId()));
			}
			else
			{
				++NumWarnings;
				UE_LOGF(LogTraceServices, Warning, "[Obj] Completed empty snapshot!");
			}
		}
		CurrentSnapshotId = InvalidSnapshotId;
		break;
	}

	case RouteId_ObjectSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("ObjectSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");
		uint32 ClassId = EventData.GetValue<uint32>("ClassId");
		uint32 OuterId = EventData.GetValue<uint32>("OuterId");
		uint32 Flags = EventData.GetValue<uint32>("Flags");

		//////////////////////////////////////////////////
		// Read Object's Name

		const TCHAR* PersistentName = nullptr;
#if 0
		const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(Session);
		if (DefinitionProvider)
		{
			TraceServices::FProviderReadScopeLock DefinitionProviderReadLock(*DefinitionProvider);
			PersistentName = ReadString(EventData, DefinitionProvider, "Name");
		}
#endif
		if (!PersistentName)
		{
			FAnsiStringView NameAnsi;
			if (EventData.GetString("NameAnsi", NameAnsi) &&
				!NameAnsi.IsEmpty())
			{
				FAnalysisSessionEditScope _(Session);
				PersistentName = Session.StoreString(FString(NameAnsi));
			}
		}
		if (!PersistentName)
		{
			FWideStringView NameWide;
			if (EventData.GetString("NameWide", NameWide) &&
				!NameWide.IsEmpty())
			{
				FAnalysisSessionEditScope _(Session);
				PersistentName = Session.StoreString(NameWide);
			}
		}
		if (!PersistentName)
		{
			FAnalysisSessionEditScope _(Session);
			PersistentName = Session.StoreString(TEXTVIEW("<noname>"));
		}
		check(PersistentName != nullptr);

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, ClassId=%u, OuterId=%u, Flags=%u, Name=\"%ls\"", Id, ClassId, OuterId, Flags, PersistentName);

		//////////////////////////////////////////////////
		// Temporary backward compatibility

		uint32 PackageId = EventData.GetValue<uint32>("PackageId");
		int32 StructureSize = EventData.GetValue<int32>("StructureSize");
		uint64 SystemMemoryBytes = EventData.GetValue<uint64>("SystemMemoryBytes");
		uint64 VideoMemoryBytes = EventData.GetValue<uint64>("VideoMemoryBytes");

		const TCHAR* PersistentVersePath = nullptr;
		{
			FStringView VersePath;
			if (EventData.GetString("VersePath", VersePath) &&
				!VersePath.IsEmpty())
			{
				FAnalysisSessionEditScope _(Session);
				PersistentVersePath = Session.StoreString(VersePath);
			}
		}

		//////////////////////////////////////////////////

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.AddObject(Id, ClassId, OuterId, Flags, PersistentName);
		if (Object)
		{
			// Temporary backward compatibility
			Object->PackageId = uint64(PackageId);
			Object->StructureSize = StructureSize;
			Object->SystemMemoryBytes = SystemMemoryBytes;
			Object->VideoMemoryBytes = VideoMemoryBytes;
			Object->VersePath = PersistentVersePath ? PersistentVersePath : TEXT("");

			if (StructureSize != 0)
			{
				Object->FlagsEx |= EObjectInfoFlags::IsStruct | EObjectInfoFlags::IsField;
			}
		}
		else
		{
			if (++NumErrors < MaxErrorMessages)
			{
				UE_LOGF(LogTraceServices, Error, "[Obj] Failed to add object with Id=%u (Name=\"%ls\")!", Id, PersistentName);
			}
		}
		break;
	}

	case RouteId_VerseExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("VerseExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");

		const TCHAR* PersistentVersePath = nullptr;
		{
			FStringView VersePath;
			if (EventData.GetString("Path", VersePath) && !VersePath.IsEmpty())
			{
				FAnalysisSessionEditScope _(Session);
				PersistentVersePath = Session.StoreString(VersePath);
			}
		}

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, Path=\"%ls\"", Id, PersistentVersePath ? PersistentVersePath : TEXT(""));

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->VersePath = PersistentVersePath;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_ResourceSizeExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("ResourceSizeExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");
		uint64 SystemMemoryBytes = EventData.GetValue<uint64>("SystemMemoryBytes");
		uint64 VideoMemoryBytes = EventData.GetValue<uint64>("VideoMemoryBytes");

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, SystemMemoryBytes=%llu, VideoMemoryBytes=%llu", Id, SystemMemoryBytes, VideoMemoryBytes);

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->SystemMemoryBytes = SystemMemoryBytes;
			Object->VideoMemoryBytes = VideoMemoryBytes;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_TotalResourceSizeExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("TotalResourceSizeExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");
		uint64 SystemMemoryBytes = EventData.GetValue<uint64>("SystemMemoryBytes");
		uint64 VideoMemoryBytes = EventData.GetValue<uint64>("VideoMemoryBytes");

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, SystemMemoryBytes=%llu, VideoMemoryBytes=%llu", Id, SystemMemoryBytes, VideoMemoryBytes);

		if (SystemMemoryBytes > 0 || VideoMemoryBytes > 0)
		{
			FProviderEditScopeLock _(EditableProvider);
			FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
			if (Object)
			{
				IObjectEditableSnapshot* Snapshot = EditableProvider.GetCurrentSnapshot();
				if (ensure(Snapshot))
				{
					Snapshot->EnableTotalMemorySizes();
				}

				Object->TotalSystemMemoryBytes = SystemMemoryBytes;
				Object->TotalVideoMemoryBytes = VideoMemoryBytes;
			}
			else
			{
				LogErrorForEventWithInvalidObjectId(EventName, Id);
			}
		}
		break;
	}

	case RouteId_FieldExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("FieldExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u", Id);

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->FlagsEx |= EObjectInfoFlags::IsField;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_StructExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("StructExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");
		uint32 SuperId = EventData.GetValue<uint32>("SuperId");
		uint32 InheritanceSuperId = EventData.GetValue<uint32>("InheritanceSuperId");
		int32 StructureSize = EventData.GetValue<int32>("StructureSize");

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, SuperId=%u, InheritanceSuperId=%u, StructureSize=%d", Id, SuperId, InheritanceSuperId, StructureSize);

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->FlagsEx |= EObjectInfoFlags::IsStruct;
			Object->SuperId = SuperId;
			Object->InheritanceSuperId = InheritanceSuperId;
			Object->StructureSize = StructureSize;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_ClassExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("ClassExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u", Id);

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->FlagsEx |= EObjectInfoFlags::IsClass;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_FunctionExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("FunctionExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");
		uint32 FunctionFlags = EventData.GetValue<uint32>("Flags");
		uint8 FunctionNumParms = EventData.GetValue<uint8>("NumParms");
		uint16 FunctionParmsSize = EventData.GetValue<uint16>("ParmsSize");

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, NumParms=%u, ParmsSize=%u, Flags=%s",
			Id, uint32(FunctionNumParms), uint32(FunctionParmsSize), *FunctionFlagsToUtf8String(FunctionFlags));

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->FlagsEx |= EObjectInfoFlags::IsFunction;
			Object->FunctionFlags = FunctionFlags;
			Object->FunctionNumParms = FunctionNumParms;
			Object->FunctionParmsSize = FunctionParmsSize;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_PackageExtraSpec:
	{
		UE_OBJECT_ANALYZER_EVENT("PackageExtraSpec");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 Id = EventData.GetValue<uint32>("Id");
		uint64 PackageId = EventData.GetValue<uint64>("PackageId");

		const TCHAR* PersistentPath = nullptr;
		{
			FStringView Path;
			if (EventData.GetString("LocalFullPath", Path) && !Path.IsEmpty())
			{
				FAnalysisSessionEditScope _(Session);
				PersistentPath = Session.StoreString(Path);
			}
		}

		const TCHAR* PersistentSourcePackageName = nullptr;
		{
			FStringView SourceName;
			if (EventData.GetString("SourcePackageName", SourceName) && !SourceName.IsEmpty())
			{
				FAnalysisSessionEditScope _(Session);
				PersistentSourcePackageName = Session.StoreString(SourceName);
			}
		}

		UE_OBJECT_ANALYZER_LOG_API_L1("Id=%u, PackageId=%llu, LocalFullPath=\"%ls\", SourcePackageName=\"%ls\"",
			Id, PackageId, PersistentPath ? PersistentPath : TEXT(""), PersistentSourcePackageName ? PersistentSourcePackageName : TEXT(""));

		FProviderEditScopeLock _(EditableProvider);
		FObjectInfo* Object = EditableProvider.GetEditableObject(Id);
		if (Object)
		{
			Object->FlagsEx |= EObjectInfoFlags::IsPackage;
			Object->PackageId = PackageId;
			Object->PackagePath = PersistentPath;
			Object->SourcePackageName = PersistentSourcePackageName;
		}
		else
		{
			LogErrorForEventWithInvalidObjectId(EventName, Id);
		}
		break;
	}

	case RouteId_ObjectRef:
	{
		UE_OBJECT_ANALYZER_EVENT("ObjectRef");
		if (!CheckValidSnapshotForEvent(EventName))
		{
			break;
		}

		uint32 ReferencerId = EventData.GetValue<uint32>("ReferencerId");
		uint32 ObjectId = EventData.GetValue<uint32>("ObjectId");

		UE_OBJECT_ANALYZER_LOG_API_L1("ReferencerId=%u, ObjectId=%u", ReferencerId, ObjectId);

		FProviderEditScopeLock _(EditableProvider);
		FObjectReferenceInfo* Reference = EditableProvider.AddObjectReference(ReferencerId, ObjectId);
		if (!Reference)
		{
			if (++NumErrors < MaxErrorMessages)
			{
				UE_LOGF(LogTraceServices, Error, "[Obj] Failed to add reference (ReferencerId=%u, ObjectId=%u)!", ReferencerId, ObjectId);
			}
		}
		break;
	}

	} // switch (RouteId)

	return true;
}

} // namespace TraceServices

#undef UE_OBJECT_ANALYZER_LOG_API_L1
#undef UE_OBJECT_ANALYZER_LOG_API_L0
#undef UE_OBJECT_ANALYZER_EVENT
