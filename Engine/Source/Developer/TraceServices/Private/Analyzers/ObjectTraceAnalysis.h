// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Trace/Analyzer.h"
#include "TraceServices/Model/ObjectProvider.h"

namespace TraceServices
{

class IAnalysisSession;
class IEditableObjectProvider;

class FObjectAnalyzer : public UE::Trace::IAnalyzer
{
private:
	enum : uint16
	{
		RouteId_BeginSnapshot,
		RouteId_EndSnapshot,
		RouteId_ObjectSpec,
		RouteId_VerseExtraSpec,
		RouteId_ResourceSizeExtraSpec,
		RouteId_TotalResourceSizeExtraSpec,
		RouteId_FieldExtraSpec,
		RouteId_StructExtraSpec,
		RouteId_ClassExtraSpec,
		RouteId_FunctionExtraSpec,
		RouteId_PackageExtraSpec,
		RouteId_ObjectRef,
	};

public:
	FObjectAnalyzer(IAnalysisSession& Session, IEditableObjectProvider& InEditableObjectProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	bool CheckValidSnapshotForEvent(const ANSICHAR* EventName);
	void LogErrorForEventWithInvalidObjectId(const ANSICHAR* EventName, uint32 Id);

private:
	IAnalysisSession& Session;
	IEditableObjectProvider& EditableProvider;

	static constexpr uint32 InvalidSnapshotId = uint32(-1);
	uint32 CurrentSnapshotId = InvalidSnapshotId;
	double SnapshotLastTime = 0.0;
	uint32 NumWarnings = 0;
	uint32 NumErrors = 0;
	static constexpr uint32 MaxWarningMessages = 100;
	static constexpr uint32 MaxErrorMessages = 100;
};

} // namespace TraceServices
