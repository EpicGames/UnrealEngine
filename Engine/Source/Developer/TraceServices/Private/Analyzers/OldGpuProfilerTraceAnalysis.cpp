// Copyright Epic Games, Inc. All Rights Reserved.

#include "OldGpuProfilerTraceAnalysis.h"

#include "CborWriter.h"
#include "CborReader.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

// TraceServices
#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "Model/TimingProfilerPrivate.h"

#define UE_INSIGHTS_MERGE_UNACCOUNTED_TIMERS 0

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FOldGpuProfilerAnalyzer::FOldGpuProfilerAnalyzer(FAnalysisSession& InSession, FTimingProfilerProvider& InTimingProfilerProvider)
	: Session(InSession)
	, TimingProfilerProvider(InTimingProfilerProvider)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FOldGpuProfilerAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	// The Old GPU Profiler (deprecated in UE 5.6)
	// Analysis code is maintained for backward compatibility with old traces.
	Builder.RouteEvent(RouteId_EventSpec, "GpuProfiler", "EventSpec");
	Builder.RouteEvent(RouteId_Frame,     "GpuProfiler", "Frame");
	Builder.RouteEvent(RouteId_Frame2,    "GpuProfiler", "Frame2");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FOldGpuProfilerAnalyzer::OnAnalysisEnd()
{
	if (NumFramesWithErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[GpuProfiler] Frames with errors: %u", NumFramesWithErrors);
	}
	
	if (NumFrames > 0 || EventTypeMap.Num() > 0)
	{
		UE_LOGF(LogTraceServices, Log, "[GpuProfiler] Analysis completed (%u frames, %d timers).", NumFrames, EventTypeMap.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FOldGpuProfilerAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FOldGpuProfilerAnalyzer"));

	switch (RouteId)
	{
	case RouteId_EventSpec:
		OnEventSpec(Context);
		break;
	case RouteId_Frame:
		OnFrame(Context, 0);
		break;
	case RouteId_Frame2:
		OnFrame(Context, 1);
		break;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FOldGpuProfilerAnalyzer::OnEventSpec(const UE::Trace::IAnalyzer::FOnEventContext& Context)
{
	const auto& EventData = Context.EventData;

	uint32 EventType = EventData.GetValue<uint32>("EventType");
	const auto& Name = EventData.GetArray<UTF16CHAR>("Name");

	auto NameTChar = StringCast<TCHAR>(Name.GetData(), Name.Num());
	uint32* TimerIndexPtr = EventTypeMap.Find(EventType);
	if (!TimerIndexPtr)
	{
		uint32 TimerId;
#if UE_INSIGHTS_MERGE_UNACCOUNTED_TIMERS
		FStringView Unaccounted = TEXTVIEW("Unaccounted -");
		if (NameTChar.Length() > Unaccounted.Len() &&
			FCString::Strncmp(Unaccounted.GetData(), NameTChar.Get(), Unaccounted.Len()) == 0)
		{
			FAnalysisSessionEditScope _(Session);
			TimerId = TimingProfilerProvider.AddGpuTimer(TEXTVIEW("Unaccounted"));
		}
		else
#endif // UE_INSIGHTS_MERGE_UNACCOUNTED_TIMERS
		{
			FAnalysisSessionEditScope _(Session);
			TimerId = TimingProfilerProvider.AddGpuTimer(FStringView(NameTChar.Get(), NameTChar.Length()));
		}
		EventTypeMap.Add(EventType, TimerId);
	}
	else
	{
		FAnalysisSessionEditScope _(Session);
		TimingProfilerProvider.SetTimerName(*TimerIndexPtr, FStringView(NameTChar.Get(), NameTChar.Length()));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FOldGpuProfilerAnalyzer::OnFrame(const UE::Trace::IAnalyzer::FOnEventContext& Context, uint32 GpuIndex)
{
	const auto& EventData = Context.EventData;

	FAnalysisSessionEditScope _(Session);

	TraceServices::FTimingProfilerProvider::TimelineInternal& ThisTimeline = (GpuIndex == 0) ?
		TimingProfilerProvider.EditGpuTimeline() :
		TimingProfilerProvider.EditGpu2Timeline();
	double& ThisMinTime = (GpuIndex == 0) ? MinTime : MinTime2;

	const auto& Data = EventData.GetArray<uint8>("Data");
	const uint8* BufferPtr = Data.GetData();
	const uint8* BufferEnd = BufferPtr + Data.Num();

	uint64 CalibrationBias = EventData.GetValue<uint64>("CalibrationBias");
	uint64 LastTimestamp = EventData.GetValue<uint64>("TimestampBase");
	uint32 RenderingFrameNumber = EventData.GetValue<uint32>("RenderingFrameNumber");

	++NumFrames;

	double LastTime = 0.0;
	uint32 CurrentDepth = 0;
	bool bHasErrors = false;

	while (BufferPtr < BufferEnd)
	{
		uint64 DecodedTimestamp = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
		uint64 ActualTimestamp = (DecodedTimestamp >> 1) + LastTimestamp;
		LastTimestamp = ActualTimestamp;
		LastTime = double(ActualTimestamp + CalibrationBias) * 0.000001;
		LastTime += Context.EventTime.AsSeconds(0);

		if (LastTime < 0.0)
		{
			if (DecodedTimestamp & 1ull)
			{
				BufferPtr += sizeof(uint32);
			}
			bHasErrors = true;
			continue;
		}

		// If it advances with more than 1h, it is probably a wrong timestamp.
		if (LastTime > ThisMinTime + 3600.0 && ThisMinTime != DBL_MIN)
		{
			LastTime = ThisMinTime;
			bHasErrors = true;
		}

		// The monolithic timeline assumes that timestamps are ever increasing, but
		// with GPU/CPU calibration and drift there can be a tiny bit of overlap between
		// frames. So we just clamp.
		if (ThisMinTime > LastTime)
		{
			LastTime = ThisMinTime;
		}
		ThisMinTime = LastTime;

		if (DecodedTimestamp & 1ull)
		{
			uint32 EventType = *reinterpret_cast<const uint32*>(BufferPtr);
			BufferPtr += sizeof(uint32);
			if (EventTypeMap.Contains(EventType))
			{
				FTimingProfilerEvent Event;
				Event.TimerIndex = EventTypeMap[EventType];
				ThisTimeline.AppendBeginEvent(LastTime, Event);
			}
			else
			{
				FTimingProfilerEvent Event;
				Event.TimerIndex = TimingProfilerProvider.AddGpuTimer(TEXTVIEW("<unknown>"));
				EventTypeMap.Add(EventType, Event.TimerIndex);
				ThisTimeline.AppendBeginEvent(LastTime, Event);
			}
			++CurrentDepth;
		}
		else
		{
			if (CurrentDepth > 0)
			{
				--CurrentDepth;
			}
			ThisTimeline.AppendEndEvent(LastTime);
		}
	}
	check(BufferPtr == BufferEnd);
	check(CurrentDepth == 0);
	if (bHasErrors && ++NumFramesWithErrors <= 100)
	{
		UE_LOGF(LogTraceServices, Error, "[GpuProfiler] The rendering frame %u has invalid timestamps!", RenderingFrameNumber);
	}
	Session.UpdateDurationSeconds(LastTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef UE_INSIGHTS_MERGE_UNACCOUNTED_TIMERS
