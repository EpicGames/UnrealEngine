// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Trace/Analyzer.h"

namespace TraceServices
{

class FAnalysisSession;
class FTimingProfilerProvider;

// The Old GPU Profiler (deprecated in UE 5.6)
// Analysis code is maintained for backward compatibility with old traces.
class FOldGpuProfilerAnalyzer : public UE::Trace::IAnalyzer
{
private:
	enum : uint16
	{
		RouteId_EventSpec,
		RouteId_Frame, // GPU Index 0
		RouteId_Frame2, // GPU Index 1
	};

public:
	FOldGpuProfilerAnalyzer(FAnalysisSession& Session, FTimingProfilerProvider& TimingProfilerProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	void OnEventSpec(const FOnEventContext& Context);
	void OnFrame(const FOnEventContext& Context, uint32 GpuIndex = 0);

private:
	FAnalysisSession& Session;
	FTimingProfilerProvider& TimingProfilerProvider;

	TMap<uint64, uint32> EventTypeMap;
	double MinTime = DBL_MIN;
	double MinTime2 = DBL_MIN;
	uint32 NumFrames = 0;
	uint32 NumFramesWithErrors = 0;
};

} // namespace TraceServices
