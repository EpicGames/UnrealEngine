// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformEventTraceAnalysis.h"

#include "HAL/LowLevelMemTracker.h"
#include "Model/ContextSwitchesPrivate.h"
#include "Model/StackSamplesPrivate.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

FPlatformEventTraceAnalyzer::FPlatformEventTraceAnalyzer(IAnalysisSession& InSession,
														 FContextSwitchesProvider& InContextSwitchesProvider,
														 FStackSamplesProvider& InStackSamplesProvider)
	: Session(InSession)
	, ContextSwitchesProvider(InContextSwitchesProvider)
	, StackSamplesProvider(InStackSamplesProvider)
{
}

void FPlatformEventTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Settings, "PlatformEvent", "Settings");
	Builder.RouteEvent(RouteId_ContextSwitch, "PlatformEvent", "ContextSwitch");
	Builder.RouteEvent(RouteId_StackSample, "PlatformEvent", "StackSample");
	Builder.RouteEvent(RouteId_ThreadName, "PlatformEvent", "ThreadName");
}

void FPlatformEventTraceAnalyzer::OnAnalysisEnd()
{
	double MaxEndTime = 0.0;
	{
		TraceServices::FProviderEditScopeLock ScopedLock(StackSamplesProvider);
		MaxEndTime = StackSamplesProvider.OnAnalysisEnd();
	}
	if (MaxEndTime > 0.0)
	{
		FAnalysisSessionEditScope _(Session);
		Session.UpdateDurationSeconds(MaxEndTime);
	}

	int32 TryCount = 10;
	while (TryCount > 0)
	{
		if (!StackSamplesProvider.UpdatePendingTimers())
		{
			break;
		}
		FPlatformProcess::Sleep(1.0f);
		--TryCount;
	}
}

bool FPlatformEventTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FPlatformEventTraceAnalyzer"));

	switch (RouteId)
	{

	case RouteId_Settings:
	{
		const auto& EventData = Context.EventData;
		uint32 SamplingIntervalUsec = EventData.GetValue<uint32>("SamplingInterval");
		{
			TraceServices::FProviderEditScopeLock ScopedLock(StackSamplesProvider);
			StackSamplesProvider.SetSamplingInterval(SamplingIntervalUsec);
		}
		break;
	}

	case RouteId_ContextSwitch:
	{
		const auto& EventData = Context.EventData;
		double Start = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("StartTime"));
		double End = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("EndTime"));
		uint32 ThreadId = EventData.GetValue<uint32>("ThreadId");
		uint32 CoreNumber = EventData.GetValue<uint8>("CoreNumber");
		{
			TraceServices::FProviderEditScopeLock ScopedLock(ContextSwitchesProvider);
			ContextSwitchesProvider.AddTimingEvent(ThreadId, Start, End, CoreNumber);
		}
		{
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(End);
		}
		break;
	}

	case RouteId_StackSample:
	{
		const auto& EventData = Context.EventData;
		double Time = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Time"));
		uint32 ThreadId = EventData.GetValue<uint32>("ThreadId");
		const TArrayReader<uint64>& Addresses = EventData.GetArray<uint64>("Addresses");
		{
			TraceServices::FProviderEditScopeLock ScopedLock(StackSamplesProvider);
			StackSamplesProvider.AddStackSample(ThreadId, Time, Addresses.Num(), Addresses.GetData());
		}
		StackSamplesProvider.UpdatePendingTimers();
		{
			FAnalysisSessionEditScope _(Session);
			Session.UpdateDurationSeconds(Time);
		}
		break;
	}

	case RouteId_ThreadName:
	{
		const auto& EventData = Context.EventData;
		uint32 ThreadId = EventData.GetValue<uint32>("ThreadId");
		uint32 ProcessId = EventData.GetValue<uint32>("ProcessId");
		FStringView Name;
		if (EventData.GetString("Name", Name))
		{
			{
				TraceServices::FProviderEditScopeLock ScopedLock(ContextSwitchesProvider);
				ContextSwitchesProvider.AddThreadName(ThreadId, ProcessId, Name);
			}
			{
				TraceServices::FProviderEditScopeLock ScopedLock(StackSamplesProvider);
				StackSamplesProvider.AddThreadName(ThreadId, ProcessId, Name);
			}
		}
		break;
	}

	} // switch (RouteId)

	return true;
}

void FPlatformEventTraceAnalyzer::OnThreadInfo(const FThreadInfo& ThreadInfo)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FPlatformEventTraceAnalyzer"));

	{
		TraceServices::FProviderEditScopeLock ScopedLock(ContextSwitchesProvider);
		ContextSwitchesProvider.AddThreadInfo(ThreadInfo.GetId(), ThreadInfo.GetSystemId());
	}
	{
		//TraceServices::FProviderEditScopeLock ScopedLock(StackSamplesProvider);
		//StackSamplesProvider.AddThreadInfo(ThreadInfo.GetId(), ThreadInfo.GetSystemId());
	}
}

} // namespace TraceServices
