// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "ProfilingDebugging/MiscTrace.h"

namespace TraceServices
{

class IAnalysisSession;
class FContextSwitchesProvider;
class FStackSamplesProvider;

class FPlatformEventTraceAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FPlatformEventTraceAnalyzer(IAnalysisSession& Session,
								FContextSwitchesProvider& ContextSwitchesProvider,
								FStackSamplesProvider& StackSamplesProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	virtual void OnThreadInfo(const FThreadInfo& ThreadInfo) override;

private:
	enum : uint16
	{
		RouteId_Settings,
		RouteId_ContextSwitch,
		RouteId_StackSample,
		RouteId_ThreadName,
	};

	IAnalysisSession& Session;
	FContextSwitchesProvider& ContextSwitchesProvider;
	FStackSamplesProvider& StackSamplesProvider;
};

} // namespace TraceServices
