// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformFileTraceAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "Model/FileActivity.h"
#include "TraceServices/Utils.h"

#define DEBUG_PLATFORMFILETRACE 0

#if DEBUG_PLATFORMFILETRACE
#define PLATFORMFILETRACE_WARNING(x) ensureMsgf(false, TEXT(x))
#else
#define PLATFORMFILETRACE_WARNING(x)
#endif

namespace TraceServices
{

FPlatformFileTraceAnalyzer::FPlatformFileTraceAnalyzer(IAnalysisSession& InSession, FFileActivityProvider& InFileActivityProvider)
	: Session(InSession)
	, FileActivityProvider(InFileActivityProvider)
{

}

void FPlatformFileTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_BeginOpen, "PlatformFile", "BeginOpen");
	Builder.RouteEvent(RouteId_EndOpen, "PlatformFile", "EndOpen");
	Builder.RouteEvent(RouteId_BeginReOpen, "PlatformFile", "BeginReOpen");
	Builder.RouteEvent(RouteId_EndReOpen, "PlatformFile", "EndReOpen");
	Builder.RouteEvent(RouteId_BeginClose, "PlatformFile", "BeginClose");
	Builder.RouteEvent(RouteId_EndClose, "PlatformFile", "EndClose");
	Builder.RouteEvent(RouteId_BeginRead, "PlatformFile", "BeginRead");
	Builder.RouteEvent(RouteId_EndRead, "PlatformFile", "EndRead");
	Builder.RouteEvent(RouteId_BeginWrite, "PlatformFile", "BeginWrite");
	Builder.RouteEvent(RouteId_EndWrite, "PlatformFile", "EndWrite");
}

bool FPlatformFileTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FPlatformFileTraceAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_BeginOpen:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		#if DEBUG_PLATFORMFILETRACE
		if (PendingOpenMap.Contains(ThreadId))
		{
			PLATFORMFILETRACE_WARNING("BeginOpen: Duplicated event!?");
		}
		#endif
		const FString FileName = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Path", Context);
		uint32 FileIndex = FileActivityProvider.GetFileIndex(*FileName);
		FPendingActivity& Open = PendingOpenMap.Add(ThreadId);
		Open.ActivityIndex = FileActivityProvider.BeginActivity(FileIndex, FileActivityType_Open, ThreadId, 0, 0, Time);
		Open.FileIndex = FileIndex;
		break;
	}
	case RouteId_EndOpen:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 FileHandle = EventData.GetValue<uint64>("FileHandle");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const FPendingActivity* FindOpen = PendingOpenMap.Find(ThreadId);
		if (FindOpen)
		{
			if (FileHandle == uint64(-1)) // invalid file handle
			{
				FileActivityProvider.EndActivity(FindOpen->FileIndex, FindOpen->ActivityIndex, 0, Time, true);
			}
			else
			{
				#if DEBUG_PLATFORMFILETRACE
				if (OpenFilesMap.Contains(FileHandle))
				{
					PLATFORMFILETRACE_WARNING("EndOpen: File already open!?");
				}
				#endif
				OpenFilesMap.Add(FileHandle, FindOpen->FileIndex);
				FileActivityProvider.EndActivity(FindOpen->FileIndex, FindOpen->ActivityIndex, 0, Time, false);
			}
			FileActivityProvider.SetActivityFileHandle(FindOpen->FileIndex, FindOpen->ActivityIndex, FileHandle);
			PendingOpenMap.Remove(ThreadId);
		}
		else
		{
			PLATFORMFILETRACE_WARNING("EndOpen: BeginOpen event not traced!?");
		}
		break;
	}
	case RouteId_BeginReOpen:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 FileHandle = EventData.GetValue<uint64>("OldFileHandle");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		#if DEBUG_PLATFORMFILETRACE
		if (PendingReOpenMap.Contains(ThreadId))
		{
			PLATFORMFILETRACE_WARNING("BeginReOpen: Duplicated event!?");
		}
		#endif
		const uint32* FindFileIndex = OpenFilesMap.Find(FileHandle);
		if (!FindFileIndex)
		{
			PLATFORMFILETRACE_WARNING("BeginReOpen: File is not open!?");
		}
		else
		{
			uint32 FileIndex = *FindFileIndex;
			FPendingActivity& Open = PendingReOpenMap.Add(ThreadId);
			Open.ActivityIndex = FileActivityProvider.BeginActivity(FileIndex, FileActivityType_ReOpen, ThreadId, 0, 0, Time);
			Open.FileIndex = FileIndex;
		}
		break;
	}
	case RouteId_EndReOpen:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 NewFileHandle = EventData.GetValue<uint64>("NewFileHandle");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const FPendingActivity* FindOpen = PendingReOpenMap.Find(ThreadId);
		if (FindOpen)
		{
			if (NewFileHandle == uint64(-1)) // invalid file handle
			{
				FileActivityProvider.EndActivity(FindOpen->FileIndex, FindOpen->ActivityIndex, 0, Time, true);
			}
			else
			{
				#if DEBUG_PLATFORMFILETRACE
				if (OpenFilesMap.Contains(NewFileHandle))
				{
					PLATFORMFILETRACE_WARNING("EndReOpen: File already open!?");
				}
				#endif
				OpenFilesMap.Add(NewFileHandle, FindOpen->FileIndex);
				FileActivityProvider.EndActivity(FindOpen->FileIndex, FindOpen->ActivityIndex, 0, Time, false);
			}
			FileActivityProvider.SetActivityFileHandle(FindOpen->FileIndex, FindOpen->ActivityIndex, NewFileHandle);
			PendingReOpenMap.Remove(ThreadId);
		}
		else
		{
			PLATFORMFILETRACE_WARNING("EndReOpen: BeginReOpen event not traced!?");
		}
		break;
	}
	case RouteId_BeginClose:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 FileHandle = EventData.GetValue<uint64>("FileHandle");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		#if DEBUG_PLATFORMFILETRACE
		if (PendingCloseMap.Contains(ThreadId))
		{
			PLATFORMFILETRACE_WARNING("BeginClose: Duplicated event!?");
		}
		#endif
		const uint32* FindFileIndex = OpenFilesMap.Find(FileHandle);
		uint32 FileIndex;
		if (FindFileIndex)
		{
			FileIndex = *FindFileIndex;
			OpenFilesMap.Remove(FileHandle);
		}
		else
		{
			PLATFORMFILETRACE_WARNING("BeginClose: File is not open!?");
			FileIndex = FileActivityProvider.GetUnknownFileIndex();
		}
		FPendingActivity& Close = PendingCloseMap.Add(ThreadId);
		Close.ActivityIndex = FileActivityProvider.BeginActivity(FileIndex, FileActivityType_Close, ThreadId, 0, 0, Time);
		FileActivityProvider.SetActivityFileHandle(FileIndex, Close.ActivityIndex, FileHandle);
		Close.FileIndex = FileIndex;
		break;
	}
	case RouteId_EndClose:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const FPendingActivity* FindClose = PendingCloseMap.Find(ThreadId);
		if (FindClose)
		{
			FileActivityProvider.EndActivity(FindClose->FileIndex, FindClose->ActivityIndex, 0, Time, false);
			PendingCloseMap.Remove(ThreadId);
		}
		else
		{
			PLATFORMFILETRACE_WARNING("EndClose: BeginClose event not traced!?");
		}
		break;
	}
	case RouteId_BeginRead:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 ReadHandle = EventData.GetValue<uint64>("ReadHandle");
		uint64 FileHandle = EventData.GetValue<uint64>("FileHandle");
		uint64 Offset = EventData.GetValue<uint64>("Offset");
		uint64 Size = EventData.GetValue<uint64>("Size");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const uint32* FindFileIndex = OpenFilesMap.Find(FileHandle);
		uint32 FileIndex;
		if (FindFileIndex)
		{
			FileIndex = *FindFileIndex;
		}
		else
		{
			PLATFORMFILETRACE_WARNING("BeginRead: File is not open!?");
			FileIndex = FileActivityProvider.GetUnknownFileIndex();
			OpenFilesMap.Add(FileHandle, FileIndex);
		}
		uint64 ReadIndex = FileActivityProvider.BeginActivity(FileIndex, FileActivityType_Read, ThreadId, Offset, Size, Time);
		FileActivityProvider.SetActivityFileHandle(FileIndex, ReadIndex, FileHandle);
		FileActivityProvider.SetActivityReadWriteHandle(FileIndex, ReadIndex, ReadHandle);
		#if DEBUG_PLATFORMFILETRACE
		if (ActiveReadsMap.Contains(ReadHandle))
		{
			PLATFORMFILETRACE_WARNING("BeginRead: Duplicated ReadHandle!?");
		}
		#endif
		FPendingActivity& Read = ActiveReadsMap.Add(ReadHandle);
		Read.FileIndex = FileIndex;
		Read.ActivityIndex = ReadIndex;
		break;
	}
	case RouteId_EndRead:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 ReadHandle = EventData.GetValue<uint64>("ReadHandle");
		uint64 SizeRead = EventData.GetValue<uint64>("SizeRead");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const FPendingActivity* FindRead = ActiveReadsMap.Find(ReadHandle);
		if (FindRead)
		{
			FileActivityProvider.EndActivity(FindRead->FileIndex, FindRead->ActivityIndex, SizeRead, Time, false);
			FileActivityProvider.CheckActivityReadWriteHandle(FindRead->FileIndex, FindRead->ActivityIndex, ReadHandle);
			ActiveReadsMap.Remove(ReadHandle);
		}
		else
		{
			PLATFORMFILETRACE_WARNING("EndRead: BeginRead event not traced!?");
		}
		break;
	}
	case RouteId_BeginWrite:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 WriteHandle = EventData.GetValue<uint64>("WriteHandle");
		uint64 FileHandle = EventData.GetValue<uint64>("FileHandle");
		uint64 Offset = EventData.GetValue<uint64>("Offset");
		uint64 Size = EventData.GetValue<uint64>("Size");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const uint32* FindFileIndex = OpenFilesMap.Find(FileHandle);
		uint32 FileIndex;
		if (FindFileIndex)
		{
			FileIndex = *FindFileIndex;
		}
		else
		{
			PLATFORMFILETRACE_WARNING("BeginWrite: File is not open!?");
			FileIndex = FileActivityProvider.GetUnknownFileIndex();
			OpenFilesMap.Add(FileHandle, FileIndex);
		}
		uint64 WriteIndex = FileActivityProvider.BeginActivity(FileIndex, FileActivityType_Write, ThreadId, Offset, Size, Time);
		FileActivityProvider.SetActivityFileHandle(FileIndex, WriteIndex, FileHandle);
		FileActivityProvider.SetActivityReadWriteHandle(FileIndex, WriteIndex, WriteHandle);
		#if DEBUG_PLATFORMFILETRACE
		if (ActiveWritesMap.Contains(WriteHandle))
		{
			PLATFORMFILETRACE_WARNING("BeginWrite: Duplicated WriteHandle!?");
		}
		#endif
		FPendingActivity& Write = ActiveWritesMap.Add(WriteHandle);
		Write.FileIndex = FileIndex;
		Write.ActivityIndex = WriteIndex;
		break;
	}
	case RouteId_EndWrite:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double Time = Context.EventTime.AsSeconds(Cycle);
		Session.UpdateDurationSeconds(Time);
		uint64 WriteHandle = EventData.GetValue<uint64>("WriteHandle");
		uint64 SizeWritten = EventData.GetValue<uint64>("SizeWritten");
		uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
		const FPendingActivity* FindWrite = ActiveWritesMap.Find(WriteHandle);
		if (FindWrite)
		{
			FileActivityProvider.EndActivity(FindWrite->FileIndex, FindWrite->ActivityIndex, SizeWritten, Time, false);
			FileActivityProvider.CheckActivityReadWriteHandle(FindWrite->FileIndex, FindWrite->ActivityIndex, WriteHandle);
			ActiveWritesMap.Remove(WriteHandle);
		}
		else
		{
			PLATFORMFILETRACE_WARNING("EndWrite: BeginWrite event not traced!?");
		}
		break;
	}
	}

	return true;
}

} // namespace TraceServices

#undef PLATFORMFILETRACE_WARNING
#undef DEBUG_PLATFORMFILETRACE
