// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/Map.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

class FTimeline;
class FFileActivityProvider;

class FPlatformFileTraceAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FPlatformFileTraceAnalyzer(IAnalysisSession& Session, FFileActivityProvider& FileActivityProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_BeginOpen,
		RouteId_EndOpen,
		RouteId_BeginReOpen,
		RouteId_EndReOpen,
		RouteId_BeginClose,
		RouteId_EndClose,
		RouteId_BeginRead,
		RouteId_EndRead,
		RouteId_BeginWrite,
		RouteId_EndWrite,

		RouteId_Count
	};

	struct FPendingActivity
	{
		uint64 ActivityIndex;
		uint32 FileIndex;
		uint32 ThreadId;
	};

	IAnalysisSession& Session;
	FFileActivityProvider& FileActivityProvider;
	TMap<uint64, uint32> OpenFilesMap;
	TMap<uint32, FPendingActivity> PendingOpenMap;
	TMap<uint32, FPendingActivity> PendingReOpenMap;
	TMap<uint32, FPendingActivity> PendingCloseMap;
	TMap<uint64, FPendingActivity> ActiveReadsMap;
	TMap<uint64, FPendingActivity> ActiveWritesMap;
};

} // namespace TraceServices
