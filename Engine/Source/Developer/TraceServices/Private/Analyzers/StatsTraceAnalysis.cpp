// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsTraceAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "TraceServices/Model/Counters.h"

#include <limits>

#define STATS_ANALYZER_DEBUG_LOG(StatId, Format, ...) //{ if (StatId == 389078 /*STAT_MeshDrawCalls*/) UE_LOG(LogTraceServices, Log, Format, ##__VA_ARGS__); }

namespace TraceServices
{

FStatsAnalyzer::FStatsAnalyzer(IAnalysisSession& InSession, IEditableCounterProvider& InEditableCounterProvider)
	: Session(InSession)
	, EditableCounterProvider(InEditableCounterProvider)
{
}

void FStatsAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Spec, "Stats", "Spec");
	Builder.RouteEvent(RouteId_EventBatch, "Stats", "EventBatch");
	Builder.RouteEvent(RouteId_EventBatch2, "Stats", "EventBatch2");
	Builder.RouteEvent(RouteId_BeginFrame, "Misc", "BeginFrame");
	//Builder.RouteEvent(RouteId_EndFrame, "Misc", "EndFrame");
}

void FStatsAnalyzer::OnAnalysisEnd()
{
	CreateFrameCounters();
}

bool FStatsAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FStatsAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_Spec:
		{
			bool bIsAlreadyDeclared = false;
			uint32 StatId = EventData.GetValue<uint32>("Id");
			IEditableCounter* EditableCounter = EditableCountersMap.FindRef(StatId);
			if (!EditableCounter)
			{
				EditableCounter = EditableCounterProvider.CreateEditableCounter();
				EditableCountersMap.Add(StatId, EditableCounter);
			}
			else
			{
				bIsAlreadyDeclared = true;
				UE_LOGF(LogTraceServices, Warning, "Stats counter %u already declared.", StatId);
			}

			const ICounter* Counter = EditableCounterProvider.GetCounter(EditableCounter);

			FString Name;
			FString Description;
			FString Group;
			if (EventData.GetString("Name", Name))
			{
				EventData.GetString("Description", Description);
				EventData.GetString("Group", Group);
			}
			else
			{
				Name = reinterpret_cast<const ANSICHAR*>(EventData.GetAttachment());
				Description = reinterpret_cast<const TCHAR*>(EventData.GetAttachment() + Name.Len() + 1);
			}

			if (Name.IsEmpty())
			{
				UE_LOGF(LogTraceServices, Warning, "Invalid counter name for Stats counter %u.", StatId);
				Name = FString::Printf(TEXT("<noname stats counter %u>"), StatId);
			}
			//if (!Name.IsEmpty())
			{
				if (!bIsAlreadyDeclared)
				{
					EditableCounter->SetName(Session.StoreString(*Name));
				}
				else if (Counter && FCString::Strcmp(*Name, Counter->GetName()) != 0)
				{
					UE_LOGF(LogTraceServices, Error, "Stats counter %u redeclared with different name.", StatId);
				}
			}

			if (!Group.IsEmpty())
			{
				if (!bIsAlreadyDeclared)
				{
					EditableCounter->SetGroup(Session.StoreString(*Group));
				}
				else if (Counter && FCString::Strcmp(*Group, Counter->GetGroup()) != 0)
				{
					UE_LOGF(LogTraceServices, Error, "Stats counter %u redeclared with different group name.", StatId);
				}
			}

			if (!Description.IsEmpty())
			{
				if (!bIsAlreadyDeclared)
				{
					EditableCounter->SetDescription(Session.StoreString(*Description));
				}
				else if (Counter && FCString::Strcmp(*Description, Counter->GetDescription()) != 0)
				{
					UE_LOGF(LogTraceServices, Error, "Stats counter %u redeclared with different description.", StatId);
				}
			}

			bool bIsFloatingPoint = EventData.GetValue<bool>("IsFloatingPoint");
			if (!bIsAlreadyDeclared)
			{
				EditableCounter->SetIsFloatingPoint(bIsFloatingPoint);
			}
			else if (Counter && bIsFloatingPoint != Counter->IsFloatingPoint())
			{
				UE_LOGF(LogTraceServices, Error, "Stats counter %u redeclared with different IsFloatingPoint flag.", StatId);
			}

			bool bIsResetEveryFrame = EventData.GetValue<bool>("ShouldClearEveryFrame");
			if (!bIsAlreadyDeclared)
			{
				EditableCounter->SetIsResetEveryFrame(bIsResetEveryFrame);
				if (bIsResetEveryFrame)
				{
					if (bIsFloatingPoint)
					{
						IEditableCounter* ResetEveryFrameEditableCounter = FloatResetEveryFrameCountersMap.FindRef(StatId);
						if (!ResetEveryFrameEditableCounter)
						{
							FloatResetEveryFrameCountersMap.Add(StatId, EditableCounter);
						}
					}
					else
					{
						IEditableCounter* ResetEveryFrameEditableCounter = Int64ResetEveryFrameCountersMap.FindRef(StatId);
						if (!ResetEveryFrameEditableCounter)
						{
							Int64ResetEveryFrameCountersMap.Add(StatId, EditableCounter);
						}
					}
				}
			}
			else if (Counter && bIsResetEveryFrame != Counter->IsResetEveryFrame())
			{
				UE_LOGF(LogTraceServices, Error, "Stats counter %u redeclared with different IsResetEveryFrame flag.", StatId);
			}

			ECounterDisplayHint DisplayHint = CounterDisplayHint_None;
			if (EventData.GetValue<bool>("IsMemory"))
			{
				DisplayHint = CounterDisplayHint_Memory;
			}
			if (!bIsAlreadyDeclared)
			{
				EditableCounter->SetDisplayHint(DisplayHint);
			}
			else if (Counter && DisplayHint != Counter->GetDisplayHint())
			{
				UE_LOGF(LogTraceServices, Error, "Stats counter %u redeclared with different display hint.", StatId);
			}
			break;
		}

		case RouteId_EventBatch: // deprecated in UE 5.3
		case RouteId_EventBatch2: // added in UE 5.3
		{
			uint32 ThreadId = FTraceAnalyzerUtils::GetThreadIdField(Context);
			TSharedRef<FThreadState> ThreadState = GetThreadState(ThreadId);

			const uint64 BaseTimestamp = Context.EventTime.AsCycle64() - Context.EventTime.GetTimestamp();

			TArrayView<const uint8> DataView = FTraceAnalyzerUtils::LegacyAttachmentArray("Data", Context);
			uint64 BufferSize = DataView.Num();
			const uint8* BufferPtr = DataView.GetData();
			const uint8* BufferEnd = BufferPtr + BufferSize;
			while (BufferPtr < BufferEnd)
			{
				enum EOpType
				{
					Increment = 0,
					Decrement = 1,
					AddInteger = 2,
					SetInteger = 3,
					AddFloat = 4,
					SetFloat = 5,
				};

				uint64 DecodedIdAndOp = FTraceAnalyzerUtils::Decode7bit(BufferPtr);
				uint32 StatId = static_cast<uint32>(DecodedIdAndOp >> 3);
				IEditableCounter* EditableCounter = EditableCountersMap.FindRef(StatId);
				if (!EditableCounter)
				{
					EditableCounter = EditableCounterProvider.CreateEditableCounter();
					FString Name = FString::Printf(TEXT("<unknown stats counter %u>"), StatId);
					EditableCounter->SetName(Session.StoreString(*Name));
					EditableCountersMap.Add(StatId, EditableCounter);
				}

				uint8 Op = DecodedIdAndOp & 0x7;
				uint64 CycleDiff = FTraceAnalyzerUtils::Decode7bit(BufferPtr);

				if (RouteId == RouteId_EventBatch2)
				{
					if (CycleDiff >= BaseTimestamp)
					{
						ThreadState->LastCycle = 0;
					}
				}

				uint64 Cycle = ThreadState->LastCycle + CycleDiff;
				double Time = Context.EventTime.AsSeconds(Cycle);
				ThreadState->LastCycle = Cycle;

				switch (Op)
				{
				case Increment:
				{
					EditableCounter->AddValue(Time, int64(1));
					STATS_ANALYZER_DEBUG_LOG(StatId, TEXT("%f INC() %u"), Time, ThreadId);
					break;
				}
				case Decrement:
				{
					EditableCounter->AddValue(Time, int64(-1));
					STATS_ANALYZER_DEBUG_LOG(StatId, TEXT("%f DEC() %u"), Time, ThreadId);
					break;
				}
				case AddInteger:
				{
					int64 Amount = FTraceAnalyzerUtils::DecodeZigZag(BufferPtr);
					EditableCounter->AddValue(Time, Amount);
					STATS_ANALYZER_DEBUG_LOG(StatId, TEXT("%f ADD(%lli) %u"), Time, Amount, ThreadId);
					break;
				}
				case SetInteger:
				{
					int64 Value = FTraceAnalyzerUtils::DecodeZigZag(BufferPtr);
					EditableCounter->SetValue(Time, Value);
					STATS_ANALYZER_DEBUG_LOG(StatId, TEXT("%f SET(%lli) %u"), Time, Value, ThreadId);
					break;
				}
				case AddFloat:
				{
					double Amount;
					memcpy(&Amount, BufferPtr, sizeof(double));
					BufferPtr += sizeof(double);
					EditableCounter->AddValue(Time, Amount);
					STATS_ANALYZER_DEBUG_LOG(StatId, TEXT("%f ADD(%f) %u"), Time, Amount, ThreadId);
					break;
				}
				case SetFloat:
				{
					double Value;
					memcpy(&Value, BufferPtr, sizeof(double));
					BufferPtr += sizeof(double);
					EditableCounter->SetValue(Time, Value);
					STATS_ANALYZER_DEBUG_LOG(StatId, TEXT("%f SET(%f) %u"), Time, Value, ThreadId);
					break;
				}
				}
			}
			check(BufferPtr == BufferEnd);
			break;
		}

		case RouteId_BeginFrame:
		//case RouteId_EndFrame:
		{
			uint8 FrameType = EventData.GetValue<uint8>("FrameType");
			check(FrameType < TraceFrameType_Count);
			if (ETraceFrameType(FrameType) == ETraceFrameType::TraceFrameType_Game)
			{
				uint64 Cycle = EventData.GetValue<uint64>("Cycle");
				double Time = Context.EventTime.AsSeconds(Cycle);
				for (auto& KV : FloatResetEveryFrameCountersMap)
				{
					STATS_ANALYZER_DEBUG_LOG(KV.Key, TEXT("%f RESET"), Time);
					KV.Value->SetValue(Time, 0.0);
				}
				for (auto& KV : Int64ResetEveryFrameCountersMap)
				{
					STATS_ANALYZER_DEBUG_LOG(KV.Key, TEXT("%f RESET"), Time);
					KV.Value->SetValue(Time, 0ll);
				}
			}
			break;
		}
	}

	return true;
}

TSharedRef<FStatsAnalyzer::FThreadState> FStatsAnalyzer::GetThreadState(uint32 ThreadId)
{
	if (!ThreadStatesMap.Contains(ThreadId))
	{
		TSharedRef<FThreadState> ThreadState = MakeShared<FThreadState>();
		ThreadState->LastCycle = 0;
		ThreadStatesMap.Add(ThreadId, ThreadState);
		return ThreadState;
	}
	else
	{
		return ThreadStatesMap[ThreadId];
	}
}

void FStatsAnalyzer::CreateFrameCounters()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FStatsAnalyzer"));

	FAnalysisSessionEditScope _(Session);

	auto CreateFrameCounter = [this](IEditableCounter* EditableCounter, const ICounter*& Counter, IEditableCounter*& FrameCounter)
	{
		Counter = EditableCounterProvider.GetCounter(EditableCounter);
		if (!Counter)
		{
			return;
		}

		FrameCounter = EditableCounterProvider.CreateEditableCounter();

		FString FrameCounterName = FString(Counter->GetName()) + TEXT(" (1/frame)");
		FrameCounter->SetName(Session.StoreString(*FrameCounterName));
		if (Counter->GetGroup())
		{
			FrameCounter->SetGroup(Counter->GetGroup());
		}
		if (Counter->GetDescription())
		{
			FrameCounter->SetDescription(Counter->GetDescription());
		}

		FrameCounter->SetIsFloatingPoint(Counter->IsFloatingPoint());
		FrameCounter->SetIsResetEveryFrame(false);
		FrameCounter->SetDisplayHint(Counter->GetDisplayHint());
	};

	constexpr double InfiniteTime = std::numeric_limits<double>::infinity();

	for (auto& KV : FloatResetEveryFrameCountersMap)
	{
		const ICounter* Counter = nullptr;
		IEditableCounter* FrameCounter = nullptr;
		CreateFrameCounter(KV.Value, Counter, FrameCounter);
		if (!FrameCounter)
		{
			continue;
		}

		bool bFirst = true;
		double FrameTime = 0.0;
		double FrameValue = 0.0;
		Counter->EnumerateFloatValues(-InfiniteTime, InfiniteTime, false, [FrameCounter, &bFirst, &FrameTime, &FrameValue](double Time, double Value)
			{
				if (bFirst && Value != 0.0)
				{
					bFirst = false;
					FrameCounter->SetValue(Time, 0.0);
				}
				if (Value == 0.0 && FrameValue != 0.0)
				{
					FrameCounter->SetValue(FrameTime, FrameValue);
				}
				FrameTime = Time;
				FrameValue = Value;
			});
	}

	for (auto& KV : Int64ResetEveryFrameCountersMap)
	{
		const ICounter* Counter = nullptr;
		IEditableCounter* FrameCounter = nullptr;
		CreateFrameCounter(KV.Value, Counter, FrameCounter);
		if (!FrameCounter)
		{
			continue;
		}

		bool bFirst = true;
		double FrameTime = 0.0;
		int64 FrameValue = 0;
		Counter->EnumerateValues(-InfiniteTime, InfiniteTime, false, [FrameCounter, &bFirst, &FrameTime, &FrameValue](double Time, int64 Value)
			{
				if (bFirst && Value != 0)
				{
					bFirst = false;
					FrameCounter->SetValue(Time, (int64)0);
				}
				if (Value == 0 && FrameValue != 0)
				{
					FrameCounter->SetValue(FrameTime, FrameValue);
				}
				FrameTime = Time;
				FrameValue = Value;
			});
	}
}

} // namespace TraceServices

#undef STATS_ANALYZER_DEBUG_LOG
