// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Templates/SharedPointer.h"

namespace TraceServices
{

class IAnalysisSession;
class ICounter;
class ICounterProvider;
class IEditableCounter;
class IEditableCounterProvider;

class FStatsAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FStatsAnalyzer(IAnalysisSession& Session, IEditableCounterProvider& InEditableCounterProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	void CreateFrameCounters();

private:
	enum : uint16
	{
		RouteId_Spec,
		RouteId_EventBatch,
		RouteId_EventBatch2,
		RouteId_BeginFrame,
		RouteId_EndFrame
	};

	struct FThreadState
	{
		uint64 LastCycle = 0;
	};

	TSharedRef<FThreadState> GetThreadState(uint32 ThreadId);

	IAnalysisSession& Session;
	IEditableCounterProvider& EditableCounterProvider;
	TMap<uint32, IEditableCounter*> EditableCountersMap;
	TMap<uint32, IEditableCounter*> Int64ResetEveryFrameCountersMap;
	TMap<uint32, IEditableCounter*> FloatResetEveryFrameCountersMap;
	TMap<uint32, TSharedRef<FThreadState>> ThreadStatesMap;
};

} // namespace TraceServices
