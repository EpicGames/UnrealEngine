// Copyright Epic Games, Inc. All Rights Reserved.

#include "StringsAnalyzer.h"
#include "Common/ProviderLock.h"
#include "Common/Utils.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Strings.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStringsAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	Context.InterfaceBuilder.RouteEvent(RouteId_StaticString, "Strings", "StaticString");
	Context.InterfaceBuilder.RouteEvent(RouteId_FName, "Strings", "FName");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FStringsAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	switch (RouteId)
	{
		// The layout of these two events is identical so they can be handled using the same code.
	case RouteId_StaticString:
	case RouteId_FName:
		{
			const FEventData& EventData = Context.EventData;
			IDefinitionProvider* DefinitionProvider = EditDefinitionProvider(Session);
			check(DefinitionProvider != nullptr);

			FWideStringView DisplayWide;
			FAnsiStringView DisplayAnsi;
			const TCHAR* Display = nullptr;
			if (EventData.GetString("DisplayWide", DisplayWide) && DisplayWide.Len() > 0)
			{
				Display = Session.StoreString(DisplayWide);
			}
			else if (EventData.GetString("DisplayAnsi", DisplayAnsi) && DisplayAnsi.Len() > 0)
			{
				const FString Str(DisplayAnsi);
				Display = Session.StoreString(Str);
			}
			else
			{
				UE_LOGF(LogTraceServices, Warning, "Empty string definition detected.");
				return true;
			}

			FProviderEditScopeLock Lock(*DefinitionProvider);
			FStringDefinition* Instance = DefinitionProvider->Create<FStringDefinition>();
			Instance->Display = Display;

			if (RouteId == RouteId_FName)
			{
				auto Id = EventData.GetDefinitionId<uint32>();
				DefinitionProvider->Register<FStringDefinition>(Instance, Id);
			}
			else
			{
				auto Id = EventData.GetDefinitionId<uint64>();
				DefinitionProvider->Register<FStringDefinition>(Instance, Id);
			}
		}
		break;
	default: ;
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
