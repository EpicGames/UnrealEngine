// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

class FStringsAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FStringsAnalyzer(IAnalysisSession& InSession)
		: Session(InSession)
	{
	}

private:
	enum ERouteId : uint16
	{
		RouteId_StaticString,
		RouteId_FName,
	};
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

	IAnalysisSession& Session;
};

} // namespace TraceServices
