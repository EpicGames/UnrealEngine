// Copyright Epic Games, Inc. All Rights Reserved.

#include "TasksAnalysis.h"

#include "AnalysisServicePrivate.h"
#include "Containers/UnrealString.h"
#include "HAL/LowLevelMemTracker.h"
#include "Model/TasksProfilerPrivate.h"
#include "Common/Utils.h"

namespace TraceServices
{

FTasksAnalyzer::FTasksAnalyzer(IAnalysisSession& InSession, FTasksProvider& InTasksProvider)
	: Session(InSession)
	, TasksProvider(InTasksProvider)
{
}

void FTasksAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Init, "TaskTrace", "Init");
	Builder.RouteEvent(RouteId_Created, "TaskTrace", "Created");
	Builder.RouteEvent(RouteId_Launched, "TaskTrace", "Launched");
	Builder.RouteEvent(RouteId_Scheduled, "TaskTrace", "Scheduled");
	Builder.RouteEvent(RouteId_SubsequentAdded, "TaskTrace", "SubsequentAdded");
	Builder.RouteEvent(RouteId_Started, "TaskTrace", "Started");
	Builder.RouteEvent(RouteId_NestedAdded, "TaskTrace", "NestedAdded");
	Builder.RouteEvent(RouteId_Finished, "TaskTrace", "Finished");
	Builder.RouteEvent(RouteId_Completed, "TaskTrace", "Completed");
	Builder.RouteEvent(RouteId_Destroyed, "TaskTrace", "Destroyed");

	Builder.RouteEvent(RouteId_WaitingStarted, "TaskTrace", "WaitingStarted");
	Builder.RouteEvent(RouteId_WaitingFinished, "TaskTrace", "WaitingFinished");
}

bool FTasksAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FTasksAnalyzer"));

	FAnalysisSessionEditScope _(Session);
	
	// the protocol must match TaskTrace.cpp

	const auto& EventData = Context.EventData;
	uint32 ThreadId = Context.ThreadInfo.GetId();
	switch (RouteId)
	{
		case RouteId_Init:
		{
			uint32 Version = EventData.GetValue<uint32>("Version");
			TasksProvider.Init(Version);
			break;
		}
		case RouteId_Created:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			uint64 TaskSize = EventData.GetValue<uint64>("TaskSize");
			TasksProvider.TaskCreated(TaskId, Timestamp, ThreadId, TaskSize);
			break;
		}
		case RouteId_Launched:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");

			FString DebugNameStr;
			const TCHAR* DebugName = nullptr;
			if (EventData.GetString("DebugName", DebugNameStr) && DebugNameStr.Len() != 0)
			{
				DebugName = Session.StoreString(*DebugNameStr);
			}

			bool bTracked = EventData.GetValue<bool>("Tracked");
			int32 ThreadToExecuteOn = EventData.GetValue<int32>("ThreadToExecuteOn");
			uint64 TaskSize = EventData.GetValue<uint64>("TaskSize");

			TasksProvider.TaskLaunched(TaskId, DebugName, bTracked, ThreadToExecuteOn, Timestamp, ThreadId, TaskSize);

			break;
		}
		case RouteId_Scheduled:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			uint64 TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			TasksProvider.TaskScheduled(TaskId, Timestamp, ThreadId);
			break;
		}
		case RouteId_SubsequentAdded:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			TaskTrace::FId SubsequentId = EventData.GetValue<TaskTrace::FId>("SubsequentId");
			TasksProvider.SubsequentAdded(TaskId, SubsequentId, Timestamp, ThreadId);
			break;
		}
		case RouteId_Started:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			TasksProvider.TaskStarted(TaskId, Timestamp, ThreadId);
			break;
		}
		case RouteId_NestedAdded:
		{
			static bool bLogged = false;
			if (!bLogged)
			{
				UE_LOGF(LogTraceServices, Log, "An old TaskTrace format detected. Nested tasks will be ignored");
				bLogged = true;
			}
			break;
		}
		case RouteId_Finished:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			TasksProvider.TaskFinished(TaskId, Timestamp);
			break;
		}
		case RouteId_Completed:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			TasksProvider.TaskCompleted(TaskId, Timestamp, ThreadId);
			break;
		}
		case RouteId_Destroyed:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TaskTrace::FId TaskId = EventData.GetValue<TaskTrace::FId>("TaskId");
			TasksProvider.TaskDestroyed(TaskId, Timestamp, ThreadId);
			break;
		}
		case RouteId_WaitingStarted:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			const TArrayReader<TaskTrace::FId>& Tasks = EventData.GetArray<TaskTrace::FId>("Tasks");
			TasksProvider.WaitingStarted({ Tasks.GetData(), static_cast<int32>(Tasks.Num()) }, Timestamp, ThreadId);
			break;
		}
		case RouteId_WaitingFinished:
		{
			double Timestamp = Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			TasksProvider.WaitingFinished(Timestamp, ThreadId);
			break;
		}
	}

	return true;
}

} // namespace TraceServices
