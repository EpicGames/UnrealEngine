// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices
{
	class IAnalysisSession;
	class FTasksProvider;

	// deserialises tasks traces and feeds them to FTasksProvider
	class FTasksAnalyzer : public UE::Trace::IAnalyzer
	{
	public:
		FTasksAnalyzer(IAnalysisSession& Session, FTasksProvider& TasksProvider);
		virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
		virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

	private:
		enum : uint16
		{
			RouteId_Init,
			RouteId_Created,
			RouteId_Launched,
			RouteId_Scheduled,
			RouteId_SubsequentAdded,
			RouteId_Started,
			// `NestedAdded` was removed from `TaskTrace` but left here to avoid crashing TasksInsights when used with an old trace file
			RouteId_NestedAdded,
			RouteId_Finished,
			RouteId_Completed,
			RouteId_Destroyed,

			RouteId_WaitingStarted,
			RouteId_WaitingFinished,
		};

		IAnalysisSession& Session;
		FTasksProvider& TasksProvider;
	};
}
