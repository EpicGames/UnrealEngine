// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrimAnalyzer.h"

#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformTime.h"
#include "Templates/UniquePtr.h"

// TraceAnalysis
#include "Trace/Analysis.h"
#include "Trace/TraceWriter.h"

// TraceServices
#include "Common/Utils.h"
#include "TraceServices/TraceTrimmer.h"

#define TRIMMER_USE_THREAD_TABLE 0

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LogTraceTrimmerError(Format, ...) \
	{ \
		++NumErrors; \
		printf("Error: " Format "\n", ##__VA_ARGS__); \
		UE_LOGF(LogTraceServices, Error, "[TraceTrimmer] " Format, ##__VA_ARGS__); \
	}
#define LogTraceTrimmerWarning(Format, ...) \
	{ \
		++NumWarnings; \
		printf("Warning: " Format "\n", ##__VA_ARGS__); \
		UE_LOGF(LogTraceServices, Warning, "[TraceTrimmer] " Format, ##__VA_ARGS__); \
	}
#define LogTraceTrimmerMessage(Format, ...) \
	{ \
		printf(Format "\n", ##__VA_ARGS__); \
		UE_LOGF(LogTraceServices, Log, "[TraceTrimmer] " Format, ##__VA_ARGS__); \
	}

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTrimAnalyzer
////////////////////////////////////////////////////////////////////////////////////////////////////

FTrimAnalyzer::FTrimAnalyzer(FTrimParameters& InParameters, UE::Trace::FTraceWriter& InTraceWriter)
	: Parameters(InParameters)
	, TraceWriter(InTraceWriter)
{
#if TRIMMER_USE_THREAD_TABLE
	ThreadTable.AddDefaulted(ThreadTableInitialSize);
#endif

	KnownEvents.SetOnUpdateThreadTime(
		[this]
		(uint32 ThreadId, const FOnEventContext& Context, uint64 Cycle)
		{
			FThreadInfo& Thread = GetOrAddThread(ThreadId);
			UpdateThreadTime(Thread, Context, Cycle);
		});

	// Empty IncludeFilter is equivalent with "*.*" (include all).
	if (!Parameters.Include.IsEmpty() &&
		!Parameters.Include.Equals("*.*"))
	{
		InitNameFilter(Parameters.Include, IncludeFilter);
	}

	if (!Parameters.Exclude.IsEmpty())
	{
		InitNameFilter(Parameters.Exclude, ExcludeFilter);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTrimAnalyzer::~FTrimAnalyzer()
{
	for (FThreadInfo* Thread : SortedThreads)
	{
		delete Thread;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::InitNameFilter(const FAnsiString& InFilter, FNameFilter& OutNameFilter)
{
	TArray<FAnsiString> FilterPatterns;
	InFilter.ParseIntoArray(FilterPatterns, ",", true);

	for (FAnsiString& Pattern : FilterPatterns)
	{
		TArray<FAnsiString> Names;
		Pattern.ParseIntoArray(Names, ".", true);

		if (Names.Num() == 2)
		{
			OutNameFilter.Patterns.Add({ Names[0], Names[1] });
		}
		else
		{
			LogTraceTrimmerWarning("Ignoring malformed event name pattern (%s) in filter (%s)!", *Pattern, *InFilter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	LogTraceTrimmerMessage("Trimming...");

	AnalysisBeginTimestamp = FPlatformTime::Cycles64();
	LastProgressMessageTimestamp = AnalysisBeginTimestamp;
	ProgressMessageInterval = FPlatformTime::SecondsToCycles64(10.0);

	Context.InterfaceBuilder.RouteAllEvents(0, false);
	Context.InterfaceBuilder.RouteAllEvents(1, true);

	KnownEvents.OnAnalysisBegin(Context);

	TraceWriter.Begin(UE::Trace::ETraceWriterBeginOptions::DeclareDefaultEvents);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FAnsiString FormatEventCount(uint64 Count)
{
	if (Count < 1000ull)
	{
		return FAnsiString::Printf("%llu", Count);
	}
	if (Count < 1000000ull)
	{
		return FAnsiString::Printf("%lluK", Count / 1000ull);
	}
	if (Count < 1000000000ull)
	{
		return FAnsiString::Printf("%lluM", Count / 1000000ull);
	}
	return FAnsiString::Printf("%lluG", Count / 1000000000ull);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FAnsiString FormatDuration(double Duration)
{
	if (Duration < 1.0)
	{
		return FAnsiString::Printf("%.06f seconds", Duration);
	}

	if (Duration < 60.0)
	{
		return FAnsiString::Printf("%.02f seconds", Duration);
	}

	TAnsiStringBuilder<256> Str;

	Str.Appendf("%.02fs (", Duration);

	bool bIsFirst = true;
	int64 Seconds = (int64)Duration;

	auto AppendDuration =
		[&Str, &bIsFirst, &Seconds]
		(int64 UnitDuration, const ANSICHAR * Unit)
		{
			if (Seconds >= UnitDuration)
			{
				if (!bIsFirst)
				{
					Str.Append(" ");
				}
				bIsFirst = false;

				int64 Units = Seconds / UnitDuration;
				Seconds = Seconds % UnitDuration;
				Str.Appendf("%lli %s", Units, Unit);
				if (Units != 1)
				{
					Str.Append("s");
				}
			}
		};

	AppendDuration(24 * 60 * 60, "day");
	AppendDuration(60 * 60, "hour");
	AppendDuration(60, "minute");
	AppendDuration(1, "second");

	Str.Append(")");

	return FAnsiString(Str.ToView());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::OnAnalysisEnd()
{
	KnownEvents.OnAnalysisEnd();

	if (bIsTraceNewTraceEventDetected)
	{
		TraceWriter.End();
	}
	else
	{
		LogTraceTrimmerError("Invalid trace ($Trace.NewTrace not detected)!");
	}

	LogTraceTrimmerMessage("Read %llu events. Written %llu events.", ReadEvents, WrittenEvents);
	LogTraceTrimmerMessage("Analyzed Trace Session Duration: %s", *FormatDuration(SessionCurrentTime));

	const uint64 AnalysisDuration = FPlatformTime::Cycles64() - AnalysisBeginTimestamp;
	const double Duration = double(AnalysisDuration) * FPlatformTime::GetSecondsPerCycle();
	LogTraceTrimmerMessage("Trimming completed in %s.", *FormatDuration(Duration));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::OnVersion(uint32 InTransportVersion, uint32 InProtocolVersion)
{
	TransportVersion = InTransportVersion;
	ProtocolVersion = InProtocolVersion;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTrimAnalyzer::FilterByEventName(const ANSICHAR* LoggerName, const ANSICHAR* EventName)
{
	// Include only the specified loggers and events.
	if (!IncludeFilter.IsEmpty())
	{
		bool bIsIncluded = false;

		FAnsiString LoggerNameStr(LoggerName);
		FAnsiString EventNameStr(EventName);

		for (const FNameFilterPattern& FilterPattern : IncludeFilter.Patterns)
		{
			if (LoggerNameStr.MatchesWildcard(FilterPattern.LoggerName) &&
				EventNameStr.MatchesWildcard(FilterPattern.EventName))
			{
				bIsIncluded = true;
				break;
			}
		}

		if (!bIsIncluded)
		{
			return false;
		}
	}

	// Exclude the specified loggers and events.
	if (!ExcludeFilter.IsEmpty())
	{
		bool bIsExcluded = false;

		FAnsiString LoggerNameStr(LoggerName);
		FAnsiString EventNameStr(EventName);

		for (const FNameFilterPattern& FilterPattern : ExcludeFilter.Patterns)
		{
			if (LoggerNameStr.MatchesWildcard(FilterPattern.LoggerName) &&
				EventNameStr.MatchesWildcard(FilterPattern.EventName))
			{
				bIsExcluded = true;
				break;
			}
		}

		if (bIsExcluded)
		{
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTrimAnalyzer::OnNewEvent(uint16 RouteId, const FEventTypeInfo& TypeInfo)
{
	const uint32 EventId = TypeInfo.GetId();

	const ANSICHAR* LoggerName = TypeInfo.GetLoggerName();
	check(LoggerName != nullptr);

	const ANSICHAR* EventName = TypeInfo.GetName();
	check(EventName != nullptr);

	//////////////////////////////////////////////////

	FEventInfo* EventInfo = EventInfos.Find(EventId);

	// Events are not expected to be declared multiple times.
	if (EventInfo)
	{
		if (EventInfo->TypeInfo->GetId() != EventId)
		{
			LogTraceTrimmerWarning("Event (%s.%s) with id %u is already declared with id %u!",
				LoggerName, EventName, EventId, EventInfo->TypeInfo->GetId());

			EventInfo = nullptr;
		}
		else
		{
			LogTraceTrimmerWarning("Event (%s.%s) is already declared!", LoggerName, EventName);
			return true;
		}
	}

	// Register the new event.
	check(!EventInfo);
	EventInfo = &EventInfos.Add(EventId, { &TypeInfo, InvalidWriterEventId, false });

	const UE::Trace::FTraceWriterEventInfo* WriterEventInfo = TraceWriter.FindEvent(LoggerName, EventName);
	if (WriterEventInfo)
	{
		// The event was already declared by trace writer.
		EventInfo->WriterEventId = WriterEventInfo->GetId();
	}

	// Pre-filter event by name.
	check(EventInfo);
	EventInfo->bIsFilteredOut = !FilterByEventName(LoggerName, EventName);

	//////////////////////////////////////////////////

	KnownEvents.OnNewEvent(TypeInfo);

	//////////////////////////////////////////////////
	// Known $Trace.* events

	static constexpr FAnsiStringView TraceLoggerName = "$Trace";
	static constexpr FAnsiStringView TraceNewTraceEventName = "NewTrace";
	static constexpr FAnsiStringView TraceThreadInfoEventName = "ThreadInfo";

	// Is it a $Trace.* event?
	if (TraceLoggerName.Equals(LoggerName))
	{
		if (TraceNewTraceEventName.Equals(EventName))
		{
			// $Trace.NewTrace
			TraceNewTraceEventId = EventId;
		}
		else if (TraceThreadInfoEventName.Equals(EventName))
		{
			// $Trace.ThreadInfo
			TraceThreadInfoEventId = EventId;
		}

		// $Trace.* events are expected to be declared automatically (by TraceWriter.Begin() or by TraceWriter.DeclareDefaultEvents()).
		if (!WriterEventInfo)
		{
			LogTraceTrimmerWarning("Unknown default event (%s.%s)!", LoggerName, EventName);
		}
	}

	//////////////////////////////////////////////////
	// Filtering

	if (EventInfo->bIsFilteredOut)
	{
		// The event is filtered out by name.
		return true;
	}

	//////////////////////////////////////////////////
	// Declare event.

	if (WriterEventInfo)
	{
		LogTraceTrimmerMessage("Event %u (%s.%s) was already declared (writer id %u).",
			EventId, LoggerName, EventName, WriterEventInfo->GetId());
	}
	else
	{
		const UE::Trace::ETraceWriterEventFlags WriterEventFlags = UE::Trace::FTraceWriter::ConvertEventFlags(TypeInfo.GetFlags());
		auto& Builder = TraceWriter.DeclareEvent(LoggerName, EventName, WriterEventFlags);
		for (int FieldIndex = 0, FieldCount = TypeInfo.GetFieldCount(); FieldIndex < FieldCount; ++FieldIndex)
		{
			const FEventFieldInfo& FieldInfo = *(TypeInfo.GetFieldInfo(FieldIndex));
			UE::Trace::ETraceWriterFieldType WriterFieldType = UE::Trace::FTraceWriter::ConvertFieldType(FieldInfo);
			if (!EnumHasAnyFlags(WriterFieldType, UE::Trace::ETraceWriterFieldType::ReferenceFlag | UE::Trace::ETraceWriterFieldType::DefinitionIdFlag))
			{
				// Normal field
				Builder.Field(FieldInfo.GetName(), WriterFieldType);
			}
			else if (EnumHasAnyFlags(WriterFieldType, UE::Trace::ETraceWriterFieldType::ReferenceFlag))
			{
				// Reference field
				const uint32 RefUid = FieldInfo.GetRefUid();
				FEventInfo* RefEventInfo = EventInfos.Find(RefUid);
				uint16 WriterRefUid;
				if (RefEventInfo && RefEventInfo->WriterEventId != InvalidEventId)
				{
					WriterRefUid = uint16(RefEventInfo->WriterEventId);
				}
				else
				{
					WriterRefUid = 0;
					LogTraceTrimmerWarning("Invalid RefUid (%u) for event (%s.%s) field \"%s\"!",
						RefUid, LoggerName, EventName, FieldInfo.GetName());
				}
				Builder.ReferenceField(FieldInfo.GetName(), WriterFieldType, WriterRefUid);
			}
			else // if (EnumHasAnyFlags(WriterFieldType, UE::Trace::ETraceWriterFieldType::DefinitionIdFlag))
			{
				// DefinitionId field
				Builder.DefinitionIdField(WriterFieldType);
			}
		}
		EventInfo->WriterEventId = Builder.End();
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTrimAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;

	const uint32 CurrentThreadId = Context.ThreadInfo.GetId();
	FTrimAnalyzer::FThreadInfo& CurrentThread = GetOrAddThread(CurrentThreadId);

	const FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();

	const uint32 EventId = TypeInfo.GetId();
	check(EventId != InvalidEventId);

	const ANSICHAR* LoggerName = TypeInfo.GetLoggerName();
	check(LoggerName != nullptr);

	const ANSICHAR* EventName = TypeInfo.GetName();
	check(EventName != nullptr);

	++ReadEvents;

	//////////////////////////////////////////////////

	bool bShouldWriteEvent = true;

	FEventInfo* EventInfo = EventInfos.Find(EventId);
	if (!EventInfo)
	{
		if (LoggerName[0] == 0) // empty dispatch, expected on LeaveScope events
		{
			EventInfo = &EmptyEventInfo;
		}
		else
		{
			LogTraceTrimmerWarning("Event %u (%s.%s) is not registered!", EventId, LoggerName, EventName);
			const bool bIsFilteredOut = !FilterByEventName(LoggerName, EventName);
			EventInfo = &EventInfos.Add(EventId, { &TypeInfo, InvalidWriterEventId, bIsFilteredOut });
			bShouldWriteEvent = false;
		}
	}
	check(EventInfo);

	//////////////////////////////////////////////////

	KnownEvents.OnEvent(EventId, Context);

	//////////////////////////////////////////////////
	// Known $Trace.* events

	// $Trace.NewTrace
	if (EventId == TraceNewTraceEventId)
	{
		// $Trace.NewTrace should be encountered only once
		check(bIsTraceNewTraceEventDetected == false);
		bIsTraceNewTraceEventDetected = true;

		SessionCycleFrequency = Context.EventData.GetValue<uint64>("CycleFrequency");
		SessionStartCycle = Context.EventData.GetValue<uint64>("StartCycle");
		SessionStartTimeSinceEpoch = Context.EventData.GetValue<double>("StartDateTime");

		SessionCurrentCycle = SessionStartCycle;
		SessionCurrentTime = 0.0;

		auto TimeGetter = [this]() { return SessionCurrentCycle; };
		TraceWriter.SetCustomClock(SessionCycleFrequency, TimeGetter, SessionStartCycle, SessionStartTimeSinceEpoch);
		TraceWriter.WriteNewTraceEvent();
		bShouldWriteEvent = false;
	}
	else
	{
		// $Trace.NewTrace should be declared and emitted before any other event
		check(bIsTraceNewTraceEventDetected == true);
	}

	// $Trace.ThreadInfo
	if (EventId == TraceThreadInfoEventId)
	{
		uint32 ThreadId = Context.EventData.GetValue<uint32>("ThreadId");
		uint32 SystemId = Context.EventData.GetValue<uint32>("SystemId");
		int32 SortHint = Context.EventData.GetValue<int32>("SortHint");
		FAnsiStringView ThreadName;
		Context.EventData.GetString("Name", ThreadName);

		const UE::Trace::FTraceWriterThreadInfo* ThreadInfo = TraceWriter.GetThreadInfo(ThreadId);
		if (ThreadInfo != nullptr)
		{
			if (!ThreadName.Equals(ThreadInfo->GetName(), ESearchCase::CaseSensitive) ||
				SystemId != ThreadInfo->GetSystemId() ||
				SortHint != ThreadInfo->GetSortHint())
			{
				LogTraceTrimmerWarning("Thread %u is already registered with different properties (previous Name=\"%s\", SystemId=%u, SortHint=%d; new Name=\"%s\", SystemId=%u, SortHint=%d)!",
					ThreadId,
					*ThreadInfo->GetName(), ThreadInfo->GetSystemId(), ThreadInfo->GetSortHint(),
					*FAnsiString(ThreadName), SystemId, SortHint);
			}
			else
			{
				LogTraceTrimmerMessage("Thread %u is already registered (Name=\"%s\", SystemId=%u, SortHint=%d).",
					ThreadId,
					*FAnsiString(ThreadName), SystemId, SortHint);
			}
			//bShouldWriteEvent = false;
		}
		else
		{
			LogTraceTrimmerMessage("Registering thread %u (Name=\"%s\", SystemId=%u, SortHint=%d).",
				ThreadId,
				*FAnsiString(ThreadName), SystemId, SortHint);

			constexpr bool bShouldWriteThreadInfo = false;
			TraceWriter.RegisterCustomThread(ThreadId, ThreadName, SystemId, SortHint, bShouldWriteThreadInfo);
			//TraceWriter.WriteThreadInfo(ThreadId);
			//bShouldWriteEvent = false;
		}
	}

	//////////////////////////////////////////////////
	// Filters

	if (EventInfo->bIsFilteredOut)
	{
		// Event is filtered out by name.
		bShouldWriteEvent = false;
	}

	if (bShouldWriteEvent &&
		CurrentThreadId != 1 &&
		(CurrentThread.CurrentTime < Parameters.StartTime ||
		 CurrentThread.CurrentTime > Parameters.EndTime))
	{
		// Non-important event is filtered out by time range.
		bShouldWriteEvent = false;
	}

	if (bShouldWriteEvent)
	{
		if (EventInfo->WriterEventId == InvalidWriterEventId)
		{
			LogTraceTrimmerWarning("Event %u (%s.%s) is not declared!", EventId, LoggerName, EventName);
			bShouldWriteEvent = false;
		}
	}

	//////////////////////////////////////////////////

	if (RouteId == 0) // Normal events
	{
		if (Style != EStyle::Normal)
		{
			LogTraceTrimmerWarning("Event (%s.%s) is dispatched as normal event, but with Style=%u!", LoggerName, EventName, (uint32)Style);
		}
	}
	else if (RouteId == 1) // Scoped events
	{
		if (Style == EStyle::EnterScope)
		{
			if (bShouldWriteEvent)
			{
				SetCurrentThread(CurrentThreadId);

				uint64 Timestamp = Context.EventTime.GetTimestamp();
				if (Timestamp != 0)
				{
					TraceWriter.WriteStampedEnterScopeEvent(SessionStartCycle + Timestamp);
				}
				else
				{
					TraceWriter.WriteEnterScopeEvent();
				}
			}
		}
		else if (Style == EStyle::LeaveScope)
		{
			if (bShouldWriteEvent)
			{
				SetCurrentThread(CurrentThreadId);

				uint64 Timestamp = Context.EventTime.GetTimestamp();
				if (Timestamp != 0)
				{
					TraceWriter.WriteStampedLeaveScopeEvent(SessionStartCycle + Timestamp);
				}
				else
				{
					TraceWriter.WriteLeaveScopeEvent();
				}

				bShouldWriteEvent = false;
			}
		}
		else // unknown Style
		{
			LogTraceTrimmerWarning("Event (%s.%s) is dispatched as scoped event, but with Style=%u!", LoggerName, EventName, (uint32)Style);
		}
	}
	else // unknown RouteId
	{
		LogTraceTrimmerWarning("Event (%s.%s) is dispatched on unknown route %u!", LoggerName, EventName, (uint32)RouteId);
	}

	//////////////////////////////////////////////////
	// Write event

	if (bShouldWriteEvent)
	{
		SetCurrentThread(CurrentThreadId);

		check(EventInfo);
		check(EventInfo->WriterEventId != InvalidWriterEventId);
		UE::Trace::FTraceWriterEventBuilder& EventBuilder = TraceWriter.WriteEvent(EventInfo->WriterEventId);

		const uint32 FieldCount = TypeInfo.GetFieldCount();
		for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
		{
			const FEventFieldInfo& FieldInfo = *(TypeInfo.GetFieldInfo(FieldIndex));
			if (FieldInfo.IsArray())
			{
				WriteArrayField(FieldIndex, FieldInfo, EventData, EventBuilder);
			}
			else
			{
				WriteScalarField(FieldIndex, FieldInfo, EventData, EventBuilder);
			}
		}

		EventBuilder.End();
		++WrittenEvents;
	}

	//////////////////////////////////////////////////

	bool bShouldStopAnalysis = false;

	if (TypeInfo.IsSync())
	{
		bShouldStopAnalysis = UpdateThreadsOnSyncEvent(CurrentThread);
	}

	//////////////////////////////////////////////////

	const uint64 CurrentTimestamp = FPlatformTime::Cycles64();
	if (CurrentTimestamp - LastProgressMessageTimestamp > ProgressMessageInterval)
	{
		LastProgressMessageTimestamp = CurrentTimestamp;
		const double TT = double(LastProgressMessageTimestamp - AnalysisBeginTimestamp) * FPlatformTime::GetSecondsPerCycle();
		LogTraceTrimmerMessage("Trimming...   R: %-6s W: %-6s S: %.0fs   T: %.0fs",
			*FormatEventCount(ReadEvents), *FormatEventCount(WrittenEvents), SessionCurrentTime, TT);
	}

	return !bShouldStopAnalysis;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::WriteArrayField(
	uint32 FieldIndex,
	const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo,
	const UE::Trace::IAnalyzer::FEventData& EventData,
	UE::Trace::FTraceWriterEventBuilder& EventBuilder)
{
	switch (FieldInfo.GetType())
	{
		case FEventFieldInfo::EType::Integer:
		{
			if (FieldInfo.IsSigned())
			{
				switch (FieldInfo.GetTypeSize())
				{
					case 1: // int8
					{
						const TArrayReader<int8>& Reader = EventData.GetArray<int8>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<int8>(Reader.GetData(), Reader.Num()));
						break;
					}
					case 2: // int16
					{
						const TArrayReader<int16>& Reader = EventData.GetArray<int16>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<int16>(Reader.GetData(), Reader.Num()));
						break;
					}
					case 4: // int32
					{
						const TArrayReader<int32>& Reader = EventData.GetArray<int32>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<int32>(Reader.GetData(), Reader.Num()));
						break;
					}
					case 8: // int64
					{
						const TArrayReader<int64>& Reader = EventData.GetArray<int64>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<int64>(Reader.GetData(), Reader.Num()));
						break;
					}
					default:
					{
						// error
						LogTraceTrimmerWarning("Invalid array field type: unsupported signed integer size (%u)!", uint32(FieldInfo.GetTypeSize()));
					}
				} // switch (FieldInfo.GetTypeSize())
			}
			else // unsigned
			{
				switch (FieldInfo.GetTypeSize())
				{
					case 1: // uint8
					{
						const TArrayReader<uint8>& Reader = EventData.GetArray<uint8>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<uint8>(Reader.GetData(), Reader.Num()));
						break;
					}
					case 2: // uint16
					{
						const TArrayReader<uint16>& Reader = EventData.GetArray<uint16>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<uint16>(Reader.GetData(), Reader.Num()));
						break;
					}
					case 4: // uint32
					{
						const TArrayReader<uint32>& Reader = EventData.GetArray<uint32>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<uint32>(Reader.GetData(), Reader.Num()));
						break;
					}
					case 8: // uint64
					{
						const TArrayReader<uint64>& Reader = EventData.GetArray<uint64>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, TConstArrayView<uint64>(Reader.GetData(), Reader.Num()));
						break;
					}
					default:
					{
						// error
						LogTraceTrimmerWarning("Invalid array field type: unsupported unsigned integer size (%u)!", uint32(FieldInfo.GetTypeSize()));
					}
				} // switch (FieldInfo.GetTypeSize())
			}
			break;
		}

		case FEventFieldInfo::EType::Float: // Float32 or Float64
		{
			switch (FieldInfo.GetTypeSize())
			{
				case 4: // float
				{
					const TArrayReader<float>& Reader = EventData.GetArray<float>(FieldInfo.GetName());
					EventBuilder.Field(FieldIndex, TConstArrayView<float>(Reader.GetData(), Reader.Num()));
					break;
				}
				case 8: // double
				{
					const TArrayReader<double>& Reader = EventData.GetArray<double>(FieldInfo.GetName());
					EventBuilder.Field(FieldIndex, TConstArrayView<double>(Reader.GetData(), Reader.Num()));
					break;
				}
				default:
				{
					// error
					LogTraceTrimmerWarning("Invalid array field type: unsupported float size (%u)!", uint32(FieldInfo.GetTypeSize()));
				}
			} // switch (FieldInfo.GetTypeSize())
			break;
		}

		case FEventFieldInfo::EType::AnsiString:
		{
			FAnsiStringView Value;
			EventData.GetString(FieldInfo.GetName(), Value);
			EventBuilder.Field(FieldIndex, Value);
			break;
		}

		case FEventFieldInfo::EType::WideString:
		{
			FWideStringView Value;
			EventData.GetString(FieldInfo.GetName(), Value);
			EventBuilder.Field(FieldIndex, Value);
			break;
		}

		default:
		{
			// error
			LogTraceTrimmerWarning("Invalid array field type (%u)!", uint32(FieldInfo.GetType()));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::WriteScalarField(
	uint32 FieldIndex,
	const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo,
	const UE::Trace::IAnalyzer::FEventData& EventData,
	UE::Trace::FTraceWriterEventBuilder& EventBuilder)
{
	switch (FieldInfo.GetType())
	{
		case FEventFieldInfo::EType::Integer:
		{
			if (FieldInfo.IsSigned())
			{
				switch (FieldInfo.GetTypeSize())
				{
					case 1: // int8
					{
						const int8 Value = EventData.GetValue<int8>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					case 2: // int16
					{
						const int16 Value = EventData.GetValue<int16>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					case 4: // int32
					{
						const int32 Value = EventData.GetValue<int32>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					case 8: // int64
					{
						const int64 Value = EventData.GetValue<int64>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					default:
					{
						// error
						LogTraceTrimmerWarning("Invalid scalar field type: unsupported signed integer size (%u)!", uint32(FieldInfo.GetTypeSize()));
					}
				} // switch (FieldInfo.GetTypeSize())
			}
			else // unsigned
			{
				switch (FieldInfo.GetTypeSize())
				{
					case 1: // uint8
					{
						const uint8 Value = EventData.GetValue<uint8>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					case 2: // uint16
					{
						const uint16 Value = EventData.GetValue<uint16>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					case 4: // uint32
					{
						const uint32 Value = EventData.GetValue<uint32>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					case 8: // uint64
					{
						const uint64 Value = EventData.GetValue<uint64>(FieldInfo.GetName());
						EventBuilder.Field(FieldIndex, Value);
						break;
					}
					default:
					{
						// error
						LogTraceTrimmerWarning("Invalid scalar field type: unsupported unsigned integer size (%u)!", uint32(FieldInfo.GetTypeSize()));
					}
				} // switch (FieldInfo.GetTypeSize())
			}
			break;
		}

		case FEventFieldInfo::EType::Float: // Float32 or Float64
		{
			switch (FieldInfo.GetTypeSize())
			{
				case 4: // float
				{
					const float Value = EventData.GetValue<float>(FieldInfo.GetName());
					EventBuilder.Field(FieldIndex, Value);
					break;
				}
				case 8: // double
				{
					const double Value = EventData.GetValue<double>(FieldInfo.GetName());
					EventBuilder.Field(FieldIndex, Value);
					break;
				}
				default:
				{
					// error
					LogTraceTrimmerWarning("Invalid scalar field type: unsupported float size (%u)!", uint32(FieldInfo.GetTypeSize()));
				}
			} // switch (FieldInfo.GetTypeSize())
			break;
		}

		case FEventFieldInfo::EType::Reference8:
		{
			check(FieldInfo.GetSize() == 1);
			UE::Trace::FEventRef8 RefValue = EventData.GetReferenceValue<uint8>(FieldIndex);
			EventBuilder.Field(FieldIndex, RefValue);
			break;
		}

		case FEventFieldInfo::EType::Reference16:
		{
			check(FieldInfo.GetSize() == 2);
			UE::Trace::FEventRef16 RefValue = EventData.GetReferenceValue<uint16>(FieldIndex);
			EventBuilder.Field(FieldIndex, RefValue);
			break;
		}

		case FEventFieldInfo::EType::Reference32:
		{
			check(FieldInfo.GetSize() == 4);
			UE::Trace::FEventRef32 RefValue = EventData.GetReferenceValue<uint32>(FieldIndex);
			EventBuilder.Field(FieldIndex, RefValue);
			break;
		}

		case FEventFieldInfo::EType::Reference64:
		{
			check(FieldInfo.GetSize() == 8);
			UE::Trace::FEventRef64 RefValue = EventData.GetReferenceValue<uint64>(FieldIndex);
			EventBuilder.Field(FieldIndex, RefValue);
			break;
		}

		default:
		{
			// error
			LogTraceTrimmerWarning("Invalid scalar field type (%u)!", uint32(FieldInfo.GetType()));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::SetCurrentThread(uint32 ThreadId)
{
	const UE::Trace::FTraceWriterThreadInfo* ThreadInfo = TraceWriter.GetThreadInfo(ThreadId);
	if (!ThreadInfo)
	{
		LogTraceTrimmerMessage("Registering thread %u (unknown).", ThreadId);
		constexpr bool bShouldWriteThreadInfo = false;
		TraceWriter.RegisterCustomThread(ThreadId, FAnsiStringView("Unknown"), 0, 0, bShouldWriteThreadInfo);
	}
	TraceWriter.SetCurrentThread(ThreadId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTrimAnalyzer::FThreadInfo& FTrimAnalyzer::GetOrAddThread(uint32 ThreadId)
{
#if TRIMMER_USE_THREAD_TABLE
	// Try the fast access table.
	if (ThreadId < uint32(ThreadTable.Num()))
	{
		FThreadInfo* Thread = ThreadTable[ThreadId];
		if (Thread)
		{
			return *Thread;
		}
	}
#endif // TRIMMER_USE_THREAD_TABLE

	// Try the thread map.
	FThreadInfo** FoundThread = ThreadMap.Find(ThreadId);
	if (FoundThread)
	{
		return **FoundThread;
	}

	// Create a new thread info.
	FThreadInfo* Thread = new FThreadInfo();
	Thread->Id = ThreadId;

	// Add the thread to the map.
	ThreadMap.Add(ThreadId, Thread);

#if TRIMMER_USE_THREAD_TABLE
	// Add the thread to the fast access table.
	if (ThreadId < ThreadTableMaxSize)
	{
		if (ThreadId >= uint32(ThreadTable.Num()))
		{
			ThreadTable.AddDefaulted(int32(ThreadId) - ThreadTable.Num() + 1);
		}
		ThreadTable[ThreadId] = Thread;
	}
#endif // TRIMMER_USE_THREAD_TABLE

	// Add the thread to the sorting table.
	Thread->SortIndex = SortedThreads.Num();
	SortedThreads.Add(Thread);
	SortThreads(Thread->SortIndex);

	return *Thread;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::SortThreads(int32 ThreadIndex)
{
	const uint64 Cycle = SortedThreads[ThreadIndex]->CurrentCycle;

	int32 NextIndex = ThreadIndex + 1;
	while (NextIndex < SortedThreads.Num() &&
		Cycle > SortedThreads[NextIndex]->CurrentCycle)
	{
		FThreadInfo* T = SortedThreads[ThreadIndex];
		SortedThreads[ThreadIndex] = SortedThreads[NextIndex];
		SortedThreads[NextIndex] = T;
		T->SortIndex = NextIndex++;
		SortedThreads[ThreadIndex]->SortIndex = ThreadIndex++;
	}

	int32 PrevIndex = ThreadIndex - 1;
	while (PrevIndex >= 0 &&
		Cycle < SortedThreads[PrevIndex]->CurrentCycle)
	{
		FThreadInfo* T = SortedThreads[ThreadIndex];
		SortedThreads[ThreadIndex] = SortedThreads[PrevIndex];
		SortedThreads[PrevIndex] = T;
		T->SortIndex = PrevIndex--;
		SortedThreads[ThreadIndex]->SortIndex = ThreadIndex--;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTrimAnalyzer::UpdateThreadsOnSyncEvent(FThreadInfo& CurrentThread)
{
	// When we encounter a Sync event...

	check(SortedThreads.Num() > 0); // as CurrentThread exists, SortedThreads should contain at least one thread

	// Threads with old timestamps can be updated with time of the CurrentThread.
	for (auto Thread : SortedThreads)
	{
		if (Thread->CurrentCycle < CurrentThread.CurrentCycle)
		{
			Thread->CurrentCycle = CurrentThread.CurrentCycle;
			Thread->CurrentTime = CurrentThread.CurrentTime;
		}
		else
		{
			break;
		}
	}

	// If all threads have timestamps past the EndTime limit we can stop analysis.
	return (SortedThreads[0]->CurrentTime > Parameters.EndTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrimAnalyzer::UpdateThreadTime(FThreadInfo& Thread, const FOnEventContext& Context, uint64 Cycle)
{
	if (Cycle > Thread.CurrentCycle)
	{
		const double Time = Context.EventTime.AsSeconds(Cycle);

		Thread.CurrentCycle = Cycle;
		Thread.CurrentTime = Time;

		SortThreads(Thread.SortIndex);

		if (Cycle > SessionCurrentCycle)
		{
			SessionCurrentCycle = Cycle;
			SessionCurrentTime = Time;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Async Trim Processor
////////////////////////////////////////////////////////////////////////////////////////////////////

TUniquePtr<ITrimAnalysisProcessor> TrimAsync(FTrimParameters& Parameters, UE::Trace::IInDataStream& InputDataStream, UE::Trace::IOutDataStream& OutputDataStream)
{
	class FTrimAnalysisProcessor final : public ITrimAnalysisProcessor
	{
	public:
		FTrimAnalysisProcessor(FTrimParameters& Parameters, UE::Trace::IInDataStream& InputDataStream, UE::Trace::IOutDataStream& OutputDataStream)
			: TraceWriter(OutputDataStream)
			, TrimAnalyzer(Parameters, TraceWriter)
		{
			Context.AddAnalyzer(TrimAnalyzer);
			Processor = Context.Process(InputDataStream);
		}

		virtual ~FTrimAnalysisProcessor()
		{
		}

		virtual bool IsActive() const override { return Processor.IsActive(); }
		virtual void Stop() override { Processor.Stop(); }
		virtual void Wait() override { Processor.Wait(); }
		virtual void Pause(bool bState) override { Processor.Pause(bState); }

	private:
		UE::Trace::FTraceWriter TraceWriter;
		FTrimAnalyzer TrimAnalyzer;
		UE::Trace::FAnalysisContext Context;
		UE::Trace::FAnalysisProcessor Processor;
	};

	return MakeUnique<FTrimAnalysisProcessor>(Parameters, InputDataStream, OutputDataStream);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sync Trim Processor
////////////////////////////////////////////////////////////////////////////////////////////////////

int32 Trim(FTrimParameters& Parameters, UE::Trace::IInDataStream& InputDataStream, UE::Trace::IOutDataStream& OutputDataStream)
{
	UE::Trace::FTraceWriter TraceWriter(OutputDataStream);
	FTrimAnalyzer TrimAnalyzer(Parameters, TraceWriter);
	UE::Trace::FAnalysisContext Context;
	Context.AddAnalyzer(TrimAnalyzer);
	Context.Process(InputDataStream).Wait();
	return (int32)TrimAnalyzer.GetNumErrors();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef LogTraceTrimmerError
#undef LogTraceTrimmerWarning
#undef LogTraceTrimmerMessage

#undef TRIMMER_USE_THREAD_TABLE
