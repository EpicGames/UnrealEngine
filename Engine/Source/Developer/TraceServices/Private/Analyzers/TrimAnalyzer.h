// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Map.h"
#include "Containers/AnsiString.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"

// TraceAnalysis
#include "Trace/Analyzer.h"

// TraceServices
#include "Analyzers/KnownEventProcessor.h"

namespace UE::Trace
{
	class FTraceWriter;
	class FTraceWriterEventBuilder;
}

namespace TraceServices
{

struct FTrimParameters;

class FTrimAnalyzer : public UE::Trace::IAnalyzer
{
private:
	struct FEventInfo
	{
		const FEventTypeInfo* TypeInfo = nullptr;
		uint32 WriterEventId = ~0u;
		bool bIsFilteredOut = false;
	};

	struct FNameFilterPattern
	{
		FAnsiString LoggerName;
		FAnsiString EventName;
	};

	struct FNameFilter
	{
		bool IsEmpty() const { return Patterns.IsEmpty(); }

		TArray<FNameFilterPattern> Patterns;
	};

	struct FThreadInfo
	{
		uint32 Id = 0;
		int32 SortIndex = 0;
		uint64 CurrentCycle = 0;
		double CurrentTime = 0.0;
	};

public:
	FTrimAnalyzer(FTrimParameters& InParameters, UE::Trace::FTraceWriter& InTraceWriter);
	virtual ~FTrimAnalyzer();

	uint32 GetNumErrors() const { return NumErrors; }
	uint32 GetNumWarnings() const { return NumWarnings; }

private:
	void InitNameFilter(const FAnsiString& InFilter, FNameFilter& OutNameFilter);
	bool FilterByEventName(const ANSICHAR* LoggerName, const ANSICHAR* EventName);

	virtual void OnAnalysisBegin(const UE::Trace::IAnalyzer::FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual void OnVersion(uint32 InTransportVersion, uint32 InProtocolVersion) override;
	virtual bool OnNewEvent(uint16 RouteId, const FEventTypeInfo& TypeInfo) override;
	virtual bool OnEvent(uint16 RouteId, EStyle, const UE::Trace::IAnalyzer::FOnEventContext& Context) override;

	void WriteArrayField(uint32 FieldIndex, const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo, const UE::Trace::IAnalyzer::FEventData& EventData, UE::Trace::FTraceWriterEventBuilder& EventBuilder);
	void WriteScalarField(uint32 FieldIndex, const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo, const UE::Trace::IAnalyzer::FEventData& EventData, UE::Trace::FTraceWriterEventBuilder& EventBuilder);

	FThreadInfo& GetOrAddThread(uint32 ThreadId);
	void SortThreads(int32 ThreadIndex);
	bool UpdateThreadsOnSyncEvent(FThreadInfo& CurrentThread);
	void SetCurrentThread(uint32 ThreadId);
	void UpdateThreadTime(FThreadInfo& Thread, const UE::Trace::IAnalyzer::FOnEventContext& Context, uint64 Cycle);

private:
	FTrimParameters& Parameters;
	UE::Trace::FTraceWriter& TraceWriter;

	uint32 TransportVersion = 4;
	uint32 ProtocolVersion = 7;

	static constexpr uint32 ThreadTableInitialSize = 1024;
	static constexpr uint32 ThreadTableMaxSize = 4 * 1024;

	TMap<uint32, FThreadInfo*> ThreadMap; // ThreadId --> FThreadInfo
	TArray<FThreadInfo*> ThreadTable; // index == ThreadId, for ThreadId < ThreadTableMaxSize
	TArray<FThreadInfo*> SortedThreads; // sorted threads by time

	TMap<uint32, FEventInfo> EventInfos; // EventId --> FEventInfo
	FEventInfo EmptyEventInfo;

	uint64 ReadEvents = 0;
	uint64 WrittenEvents = 0;

	uint64 LastProgressMessageTimestamp = 0;
	uint64 ProgressMessageInterval = 0;

	uint64 AnalysisBeginTimestamp = 0; // realtime timestamp

	uint64 SessionCycleFrequency = 0;
	uint64 SessionStartCycle = 0;
	double SessionStartTimeSinceEpoch = 0.0;

	uint64 SessionCurrentCycle = 0; // absolute session time
	double SessionCurrentTime = 0.0; // relative session time, in [seconds]

	FNameFilter IncludeFilter;
	FNameFilter ExcludeFilter;

	static constexpr uint32 InvalidEventId = ~0u;
	static constexpr uint32 InvalidWriterEventId = ~0u;

	uint32 TraceNewTraceEventId = InvalidEventId;
	bool bIsTraceNewTraceEventDetected = false;

	uint32 TraceThreadInfoEventId = InvalidEventId;

	FKnownEventProcessor KnownEvents;

	uint32 NumErrors = 0;
	uint32 NumWarnings = 0;
};

} // namespace TraceServices
