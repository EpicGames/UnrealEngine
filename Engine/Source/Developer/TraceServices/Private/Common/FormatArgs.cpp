// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/FormatArgs.h"
#include "ProfilingDebugging/FormatArgsTrace.h"
#include "CoreMinimal.h"

namespace TraceServices
{

const TCHAR* FFormatArgsHelper::ExtractNextFormatArg(const TCHAR* FormatString, FFormatArgSpec& Spec)
{
	Spec.PassthroughLength = 0;
	Spec.AdditionalIntegerArgumentCount = 0;
	Spec.Valid = false;
	Spec.NothingPrinted = false;

	enum EState
	{
		None,
		Flags,
		Width,
		PrecisionStart,
		Precision,
		Length,
		Specifier
	};
	EState CurrentState = None;
	const TCHAR* Src = FormatString;
	const TCHAR* FormatSpecifierStart = nullptr;
	while (*Src != 0)
	{
		switch (CurrentState)
		{
		case None:
			if (*Src == '%')
			{
				FormatSpecifierStart = Src;
				CurrentState = Flags;
			}
			++Src;
			break;
		case Flags:
			if (*Src == '%')
			{
				++Src;
				CurrentState = None;
				break;
			}
			if (*Src == '-' ||
				*Src == '+' ||
				*Src == ' ' ||
				*Src == '#' ||
				*Src == '0')
			{
				++Src;
				break;
			}
			CurrentState = Width;
			break;
		case Width:
			if (*Src == '*')
			{
				++Spec.AdditionalIntegerArgumentCount;
				++Src;
				CurrentState = PrecisionStart;
				break;
			}
			else if ('0' <= *Src && *Src <= '9')
			{
				++Src;
				break;
			}
			CurrentState = PrecisionStart;
			break;
		case PrecisionStart:
			if (*Src == '.')
			{
				++Src;
				CurrentState = Precision;
				break;
			}
			CurrentState = Length;
			break;
		case Precision:
			if (*Src == '*')
			{
				++Spec.AdditionalIntegerArgumentCount;
				++Src;
				CurrentState = Length;
				break;
			}
			else if ('0' <= *Src && *Src <= '9')
			{
				++Src;
				break;
			}
			CurrentState = Length;
			break;
		case Length:
			if (*Src == 'h' ||
				*Src == 'l' ||
				*Src == 'j' ||
				*Src == 'z' ||
				*Src == 't' ||
				*Src == 'L')
			{
				++Src;
				break;
			}
			CurrentState = Specifier;
			break;
		case Specifier:
			if (*Src == 'd' ||
				*Src == 'i' ||
				*Src == 'u' ||
				*Src == 'o' ||
				*Src == 'x' ||
				*Src == 'X' ||
				*Src == 'c' ||
				*Src == 'p')
			{
				Spec.ExpectedTypeCategory = FFormatArgsTrace::FormatArgTypeCode_CategoryInteger;
			}
			else if (*Src == 'f' ||
				*Src == 'F' ||
				*Src == 'e' ||
				*Src == 'E' ||
				*Src == 'g' ||
				*Src == 'G' ||
				*Src == 'a' ||
				*Src == 'A')
			{
				Spec.ExpectedTypeCategory = FFormatArgsTrace::FormatArgTypeCode_CategoryFloatingPoint;
			}
			else if (*Src == 's' || *Src == 'S')
			{
				Spec.ExpectedTypeCategory = FFormatArgsTrace::FormatArgTypeCode_CategoryString;
			}
			else if (*Src == 'n')
			{
				Spec.ExpectedTypeCategory = FFormatArgsTrace::FormatArgTypeCode_CategoryInteger;
				Spec.NothingPrinted = true;
			}
			else
			{
				CurrentState = None;
				break;
			}

			++Src;
			int32 FormatSpecifierLength = static_cast<int32>(Src + 1 - FormatSpecifierStart);
			check(FormatSpecifierLength < 255);
			FCString::Strncpy(Spec.FormatString, FormatSpecifierStart, FormatSpecifierLength);
			Spec.Valid = true;
			Spec.PassthroughLength = static_cast<int32>(FormatSpecifierStart - FormatString);
			return Src;
		}
	}
	Spec.PassthroughLength = static_cast<int32>(Src - FormatString);
	return Src;
}

void FFormatArgsHelper::InitArgumentStream(FFormatArgsStreamContext& Context, const uint8* ArgumentsData)
{
	if (ArgumentsData == nullptr)
	{
		Context.ArgumentTypeCategory = 0;
		Context.ArgumentTypeSize = 0;
		return;
	}

	Context.ArgumentCount = *ArgumentsData++;
	Context.DescriptorPtr = ArgumentsData;
	Context.PayloadPtr = ArgumentsData + Context.ArgumentCount;
	if (Context.ArgumentCount)
	{
		Context.ArgumentTypeCategory = *Context.DescriptorPtr & FFormatArgsTrace::FormatArgTypeCode_CategoryBitMask;
		Context.ArgumentTypeSize = *Context.DescriptorPtr & FFormatArgsTrace::FormatArgTypeCode_SizeBitMask;
	}
	else
	{
		Context.ArgumentTypeCategory = 0;
		Context.ArgumentTypeSize = 0;
	}
}

bool FFormatArgsHelper::AdvanceArgumentStream(FFormatArgsStreamContext& Context)
{
	if (Context.ArgumentTypeCategory == 0)
	{
		return false;
	}
	if (Context.ArgumentTypeCategory == FFormatArgsTrace::FormatArgTypeCode_CategoryString)
	{
		if (Context.ArgumentTypeSize == 1)
		{
			const uint8* StringPtr = reinterpret_cast<const uint8*>(Context.PayloadPtr);
			while (*StringPtr++);
			Context.PayloadPtr = StringPtr;
		}
		else if (Context.ArgumentTypeSize == 2)
		{
			const uint16* StringPtr = reinterpret_cast<const uint16*>(Context.PayloadPtr);
			while (*StringPtr++);
			Context.PayloadPtr = reinterpret_cast<const uint8*>(StringPtr);
		}
		else if (Context.ArgumentTypeSize == 4)
		{
			const uint32* StringPtr = reinterpret_cast<const uint32*>(Context.PayloadPtr);
			while (*StringPtr++);
			Context.PayloadPtr = reinterpret_cast<const uint8*>(StringPtr);
		}
		else
		{
			check(false);
		}
	}
	else
	{
		Context.PayloadPtr += Context.ArgumentTypeSize;
	}
	++Context.DescriptorPtr;
	Context.ArgumentTypeCategory = *Context.DescriptorPtr & FFormatArgsTrace::FormatArgTypeCode_CategoryBitMask;
	Context.ArgumentTypeSize = *Context.DescriptorPtr & FFormatArgsTrace::FormatArgTypeCode_SizeBitMask;
	--Context.ArgumentCount;
	return true;
}

uint64 FFormatArgsHelper::ExtractIntegerArgument(FFormatArgsStreamContext& ArgStream)
{
	uint64 Result = 0;
	if (ArgStream.ArgumentTypeCategory == FFormatArgsTrace::FormatArgTypeCode_CategoryInteger)
	{
		memcpy(&Result, ArgStream.PayloadPtr, ArgStream.ArgumentTypeSize);
	}
	AdvanceArgumentStream(ArgStream);
	return Result;
}

double FFormatArgsHelper::ExtractFloatingPointArgument(FFormatArgsStreamContext& ArgStream)
{
	double Result = 0.0;
	if (ArgStream.ArgumentTypeCategory == FFormatArgsTrace::FormatArgTypeCode_CategoryFloatingPoint)
	{
		if (ArgStream.ArgumentTypeSize == 4)
		{
			Result = *reinterpret_cast<const float*>(ArgStream.PayloadPtr);
		}
		else
		{
			check(ArgStream.ArgumentTypeSize == 8)
				Result = *reinterpret_cast<const double*>(ArgStream.PayloadPtr);
		}
	}
	AdvanceArgumentStream(ArgStream);
	return Result;
}

const TCHAR* FFormatArgsHelper::ExtractStringArgument(FFormatArgsStreamContext& ArgStream, TCHAR* Temp, int32 MaxTemp)
{
	static TCHAR Empty[] = TEXT("");
	const TCHAR* Result = Empty;
	if (ArgStream.ArgumentTypeCategory == FFormatArgsTrace::FormatArgTypeCode_CategoryString)
	{
		if (ArgStream.ArgumentTypeSize == sizeof(TCHAR))
		{
			Result = reinterpret_cast<const TCHAR*>(ArgStream.PayloadPtr);
		}
		else if (ArgStream.ArgumentTypeSize == sizeof(ANSICHAR))
		{
			const ANSICHAR* AnsiString = reinterpret_cast<const ANSICHAR*>(ArgStream.PayloadPtr);
			int32 SourceLength = TCString<ANSICHAR>::Strlen(AnsiString) + 1;
			TStringConvert<ANSICHAR, TCHAR> StringConvert;
			int32 ConvertedLength = StringConvert.ConvertedLength(AnsiString, SourceLength);
			if (ConvertedLength < MaxTemp)
			{
				StringConvert.Convert(Temp, MaxTemp, AnsiString, SourceLength);
				Result = Temp;
			}
		}
	}
	AdvanceArgumentStream(ArgStream);
	return Result;
}

int32 FFormatArgsHelper::FormatArgument(TCHAR* Out, int32 MaxOut, TCHAR* Temp, int32 MaxTemp, const FFormatArgSpec& ArgSpec, FFormatArgsStreamContext& ArgStream)
{
	check(ArgSpec.AdditionalIntegerArgumentCount <= 2);
	switch (ArgSpec.ExpectedTypeCategory)
	{
	case FFormatArgsTrace::FormatArgTypeCode_CategoryInteger:
		if (ArgSpec.NothingPrinted)
		{
			for (uint8 IntegerArgIndex = 0; IntegerArgIndex < ArgSpec.AdditionalIntegerArgumentCount + 1; ++IntegerArgIndex)
			{
				ExtractIntegerArgument(ArgStream);
			}
			return 0;
		}
		else
		{
			if (ArgSpec.AdditionalIntegerArgumentCount == 2)
			{
				return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream), ExtractIntegerArgument(ArgStream), ExtractIntegerArgument(ArgStream));
			}
			else if (ArgSpec.AdditionalIntegerArgumentCount == 1)
			{
				return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream), ExtractIntegerArgument(ArgStream));
			}
			else
			{
				return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream));
			}
		}
	case FFormatArgsTrace::FormatArgTypeCode_CategoryFloatingPoint:
		if (ArgSpec.AdditionalIntegerArgumentCount == 2)
		{
			return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream), ExtractIntegerArgument(ArgStream), ExtractFloatingPointArgument(ArgStream));
		}
		else if (ArgSpec.AdditionalIntegerArgumentCount == 1)
		{
			return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream), ExtractFloatingPointArgument(ArgStream));
		}
		else
		{
			return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractFloatingPointArgument(ArgStream));
		}
	case FFormatArgsTrace::FormatArgTypeCode_CategoryString:
		if (ArgSpec.AdditionalIntegerArgumentCount == 2)
		{
			return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream), ExtractIntegerArgument(ArgStream), ExtractStringArgument(ArgStream, Temp, MaxTemp));
		}
		else if (ArgSpec.AdditionalIntegerArgumentCount == 1)
		{
			return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractIntegerArgument(ArgStream), ExtractStringArgument(ArgStream, Temp, MaxTemp));
		}
		else
		{
			return FCString::Snprintf(Out, MaxOut, ArgSpec.FormatString, ExtractStringArgument(ArgStream, Temp, MaxTemp));
		}
	default:
		check(false);
		return 0;
	}
}

void FFormatArgsHelper::Format(TCHAR* Out, uint64 MaxOut, TCHAR* Temp, uint64 MaxTemp, const TCHAR* FormatString, const uint8* FormatArgs)
{
	FFormatArgsStreamContext ArgumentStream;
	InitArgumentStream(ArgumentStream, FormatArgs);
	if (ArgumentStream.ArgumentCount == 0)
	{
		FCString::Strncpy(Out, FormatString, MaxOut);
		return;
	}

	const TCHAR* Src = FormatString;
	TCHAR* Dst = Out;
	TCHAR* DstEnd = Out + MaxOut;
	while (*Src != 0 && Dst != DstEnd)
	{
		FFormatArgSpec Spec;
		const TCHAR* NextSrc = ExtractNextFormatArg(Src, Spec);
		int32 PassthroughCopyLength = FMath::Min(Spec.PassthroughLength, static_cast<int32>(DstEnd - Dst));
		if (PassthroughCopyLength)
		{
			FCString::Strncpy(Dst, Src, PassthroughCopyLength + 1);
			Dst += PassthroughCopyLength;
		}
		if (Spec.Valid)
		{
			int32 Length = FormatArgument(Dst, static_cast<int32>(DstEnd - Dst), Temp, static_cast<int32>(MaxTemp), Spec, ArgumentStream);
			if (Length < 0)
			{
				break;
			}
			Dst += Length;
		}
		Src = NextSrc;
	}
}

} // namespace TraceServices
