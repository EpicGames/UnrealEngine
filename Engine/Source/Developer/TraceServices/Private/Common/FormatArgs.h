// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace TraceServices
{

struct FFormatArgsHelper
{
	static void Format(TCHAR* Out, uint64 MaxOut, TCHAR* Temp, uint64 MaxTemp, const TCHAR* FormatString, const uint8* FormatArgs);

private:
	struct FFormatArgSpec
	{
		int32 PassthroughLength;
		TCHAR FormatString[255];
		uint8 ExpectedTypeCategory;
		uint8 AdditionalIntegerArgumentCount;
		bool Valid;
		bool NothingPrinted;
	};

	struct FFormatArgsStreamContext
	{
		uint8 ArgumentCount;
		uint8 ArgumentTypeCategory;
		uint8 ArgumentTypeSize;
		const uint8* DescriptorPtr;
		const uint8* PayloadPtr;
	};

	static const TCHAR* ExtractNextFormatArg(const TCHAR* FormatString, FFormatArgSpec& Spec);
	static void InitArgumentStream(FFormatArgsStreamContext& Context, const uint8* ArgumentsData);
	static bool AdvanceArgumentStream(FFormatArgsStreamContext& Context);
	static uint64 ExtractIntegerArgument(FFormatArgsStreamContext& ArgStream);
	static double ExtractFloatingPointArgument(FFormatArgsStreamContext& ArgStream);
	static const TCHAR* ExtractStringArgument(FFormatArgsStreamContext& ArgStream, TCHAR* Temp, int32 MaxTemp);
	static int32 FormatArgument(TCHAR* Out, int32 MaxOut, TCHAR* Temp, int32 MaxTemp, const FFormatArgSpec& ArgSpec, FFormatArgsStreamContext& ArgStream);
};

} // namespace TraceServices
