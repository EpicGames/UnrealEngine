// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/ProviderLock.h"

namespace TraceServices
{

void FProviderLock::ReadAccessCheck(FProviderLock::FThreadLocalState& State) const
{
	checkf(State.Lock == this && (State.ReadLockCount > 0 || State.WriteLockCount > 0),
		TEXT("Trying to READ from provider outside of a READ scope"));
}

void FProviderLock::WriteAccessCheck(FProviderLock::FThreadLocalState& State) const
{
	checkf(State.Lock == this && (State.WriteLockCount > 0),
		TEXT("Trying to WRITE to provider outside of an EDIT/WRITE scope"));
}

void FProviderLock::BeginRead(FProviderLock::FThreadLocalState& State)
{
	check(!State.Lock || State.Lock == this);
	checkf(State.WriteLockCount == 0, TEXT("Trying to lock provider for READ while holding EDIT/WRITE access"));
	if (State.ReadLockCount++ == 0)
	{
		State.Lock = this;
		RWLock.ReadLock();
	}
}

void FProviderLock::EndRead(FProviderLock::FThreadLocalState& State)
{
	check(State.Lock == this);
	check(State.ReadLockCount > 0);
	if (--State.ReadLockCount == 0)
	{
		RWLock.ReadUnlock();
		State.Lock = nullptr;
	}
}

void FProviderLock::BeginWrite(FProviderLock::FThreadLocalState& State)
{
	check(!State.Lock || State.Lock == this);
	checkf(State.ReadLockCount == 0, TEXT("Trying to lock provider for EDIT/WRITE while holding READ access"));
	if (State.WriteLockCount++ == 0)
	{
		State.Lock = this;
		RWLock.WriteLock();
	}
}

void FProviderLock::EndWrite(FProviderLock::FThreadLocalState& State)
{
	check(State.Lock == this);
	check(State.WriteLockCount > 0);
	if (--State.WriteLockCount == 0)
	{
		RWLock.WriteUnlock();
		State.Lock = nullptr;
	}
}

} // namespace TraceServices
