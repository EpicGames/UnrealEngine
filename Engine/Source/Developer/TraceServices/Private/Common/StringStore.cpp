// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/StringStore.h"
#include "Misc/ScopeLock.h"

namespace TraceServices
{

FStringStore::FStringStore(FSlabAllocator& InAllocator)
	: Allocator(InAllocator)
{
}

FStringStore::~FStringStore()
{
}

const TCHAR* FStringStore::Find(const TCHAR* String) const
{
	return Find(FStringView(String));
}

const TCHAR* FStringStore::Find(const FStringView& String) const
{
	FScopeLock _(&Cs);

	uint32 Hash = GetTypeHash(String);

	FindStoredStrings.Reset();
	StoredStrings.MultiFind(Hash, FindStoredStrings);
	for (const TCHAR* FoundStoredString : FindStoredStrings)
	{
		if (!String.Compare(FStringView(FoundStoredString)))
		{
			return FoundStoredString;
		}
	}

	return nullptr;
}

const TCHAR* FStringStore::Store(const TCHAR* String)
{
	return Store(FStringView(String));
}

const TCHAR* FStringStore::Store(const FStringView& String)
{
	FScopeLock _(&Cs);

	uint32 Hash = GetTypeHash(String);

	FindStoredStrings.Reset();
	StoredStrings.MultiFind(Hash, FindStoredStrings);
	for (const TCHAR* FoundStoredString : FindStoredStrings)
	{
		if (!String.Compare(FStringView(FoundStoredString)))
		{
			return FoundStoredString;
		}
	}

	int32 StringLength = String.Len() + 1;
	if (BufferLeft < StringLength)
	{
		BufferPtr = reinterpret_cast<TCHAR*>(Allocator.Allocate(BlockSize * sizeof(TCHAR)));
		++BlockCount;
		BufferLeft = BlockSize;
	}
	const TCHAR* Stored = BufferPtr;
	memcpy(BufferPtr, String.GetData(), (StringLength - 1) * sizeof(TCHAR));
	BufferPtr[StringLength - 1] = TEXT('\0');
	BufferLeft -= StringLength;
	BufferPtr += StringLength;

	StoredStrings.Add(Hash, Stored);

	return Stored;
}

} // namespace TraceServices
