// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "HAL/CriticalSection.h"
#include "TraceServices/Containers/SlabAllocator.h"

namespace TraceServices
{

class FStringStore : public IStringStore
{
public:
	FStringStore(FSlabAllocator& Allocator);
	virtual ~FStringStore();

	virtual const TCHAR* Find(const TCHAR* String) const override;
	virtual const TCHAR* Find(const FStringView& String) const override;

	virtual const TCHAR* Store(const TCHAR* String) override;
	virtual const TCHAR* Store(const FStringView& String) override;

private:
	enum
	{
		BlockSize = 4 << 20
	};
	mutable FCriticalSection Cs;
	FSlabAllocator& Allocator;
	TMultiMap<uint32, const TCHAR*> StoredStrings;
	mutable TArray<const TCHAR*> FindStoredStrings;
	TCHAR* BufferPtr = nullptr;
	uint64 BufferLeft = 0;
	uint64 BlockCount = 0;
};

} // namespace TraceServices
