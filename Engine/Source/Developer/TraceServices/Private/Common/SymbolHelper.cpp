// Copyright Epic Games, Inc. All Rights Reserved.

#include "SymbolHelper.h"

#include "Misc/ConfigContext.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSymbolHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymbolHelper::GetSymbolSearchPaths(const FLogCategoryAlias& LogCategory, TArray<FString>& OutSearchPaths)
{
	GetPathsFromEnvVars(LogCategory, OutSearchPaths);
	GetPathsFromSettings(LogCategory, OutSearchPaths);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymbolHelper::GetPathsFromEnvVars(const FLogCategoryAlias& LogCategory, TArray<FString>& OutSearchPaths)
{
	auto SplitEnvPaths = [](FStringView EnvVariable, TArray<FString>& OutList)
		{
			FString SymbolPathPart, SymbolPathRemainder(EnvVariable);
			while (SymbolPathRemainder.Split(TEXT(";"), &SymbolPathPart, &SymbolPathRemainder))
			{
				SymbolPathPart.TrimQuotesInline();
				if (!SymbolPathPart.IsEmpty())
				{
					OutList.Emplace(SymbolPathPart);
				}
			}
			SymbolPathRemainder.TrimQuotesInline();
			if (!SymbolPathRemainder.IsEmpty())
			{
				OutList.Emplace(SymbolPathRemainder);
			}
		};

	const FString InsightsSymbolPath = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_INSIGHTS_SYMBOL_PATH"));
	UE_LOG_REF(LogCategory, Log, TEXT("UE_INSIGHTS_SYMBOL_PATH: %s"), InsightsSymbolPath.IsEmpty() ? TEXT("not set") : *InsightsSymbolPath);
	SplitEnvPaths(InsightsSymbolPath, OutSearchPaths);

#if PLATFORM_WINDOWS
	const FString NTSymbolPath = FPlatformMisc::GetEnvironmentVariable(TEXT("_NT_SYMBOL_PATH"));
	UE_LOG_REF(LogCategory, Log, TEXT("_NT_SYMBOL_PATH: %s"), NTSymbolPath.IsEmpty() ? TEXT("not set") : *NTSymbolPath);
	SplitEnvPaths(NTSymbolPath, OutSearchPaths);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymbolHelper::GetPathsFromSettings(const FLogCategoryAlias& LogCategory, TArray<FString>& OutSearchPaths)
{
	FString SettingsIni;
	TArray<FString> SymbolSearchPaths;

	if (FConfigContext::ReadIntoGConfig().Load(TEXT("UnrealInsightsSettings"), SettingsIni))
	{
		GConfig->GetArray(TEXT("Insights"), TEXT("SymbolSearchPaths"), SymbolSearchPaths, SettingsIni);
	}

#if !NO_LOGGING
	if (SymbolSearchPaths.IsEmpty())
	{
		UE_LOG_REF(LogCategory, Log, TEXT("SymbolSearchPaths not set in the UnrealInsightsSettings.ini file."));
	}
	else
	{
		for (const FString& Path : SymbolSearchPaths)
		{
			UE_LOG_REF(LogCategory, Log, TEXT("+SymbolSearchPaths=%s"), *Path);
		}
	}
#endif

	OutSearchPaths += SymbolSearchPaths;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
