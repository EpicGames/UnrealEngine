// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogCategory.h"

namespace TraceServices
{

class FSymbolHelper
{
public:
#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif

	/**
	* Gets the list of symbol search paths declared by environment variables and
	* the symbol search paths declared in the UnrealInsightsSettings.ini configuration file.
	* @param LogCategory The log category the messages should be emitted to.
	* @param OutSearchPaths An array where the found paths area added.
	*/
	static void GetSymbolSearchPaths(const FLogCategoryAlias& LogCategory, TArray<FString>& OutSearchPaths);

private:
	/**
	* Gets the list of symbol search paths declared by environment variables.
	* @param LogCategory The log category the messages should be emitted to.
	* @param OutSearchPaths An array where the found paths area added.
	*/
	static void GetPathsFromEnvVars(const FLogCategoryAlias& LogCategory, TArray<FString>& OutSearchPaths);

	/**
	* Gets the list of symbol search paths declared in the UnrealInsightsSettings.ini configuration file.
	* @param LogCategory The log category the messages should be emitted to.
	* @param OutSearchPaths An array where the found paths area added.
	*/
	static void GetPathsFromSettings(const FLogCategoryAlias& LogCategory, TArray<FString>& OutSearchPaths);
};

} // namespace TraceServices
