// Copyright Epic Games, Inc. All Rights Reserved.

#include "SymbolStringAllocator.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSymbolStringAllocator
////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FSymbolStringAllocator::Store(const FStringView InString)
{
	const uint32 StringSize = InString.Len() + 1;
	TotalUsedSize += StringSize * sizeof(TCHAR);
	if (StringSize <= BlockSize)
	{
		if (StringSize > BlockRemaining)
		{
			++NumAllocatedBlocks;
			const uint32 AllocSize = BlockSize * sizeof(TCHAR);
			TotalAllocatedSize += AllocSize;
			// Allocate a new block. The allocated memory is owned by the linear allocator.
			Block = (TCHAR*)Allocator.Allocate(AllocSize);
			BlockRemaining = BlockSize;
		}
		const uint32 CopiedSize = InString.CopyString(Block, BlockRemaining - 1, 0);
		check(StringSize == CopiedSize + 1);
		Block[StringSize - 1] = TEXT('\0');
		BlockRemaining -= StringSize;
		const TCHAR* OutString = Block;
		Block += StringSize;
		return OutString;
	}
	else
	{
		++NumAllocatedBlocks;
		const uint32 AllocSize = StringSize * sizeof(TCHAR);
		TotalAllocatedSize += AllocSize;
		// Allocate memory for the current string. The allocated memory is owned by the linear allocator.
		TCHAR* OutString = (TCHAR*)Allocator.Allocate(AllocSize);
		const uint32 CopiedSize = InString.CopyString(OutString, StringSize - 1, 0);
		check(StringSize == CopiedSize + 1);
		OutString[StringSize - 1] = TEXT('\0');
		return OutString;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
