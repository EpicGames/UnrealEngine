// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"

// TraceServices
#include "TraceServices/Containers/Allocators.h"

namespace TraceServices
{

/// An allocator for strings as a wrapper over a linear allocator.
/// The memory allocated is owned by the provided linear allocator.
/// So, the lifetime of stored strings is the same as the linear allocator.
class FSymbolStringAllocator
{
public:
	FSymbolStringAllocator(ILinearAllocator& InAllocator, uint32 InBlockSize)
		: Allocator(InAllocator)
		, BlockSize(InBlockSize)
	{
	}

	const TCHAR* Store(const TCHAR* InString)
	{
		return Store(FStringView(InString));
	}

	const TCHAR* Store(const FStringView InString);

	uint64 GetUsedSize() const
	{
		return TotalUsedSize;
	}

	uint64 GetAllocatedSize() const
	{
		return TotalAllocatedSize;
	}

	uint32 GetNumAllocatedBlocks() const
	{
		return NumAllocatedBlocks;
	}

private:
	ILinearAllocator& Allocator;
	TCHAR* Block = nullptr;
	uint32 BlockSize;
	uint32 BlockRemaining = 0;
	uint32 NumAllocatedBlocks = 0;
	uint64 TotalAllocatedSize = 0;
	uint64 TotalUsedSize = 0;
};

} // namespace TraceServices
