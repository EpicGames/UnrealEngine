// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimelineStatistics.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInternalHistogram
////////////////////////////////////////////////////////////////////////////////////////////////////

class FInternalHistogram
{
private:
	static constexpr int32 HistogramLen = 100;

public:
	FInternalHistogram()
	{
	}

	double GetBucketSize() const
	{
		return BucketSize;
	}

	uint64 GetTotalCount() const
	{
		return TotalCount;
	}

	void Init(double MinTime, double MaxTime);
	void AddValue(double RelativeTime);
	double ComputeMedian() const;

private:
	uint32 Histogram[HistogramLen];
	double BucketSize = 0.0;
	uint64 TotalCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInternalHistogram::Init(double MinTime, double MaxTime)
{
	FMemory::Memzero(Histogram, sizeof(int32) * HistogramLen);

	// Each bucket (Histogram[i]) will be centered on a value.
	// I.e. First bucket (bucket 0) is centered on Min value: [Min-BucketSize/2, Min+BucketSize/2)
	// and last bucket (bucket N-1) is centered on Max value: [Max-BucketSize/2, Max+BucketSize/2).
	if (MaxTime == MinTime)
	{
		BucketSize = 1.0; // single large bucket
	}
	else
	{
		constexpr double InvHistogramLen = 1.0 / double(HistogramLen - 1);
		BucketSize = (MaxTime - MinTime) * InvHistogramLen;
	}

	TotalCount = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInternalHistogram::AddValue(double RelativeTime)
{
	int32 Index = static_cast<int32>((RelativeTime + BucketSize / 2) / BucketSize);
	ensure(Index >= 0);
	if (Index < 0)
	{
		Index = 0;
	}
	ensure(Index < HistogramLen);
	if (Index >= HistogramLen)
	{
		Index = HistogramLen - 1;
	}
	Histogram[Index]++;
	TotalCount++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FInternalHistogram::ComputeMedian() const
{
	const uint64 HalfCount = TotalCount / 2;
	uint64 Count = 0;
	for (int32 HistogramIndex = 0; HistogramIndex < HistogramLen; HistogramIndex++)
	{
		Count += Histogram[HistogramIndex];
		if (Count > HalfCount)
		{
			double Median = HistogramIndex * BucketSize;
			if (HistogramIndex > 0 &&
				TotalCount % 2 == 0 &&
				Count - Histogram[HistogramIndex] == HalfCount)
			{
				const double PrevMedian = (HistogramIndex - 1) * BucketSize;
				Median = (Median + PrevMedian) / 2;
			}
			return Median;
		}
	}
	return 0.0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInternalAggregationEntry
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimelineStatistics::FInternalAggregationEntry::~FInternalAggregationEntry()
{
	if (InclHistogram)
	{
		InclHistogram->~FInternalHistogram(); // destruct
		InclHistogram = nullptr;
	}
	if (ExclHistogram)
	{
		ExclHistogram->~FInternalHistogram(); // destruct
		ExclHistogram = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimelineStatistics
////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimelineStatistics::UpdateTotalMinMaxTimerAggregationStats(FTimelineStatistics::FInternalAggregationEntry& AggregationEntry, double InclTime, double ExclTime)
{
	UpdateTotalMinMaxTimerStats(AggregationEntry.Stats, InclTime, ExclTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimelineStatistics::UpdateTotalMinMaxTimerStats(FAggregatedTimingStats& Stats, double InclTime, double ExclTime)
{
	Stats.TotalInclusiveTime += InclTime;
	if (InclTime < Stats.MinInclusiveTime)
	{
		Stats.MinInclusiveTime = InclTime;
	}
	if (InclTime > Stats.MaxInclusiveTime)
	{
		Stats.MaxInclusiveTime = InclTime;
	}

	Stats.TotalExclusiveTime += ExclTime;
	if (ExclTime < Stats.MinExclusiveTime)
	{
		Stats.MinExclusiveTime = ExclTime;
	}
	if (ExclTime > Stats.MaxExclusiveTime)
	{
		Stats.MaxExclusiveTime = ExclTime;
	}

	Stats.InstanceCount++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimelineStatistics::InitHistogram(FTimelineStatistics::FInternalAggregationEntry& AggregationEntry, ILinearAllocator& Allocator)
{
	const FAggregatedTimingStats& Stats = AggregationEntry.Stats;

	check(AggregationEntry.InclHistogram == nullptr);
	AggregationEntry.InclHistogram = (FInternalHistogram*)Allocator.Allocate(sizeof(FInternalHistogram));
	new (AggregationEntry.InclHistogram) FInternalHistogram(); // construct
	AggregationEntry.InclHistogram->Init(Stats.MinInclusiveTime, Stats.MaxInclusiveTime);

	check(AggregationEntry.ExclHistogram == nullptr);
	AggregationEntry.ExclHistogram = (FInternalHistogram*)Allocator.Allocate(sizeof(FInternalHistogram));
	new (AggregationEntry.ExclHistogram) FInternalHistogram(); // construct
	AggregationEntry.ExclHistogram->Init(Stats.MinExclusiveTime, Stats.MaxExclusiveTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimelineStatistics::UpdateHistogram(FTimelineStatistics::FInternalAggregationEntry& AggregationEntry, double InclTime, double ExclTime)
{
	const FAggregatedTimingStats& Stats = AggregationEntry.Stats;

	check(AggregationEntry.InclHistogram != nullptr);
	AggregationEntry.InclHistogram->AddValue(InclTime - Stats.MinInclusiveTime);

	check(AggregationEntry.ExclHistogram != nullptr);
	AggregationEntry.ExclHistogram->AddValue(ExclTime - Stats.MinExclusiveTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimelineStatistics::ComputeMedianFromHistogram(FTimelineStatistics::FInternalAggregationEntry& AggregationEntry)
{
	FAggregatedTimingStats& Stats = AggregationEntry.Stats;

	check(AggregationEntry.InclHistogram != nullptr);
	Stats.MedianInclusiveTime = Stats.MinInclusiveTime + AggregationEntry.InclHistogram->ComputeMedian();

	check(AggregationEntry.ExclHistogram != nullptr);
	Stats.MedianExclusiveTime = Stats.MinExclusiveTime + AggregationEntry.ExclHistogram->ComputeMedian();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimelineStatistics::PostProcessTimerStats(FTimelineStatistics::FInternalAggregationEntry& AggregationEntry, bool bComputeMedian)
{
	FAggregatedTimingStats& Stats = AggregationEntry.Stats;

	// Compute average inclusive/exclusive times.
	ensure(Stats.InstanceCount > 0);
	double InvCount = 1.0f / static_cast<double>(Stats.InstanceCount);
	Stats.AverageInclusiveTime = Stats.TotalInclusiveTime * InvCount;
	Stats.AverageExclusiveTime = Stats.TotalExclusiveTime * InvCount;
	Stats.AverageInstanceCount = static_cast<double>(Stats.InstanceCount);

	if (bComputeMedian)
	{
		ComputeMedianFromHistogram(AggregationEntry);
		check(AggregationEntry.InclHistogram->GetTotalCount() == Stats.InstanceCount);
		check(AggregationEntry.ExclHistogram->GetTotalCount() == Stats.InstanceCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
