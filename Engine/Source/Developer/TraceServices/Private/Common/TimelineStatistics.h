// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Misc/QueuedThreadPool.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"

// TraceServices
#include "TraceServices/Common/CancellationToken.h"
#include "TraceServices/Containers/Allocators.h"
#include "TraceServices/Containers/SlabAllocator.h"
#include "TraceServices/Containers/Timelines.h"

#include <cfloat>

namespace TraceServices
{

struct FAggregatedTimingStats
{
	uint64 InstanceCount = 0;
	double AverageInstanceCount = 0.0;
	double TotalInclusiveTime = 0.0;
	double MinInclusiveTime = DBL_MAX;
	double MaxInclusiveTime = -DBL_MAX;
	double AverageInclusiveTime = 0.0;
	double MedianInclusiveTime = 0.0;
	double TotalExclusiveTime = 0.0;
	double MinExclusiveTime = DBL_MAX;
	double MaxExclusiveTime = -DBL_MAX;
	double AverageExclusiveTime = 0.0;
	double MedianExclusiveTime = 0.0;
};

struct FFrameData
{
	double StartTime;
	double EndTime;
};

class FTimelineStatistics
{
public:
	template<typename KeyType, typename ValueType>
	class TAggregationResult
	{
	private:
		static constexpr uint64 AllocatorSlabSize = 1 * 1024 * 1024;

	public:
		TAggregationResult()
			: Allocator(AllocatorSlabSize)
		{
		}

		TAggregationResult(TSparseMap<KeyType, ValueType>& InMap)
			: Allocator(AllocatorSlabSize)
			, Map(InMap)
		{
		}

		// Non-copyable
		TAggregationResult(const TAggregationResult&) = delete;
		TAggregationResult& operator=(const TAggregationResult&) = delete;

		~TAggregationResult()
		{
		}

		ILinearAllocator& GetAllocator()
		{
			return Allocator;
		}

		TSparseMap<KeyType, ValueType>& GetMap()
		{
			return Map;
		}

		const TSparseMap<KeyType, ValueType>& GetMap() const
		{
			return Map;
		}

	private:
		FSlabAllocator Allocator; // to allocate extra data for histograms
		TSparseMap<KeyType, ValueType> Map;
	};

public:
	template<
		typename TimelineType,
		typename BucketMappingFunc,
		typename BucketKeyType
	>
	static void CreateAggregation(
		const TArray<const TimelineType*>& Timelines,
		BucketMappingFunc BucketMapper,
		double IntervalStart,
		double IntervalEnd,
		TSharedPtr<FCancellationToken> CancellationToken,
		TAggregationResult<BucketKeyType, FAggregatedTimingStats>& Result)
	{
		TAggregationResult<BucketKeyType, FInternalAggregationEntry> InternalResult;

		// Compute instance count and total/min/max inclusive/exclusive times for each timer.
		for (const TimelineType* Timeline : Timelines)
		{
			if (CancellationToken.IsValid() && CancellationToken->ShouldCancel())
			{
				return;
			}
			ProcessTimeline(Timeline, BucketMapper, UpdateTotalMinMaxTimerAggregationStats, IntervalStart, IntervalEnd, InternalResult);
		}

		// Now, as we know min/max inclusive/exclusive times for each timer, we can compute histogram and median values.
		const bool bComputeMedian = true;
		if (bComputeMedian)
		{
			// Update bucket size (DT) for computing histogram.
			for (auto& KV : InternalResult.GetMap())
			{
				InitHistogram(KV.Value, InternalResult.GetAllocator());
			}

			// Compute histogram.
			for (const TimelineType* Timeline : Timelines)
			{
				if (CancellationToken.IsValid() && CancellationToken->ShouldCancel())
				{
					return;
				}
				ProcessTimeline(Timeline, BucketMapper, UpdateHistogram, IntervalStart, IntervalEnd, InternalResult);
			}
		}

		// Compute average and median inclusive/exclusive times.
		for (auto& KV : InternalResult.GetMap())
		{
			PostProcessTimerStats(KV.Value, bComputeMedian);
			Result.GetMap().Add(KV.Key, KV.Value.Stats);
		}
	};

	template<
		typename TimelineType,
		typename BucketMappingFunc,
		typename BucketKeyType
	>
	static void CreateFrameStatsAggregation(
		const TArray<const TimelineType*>& Timelines,
		BucketMappingFunc BucketMapper,
		const TArray<FFrameData>& Frames,
		TSharedPtr<FCancellationToken> CancellationToken,
		TAggregationResult<BucketKeyType, FAggregatedTimingStats>& Result,
		double RatioOfThreadsToUse)
	{
		int32 FramesNum = Frames.Num();
		if (FramesNum == 0)
		{
			return;
		}

		TAggregationResult<BucketKeyType, FInternalFrameAggregationEntry> GlobalResult;
		TAggregationResult<BucketKeyType, FAggregatedTimingStats> InitialTimerMap;

		double StartTime = Frames[0].StartTime;
		double EndTime = Frames[FramesNum - 1].EndTime;

		// Gather the keys for all timers so we have a stable map (no insertions necessary) for the next step.
		// This is to guarantee that GlobalResult and FrameResult have the same iteration order.
		for (const TimelineType* Timeline : Timelines)
		{
			GatherKeysFromTimeline(Timeline, BucketMapper, StartTime, EndTime, InitialTimerMap);
		}

		// For very large numbers of timers or frames, an out of memory is possible so don't compute the median.
		constexpr int64 MaxSize = 5 * 100 * 1000 * 1000;
		bool bComputeMedian = InitialTimerMap.GetMap().Num() * (int64)FramesNum < MaxSize;

		for (auto& KV : InitialTimerMap.GetMap())
		{
			FInternalFrameAggregationEntry Entry;
			if (bComputeMedian)
			{
				Entry.FrameInclusiveTimes.AddUninitialized(Frames.Num());
				Entry.FrameExclusiveTimes.AddUninitialized(Frames.Num());
			}
			Entry.Inner.Stats = KV.Value;
			GlobalResult.GetMap().Add(KV.Key, MoveTemp(Entry));
		}

		TQueue<TSharedPtr<TAggregationResult<BucketKeyType, FAggregatedTimingStats>>, EQueueMode::Mpsc> FrameResultsQueue;

		int32 NumTasks = FMath::Max(1, (int32)((float)GThreadPool->GetNumThreads() * RatioOfThreadsToUse));
		NumTasks = FMath::Min((int32)NumTasks, Frames.Num());
		int32 FramesPerTask = Frames.Num() / NumTasks;
		int32 ExtraFrameTasks = Frames.Num() - NumTasks * FramesPerTask;

		int32 StartIndex = 0;
		int32 EndIndex = FramesPerTask + (ExtraFrameTasks > 0 ? 1 : 0);
		--ExtraFrameTasks;

		TArray<UE::Tasks::TTask<void>> AsyncTasks;
		for (int32 Index = 0; Index < NumTasks; ++Index)
		{
			AsyncTasks.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION,
				[StartIndex, EndIndex, &Timelines, bComputeMedian, BucketMapper, &Frames, &FrameResultsQueue, &InitialTimerMap, &GlobalResult, CancellationToken]()
				{
					if (CancellationToken.IsValid() && CancellationToken->ShouldCancel())
					{
						return;
					}

					TSharedPtr<TAggregationResult<BucketKeyType, FAggregatedTimingStats>> TaskResult = MakeShared<TAggregationResult<BucketKeyType, FAggregatedTimingStats>>(InitialTimerMap.GetMap());
					TAggregationResult<BucketKeyType, FAggregatedTimingStats> FrameResult(InitialTimerMap.GetMap());

					for (int FrameIndex = StartIndex; FrameIndex < EndIndex; ++FrameIndex)
					{
						if (CancellationToken.IsValid() && CancellationToken->ShouldCancel())
						{
							return;
						}

						// Compute instance count and total/min/max inclusive/exclusive times for each timer.
						for (const TimelineType* Timeline : Timelines)
						{
							ProcessTimelineForFrameStats(Timeline, BucketMapper, UpdateTotalMinMaxTimerStats, Frames[FrameIndex].StartTime, Frames[FrameIndex].EndTime, FrameResult);
						}

						typename TSparseMap<BucketKeyType, FAggregatedTimingStats>::TIterator FrameResultIterator(FrameResult.GetMap());
						typename TSparseMap<BucketKeyType, FAggregatedTimingStats>::TIterator TaskResultIterator(TaskResult->GetMap());
						typename TSparseMap<BucketKeyType, FInternalFrameAggregationEntry>::TIterator GlobalResultIterator(GlobalResult.GetMap());
						while (FrameResultIterator)
						{
							check(FrameResultIterator->Key == TaskResultIterator->Key);
							check(GlobalResultIterator->Key == TaskResultIterator->Key);

							FAggregatedTimingStats& FrameStats = FrameResultIterator->Value;
							FAggregatedTimingStats& TaskResultStats = TaskResultIterator->Value;

							TaskResultStats.InstanceCount += FrameStats.InstanceCount;

							TaskResultStats.TotalInclusiveTime += FrameStats.TotalInclusiveTime;
							TaskResultStats.MinInclusiveTime = FMath::Min(TaskResultStats.MinInclusiveTime, FrameStats.TotalInclusiveTime);
							TaskResultStats.MaxInclusiveTime = FMath::Max(TaskResultStats.MaxInclusiveTime, FrameStats.TotalInclusiveTime);

							TaskResultStats.TotalExclusiveTime += FrameStats.TotalExclusiveTime;
							TaskResultStats.MinExclusiveTime = FMath::Min(TaskResultStats.MinExclusiveTime, FrameStats.TotalExclusiveTime);
							TaskResultStats.MaxExclusiveTime = FMath::Max(TaskResultStats.MaxExclusiveTime, FrameStats.TotalExclusiveTime);

							if (bComputeMedian)
							{
								FInternalFrameAggregationEntry& GlobalResultValue = GlobalResultIterator->Value;
								GlobalResultValue.FrameInclusiveTimes[FrameIndex] = FrameStats.TotalInclusiveTime;
								GlobalResultValue.FrameExclusiveTimes[FrameIndex] = FrameStats.TotalExclusiveTime;
							}

							// Reset the per frame stats.
							FrameStats = FAggregatedTimingStats();

							++FrameResultIterator;
							++TaskResultIterator;
							++GlobalResultIterator;
						}
					}

					FrameResultsQueue.Enqueue(MoveTemp(TaskResult));
				}));

			StartIndex = EndIndex;
			EndIndex = StartIndex + FramesPerTask + (ExtraFrameTasks > 0 ? 1 : 0);
			--ExtraFrameTasks;
		}

		check(StartIndex == Frames.Num());

		int ProcessedResults = 0;
		while (ProcessedResults < NumTasks)
		{
			if (CancellationToken.IsValid() && CancellationToken->ShouldCancel())
			{
				UE::Tasks::Wait(AsyncTasks);
				return;
			}

			if (FrameResultsQueue.IsEmpty())
			{
				FPlatformProcess::SleepNoStats(0.1f);
				continue;
			}

			TSharedPtr<TAggregationResult<BucketKeyType, FAggregatedTimingStats>> TaskResult;
			ensure(FrameResultsQueue.Dequeue(TaskResult));
			++ProcessedResults;

			typename TSparseMap<BucketKeyType, FAggregatedTimingStats>::TIterator TaskResultIterator(TaskResult->GetMap());
			typename TSparseMap<BucketKeyType, FInternalFrameAggregationEntry>::TIterator ResultIterator(GlobalResult.GetMap());
			while (ResultIterator)
			{
				check(ResultIterator->Key == TaskResultIterator->Key);

				FAggregatedTimingStats& TaskStats = TaskResultIterator->Value;
				FAggregatedTimingStats& ResultStats = ResultIterator->Value.Inner.Stats;

				ResultStats.InstanceCount += TaskStats.InstanceCount;

				ResultStats.TotalInclusiveTime += TaskStats.TotalInclusiveTime;
				ResultStats.MinInclusiveTime = FMath::Min(ResultStats.MinInclusiveTime, TaskStats.MinInclusiveTime);
				ResultStats.MaxInclusiveTime = FMath::Max(ResultStats.MaxInclusiveTime, TaskStats.MaxInclusiveTime);

				ResultStats.TotalExclusiveTime += TaskStats.TotalExclusiveTime;
				ResultStats.MinExclusiveTime = FMath::Min(ResultStats.MinExclusiveTime, TaskStats.MinExclusiveTime);
				ResultStats.MaxExclusiveTime = FMath::Max(ResultStats.MaxExclusiveTime, TaskStats.MaxExclusiveTime);

				++TaskResultIterator;
				++ResultIterator;
			}
		}

		if (bComputeMedian)
		{
			// Compute the median inclusive and exclusive time.
			for (auto& KV : GlobalResult.GetMap())
			{
				FInternalFrameAggregationEntry& FrameEntry = KV.Value;
				FInternalAggregationEntry& AggregationEntry = FrameEntry.Inner;

				if (AggregationEntry.Stats.InstanceCount == 0)
				{
					// Some timers might have 0 instances because the gathering phase also searched between frames.
					continue;
				}

				InitHistogram(AggregationEntry, GlobalResult.GetAllocator());

				for (int32 Index = 0; Index < FramesNum; ++Index)
				{
					UpdateHistogram(AggregationEntry, FrameEntry.FrameInclusiveTimes[Index], FrameEntry.FrameExclusiveTimes[Index]);
				}

				ComputeMedianFromHistogram(AggregationEntry);
			}
		}

		for (auto& KV : GlobalResult.GetMap())
		{
			FAggregatedTimingStats& Stats = KV.Value.Inner.Stats;

			Stats.AverageInclusiveTime = Stats.TotalInclusiveTime / FramesNum;
			Stats.AverageExclusiveTime = Stats.TotalExclusiveTime / FramesNum;
			Stats.AverageInstanceCount = static_cast<double>(Stats.InstanceCount) / FramesNum;

			Result.GetMap().Add(KV.Key, Stats);
		}
	}

private:
	struct FInternalAggregationEntry
	{
		FInternalAggregationEntry() = default;
		~FInternalAggregationEntry();

		FAggregatedTimingStats Stats;

		// Histogram for computing median inclusive time.
		class FInternalHistogram* InclHistogram = nullptr;

		// Histogram for computing median exclusive time.
		class FInternalHistogram* ExclHistogram = nullptr;
	};

	struct FInternalFrameAggregationEntry
	{
		FInternalAggregationEntry Inner;

		TArray<double> FrameInclusiveTimes;
		TArray<double> FrameExclusiveTimes;
	};

	template<
		typename TimelineType,
		typename BucketMappingFunc,
		typename BucketKeyType,
		typename CallbackType
	>
	static void ProcessTimeline(
		const TimelineType* Timeline,
		BucketMappingFunc BucketMapper,
		CallbackType Callback,
		double IntervalStart,
		double IntervalEnd,
		TAggregationResult<BucketKeyType, FInternalAggregationEntry>& InternalResult)
	{
		struct FStackEntry
		{
			double StartTime;
			double ExclusiveTime;
			BucketKeyType BucketKey;
		};

		TArray<FStackEntry> Stack;
		Stack.Reserve(1024);
		double LastTime = 0.0;
		Timeline->EnumerateEvents(IntervalStart, IntervalEnd,
			[BucketMapper, Callback, IntervalStart, IntervalEnd, &Stack, &LastTime, &InternalResult]
			(bool IsEnter, double Time, const typename TimelineType::EventType& Event)
			{
				Time = FMath::Clamp(Time, IntervalStart, IntervalEnd);
				BucketKeyType BucketKey = BucketMapper(Event);
				if (Stack.Num())
				{
					FStackEntry& StackEntry = Stack.Top();
					StackEntry.ExclusiveTime += Time - LastTime;
				}
				LastTime = Time;
				if (IsEnter)
				{
					FStackEntry& StackEntry = Stack.AddDefaulted_GetRef();
					StackEntry.StartTime = Time;
					StackEntry.ExclusiveTime = 0.0;
					StackEntry.BucketKey = BucketKey;
					InternalResult.GetMap().FindOrAdd(BucketKey);
				}
				else
				{
					check(Stack.Num() > 0);
					FStackEntry& StackEntry = Stack.Last();
					double EventInclusiveTime = Time - StackEntry.StartTime;
					check(EventInclusiveTime >= 0.0);
					double EventExclusiveTime = StackEntry.ExclusiveTime;
					check(EventExclusiveTime >= 0.0 && EventExclusiveTime <= EventInclusiveTime);
					Stack.Pop(EAllowShrinking::No);
					double EventNonRecursiveInclusiveTime = EventInclusiveTime;
					for (const FStackEntry& AncestorStackEntry : Stack)
					{
						if (AncestorStackEntry.BucketKey == BucketKey)
						{
							EventNonRecursiveInclusiveTime = 0.0;
						}
					}
					Callback(InternalResult.GetMap()[BucketKey], EventNonRecursiveInclusiveTime, EventExclusiveTime);
				}
				return EEventEnumerate::Continue;
			});
	}

	template<
		typename TimelineType,
		typename BucketMappingFunc,
		typename BucketKeyType,
		typename CallbackType
	>
	static void ProcessTimelineForFrameStats(
		const TimelineType* Timeline,
		BucketMappingFunc BucketMapper,
		CallbackType Callback,
		double IntervalStart,
		double IntervalEnd,
		TAggregationResult<BucketKeyType, FAggregatedTimingStats>& InternalResult)
	{
		struct FStackEntry
		{
			double StartTime;
			double ExclusiveTime;
			BucketKeyType BucketKey;
		};

		TArray<FStackEntry> Stack;
		Stack.Reserve(1024);
		double LastTime = 0.0;
		Timeline->EnumerateEvents(IntervalStart, IntervalEnd,
			[BucketMapper, Callback, IntervalStart, IntervalEnd, &Stack, &LastTime, &InternalResult]
			(bool IsEnter, double Time, const typename TimelineType::EventType& Event)
			{
				Time = FMath::Clamp(Time, IntervalStart, IntervalEnd);
				BucketKeyType BucketKey = BucketMapper(Event);
				if (Stack.Num())
				{
					FStackEntry& StackEntry = Stack.Top();
					StackEntry.ExclusiveTime += Time - LastTime;
				}
				LastTime = Time;
				if (IsEnter)
				{
					FStackEntry& StackEntry = Stack.AddDefaulted_GetRef();
					StackEntry.StartTime = Time;
					StackEntry.ExclusiveTime = 0.0;
					StackEntry.BucketKey = BucketKey;
				}
				else
				{
					check(Stack.Num() > 0);
					FStackEntry& StackEntry = Stack.Last();
					double EventInclusiveTime = Time - StackEntry.StartTime;
					check(EventInclusiveTime >= 0.0);
					double EventExclusiveTime = StackEntry.ExclusiveTime;
					check(EventExclusiveTime >= 0.0 && EventExclusiveTime <= EventInclusiveTime);
					Stack.Pop(EAllowShrinking::No);
					double EventNonRecursiveInclusiveTime = EventInclusiveTime;
					for (const FStackEntry& AncestorStackEntry : Stack)
					{
						if (AncestorStackEntry.BucketKey == BucketKey)
						{
							EventNonRecursiveInclusiveTime = 0.0;
						}
					}
					Callback(InternalResult.GetMap()[BucketKey], EventNonRecursiveInclusiveTime, EventExclusiveTime);
				}
				return EEventEnumerate::Continue;
			});
	}

	template<
		typename TimelineType,
		typename BucketMappingFunc,
		typename BucketKeyType
	>
	static void GatherKeysFromTimeline(
		const TimelineType* Timeline,
		BucketMappingFunc BucketMapper,
		double IntervalStart,
		double IntervalEnd,
		TAggregationResult<BucketKeyType, FAggregatedTimingStats>& InternalResult)
	{
		Timeline->EnumerateEvents(IntervalStart, IntervalEnd,
			[BucketMapper, IntervalStart, IntervalEnd, &InternalResult]
			(bool IsEnter, double Time, const typename TimelineType::EventType& Event)
			{
				BucketKeyType BucketKey = BucketMapper(Event);

				if (IsEnter)
				{
					InternalResult.GetMap().FindOrAdd(BucketKey);
				}

				return EEventEnumerate::Continue;
			});
	}

	static void UpdateTotalMinMaxTimerAggregationStats(FInternalAggregationEntry& AggregationEntry, double InclTime, double ExclTime);
	static void UpdateTotalMinMaxTimerStats(FAggregatedTimingStats& Stats, double InclTime, double ExclTime);
	static void InitHistogram(FInternalAggregationEntry& AggregationEntry, ILinearAllocator& Allocator);
	static void UpdateHistogram(FInternalAggregationEntry& AggregationEntry, double InclTime, double ExclTime);
	static void ComputeMedianFromHistogram(FInternalAggregationEntry& AggregationEntry);
	static void PostProcessTimerStats(FInternalAggregationEntry& AggregationEntry, bool bComputeMedian);
};

} // namespace TraceServices
