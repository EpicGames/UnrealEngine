// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils.h"

#include "HAL/FileManager.h"
#include "Templates/SharedPointer.h"

// TraceServices
#include "Common/FormatArgs.h"
#include "TraceServices/Utils.h"
#include "TraceServices/Containers/Tables.h"

DEFINE_LOG_CATEGORY(LogTraceServices);

namespace TraceServices
{

bool Table2Csv(const IUntypedTable& Table, const TCHAR* Filename)
{
	TUniquePtr<FArchive> OutputFile(IFileManager::Get().CreateFileWriter(Filename));
	if (!OutputFile.IsValid())
	{
		return false;
	}

	FString Header;
	const ITableLayout& Layout = Table.GetLayout();
	const uint64 ColumnCount = Layout.GetColumnCount();
	for (uint64 ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
	{
		Header += Layout.GetColumnName(ColumnIndex);
		if (ColumnIndex < ColumnCount - 1)
		{
			Header += TEXT(",");
		}
		else
		{
			Header += TEXT("\n");
		}
	}
	auto AnsiHeader = StringCast<ANSICHAR>(*Header);
	OutputFile->Serialize((void*)AnsiHeader.Get(), AnsiHeader.Length());
	TUniquePtr<IUntypedTableReader> TableReader(Table.CreateReader());
	for (; TableReader->IsValid(); TableReader->NextRow())
	{
		FString Line;
		for (uint64 ColumnIndex = 0; ColumnIndex < ColumnCount; ++ColumnIndex)
		{
			switch (Layout.GetColumnType(ColumnIndex))
			{
			case TableColumnType_Bool:
				Line += TableReader->GetValueBool(ColumnIndex) ? "true" : "false";
				break;
			case TableColumnType_Int:
				Line += FString::Printf(TEXT("%lld"), TableReader->GetValueInt(ColumnIndex));
				break;
			case TableColumnType_Float:
				Line += FString::Printf(TEXT("%f"), TableReader->GetValueFloat(ColumnIndex));
				break;
			case TableColumnType_Double:
				Line += FString::Printf(TEXT("%f"), TableReader->GetValueDouble(ColumnIndex));
				break;
			case TableColumnType_CString:
				FString ValueString = TableReader->GetValueCString(ColumnIndex);
				ValueString.ReplaceInline(TEXT(","), TEXT(" "));
				Line += ValueString;
				break;
			}
			if (ColumnIndex < ColumnCount - 1)
			{
				Line += TEXT(",");
			}
			else
			{
				Line += TEXT("\n");
			}
		}
		auto AnsiLine = StringCast<ANSICHAR>(*Line);
		OutputFile->Serialize((void*)AnsiLine.Get(), AnsiLine.Length());
	}
	OutputFile->Close();

	return true;
}

void StringFormat(TCHAR* Out, uint64 MaxOut, TCHAR* Temp, uint64 MaxTemp, const TCHAR* FormatString, const uint8* FormatArgs)
{
	FFormatArgsHelper::Format(Out, MaxOut, Temp, MaxTemp, FormatString, FormatArgs);
}

} // namespace TraceServices
