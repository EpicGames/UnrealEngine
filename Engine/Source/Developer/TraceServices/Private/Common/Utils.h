// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Utils.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTraceServices, Log, All);

namespace TraceServices {
namespace FTraceAnalyzerUtils {

inline uint64 Decode7bit(const uint8*& BufferPtr)
{
	uint64 Value = 0;
	uint64 ByteIndex = 0;
	bool HasMoreBytes;
	do
	{
		uint8 ByteValue = *BufferPtr++;
		HasMoreBytes = ByteValue & 0x80;
		Value |= uint64(ByteValue & 0x7f) << (ByteIndex * 7);
		++ByteIndex;
	} while (HasMoreBytes);
	return Value;
}

inline int64 DecodeZigZag(const uint8*& BufferPtr)
{
	uint64 Z = Decode7bit(BufferPtr);
	return (Z & 1) ? (Z >> 1) ^ -1 : (Z >> 1);
}

inline uint32 GetThreadIdField(
	const UE::Trace::IAnalyzer::FOnEventContext& Context,
	const ANSICHAR* FieldName="ThreadId")
{
	// Trace analysis was changed to be able to provide a suitable id. Prior to
	// this users of Trace would send along their own thread ids. For backwards
	// compatibility we'll bias field thread ids to avoid collision with Trace's.
	static const uint32 Bias = 0x70000000;
	uint32 ThreadId = Context.EventData.GetValue<uint32>(FieldName, 0);
	ThreadId |= ThreadId ? Bias : Context.ThreadInfo.GetId();
	return ThreadId;
}

} // namespace FTraceAnalyzerUtils
} // namespace TraceServices
