// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllocMap.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

//...

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
