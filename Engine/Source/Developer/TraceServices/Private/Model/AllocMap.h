// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Model/AllocationItem.h"

namespace TraceServices
{

class FAllocMap
{
public:
	typedef int32 SizeType;

public:
	FORCEINLINE FAllocMap()
	{
	}

	FORCEINLINE ~FAllocMap()
	{
		Empty();
	}

	FORCEINLINE void Empty(SizeType ExpectedNumElements = 0)
	{
		DestructAllocs();
		AllocsMap.Empty(ExpectedNumElements);
	}

	FORCEINLINE void Reset()
	{
		DestructAllocs();
		AllocsMap.Reset();
	}

	FORCEINLINE SizeType Num() const
	{
		return AllocsMap.Num();
	}

	// Finds the allocation with the specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRef(uint64 Address) const
	{
		return AllocsMap.FindRef(Address);
	}

	// Finds an allocation containing the specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRange(uint64 Address) const
	{
		for (const auto& Alloc : AllocsMap)
		{
			if (Alloc.Value->IsContained(Address))
			{
				return Alloc.Value;
			}
		}
		return nullptr;
	}

	FORCEINLINE void Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
	{
		for (const auto& KV : AllocsMap)
		{
			Callback(*KV.Value);
		}
	}

	FORCEINLINE void Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
	{
		for (const auto& KV : AllocsMap)
		{
			const FAllocationItem& Allocation = *KV.Value;
			if (Allocation.Address >= StartAddress && Allocation.Address < EndAddress)
			{
				Callback(Allocation);
			}
		}
	}

	// The collection keeps ownership of FAllocationItem* until Remove is called.
	FORCEINLINE void Add(FAllocationItem* Alloc)
	{
		AllocsMap.Add(Alloc->Address, Alloc);
	}

	// The caller takes ownership of FAllocationItem*. Returns nullptr if Address is not found.
	FORCEINLINE FAllocationItem* Remove(uint64 Address)
	{
		FAllocationItem* RemovedLongLivingAlloc;
		if (AllocsMap.RemoveAndCopyValue(Address, RemovedLongLivingAlloc))
		{
			return RemovedLongLivingAlloc;
		}
		return nullptr;
	}

private:
	FORCEINLINE void DestructAllocs()
	{
		for (const auto& KV : AllocsMap)
		{
			const FAllocationItem* Allocation = KV.Value;
			delete Allocation;
		}
	}

private:
	template<typename ValueType>
	struct TAddressKeyFuncs : BaseKeyFuncs<TPair<uint64, ValueType>, uint64, false>
	{
		typedef typename TTypeTraits<uint64>::ConstPointerType KeyInitType;
		typedef const TPairInitializer<typename TTypeTraits<uint64>::ConstInitType, typename TTypeTraits<ValueType>::ConstInitType>& ElementInitType;

		static FORCEINLINE uint64 GetSetKey(ElementInitType Element)
		{
			return Element.Key;
		}

		static FORCEINLINE bool Matches(uint64 A, uint64 B)
		{
			return A == B;
		}

		static FORCEINLINE uint32 GetKeyHash(uint64 Key)
		{
			return uint32(Key >> 6);
		}
	};

private:
	TSparseMap<uint64, FAllocationItem*, FDefaultSparseSetAllocator, TAddressKeyFuncs<FAllocationItem*>> AllocsMap;
};

} // namespace TraceServices
