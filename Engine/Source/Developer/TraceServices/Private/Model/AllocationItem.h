// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "ProfilingDebugging/MemoryTrace.h" // for EMemoryTraceHeapAllocationFlags
#include "TraceServices/Model/AllocationsProvider.h" // for TagIdType

namespace TraceServices
{

struct FAllocationItem
{
	static constexpr uint64 MaxSize = (1ull << 56) - 1ull;
	static constexpr uint32 MaxThreadId = (1u << 16) - 1u;
	static constexpr uint32 MaxHeapId = (1u << 24) - 1u;
	static constexpr uint32 MaxFlagsValue = (1u << 8) - 1u;

	FORCEINLINE bool IsContained(uint64 InAddress) const
	{
		return InAddress >= Address && InAddress < Address + GetSize();
	}

	FORCEINLINE uint64 GetEndAddress() const
	{
		return Address + GetSize();
	}

	FORCEINLINE uint64 GetSize() const
	{
		return Size;
	}

	FORCEINLINE void SetSize(uint64 InSize)
	{
		check((InSize & ~MaxSize) == 0);
		Size = InSize;
	}

	FORCEINLINE uint32 GetAlignment() const
	{
		return AlignmentPow2 ? 1u << (AlignmentPow2 - 1) : 0u;
	}

	FORCEINLINE void SetAlignment(uint32 InAlignment)
	{
		check(InAlignment == 0 || FMath::IsPowerOfTwo(InAlignment));
		AlignmentPow2 = 32 - FMath::CountLeadingZeros(InAlignment);
	}

	FORCEINLINE uint32 GetEventDistance() const
	{
		return EndEventIndex - StartEventIndex;
	}

	FORCEINLINE void SetStartEventIndex(uint32 EventIndex)
	{
		StartEventIndex = EventIndex;
	}

	FORCEINLINE void SetEndEventIndex(uint32 EventIndex)
	{
		check(EventIndex > StartEventIndex);
		EndEventIndex = EventIndex;
	}

	FORCEINLINE void SetEndTime(double Time)
	{
		check(Time >= StartTime);
		EndTime = Time;
	}

	FORCEINLINE void SetAllocThreadId(uint32 ThreadId)
	{
		check((ThreadId & ~MaxThreadId) == 0);
		AllocThreadId = static_cast<uint16>(ThreadId);
	}

	FORCEINLINE void SetFreeThreadId(uint32 ThreadId)
	{
		check((ThreadId & ~MaxThreadId) == 0);
		FreeThreadId = static_cast<uint16>(ThreadId);
	}

	FORCEINLINE HeapId GetHeapId() const
	{
		return HeapId(Heap);
	}

	FORCEINLINE void SetHeapId(HeapId InHeapId)
	{
		check((uint32(InHeapId) & ~MaxHeapId) == 0);
		Heap = InHeapId;
	}

	FORCEINLINE EMemoryTraceHeapAllocationFlags GetFlags() const
	{
		return EMemoryTraceHeapAllocationFlags(Flags);
	}

	FORCEINLINE bool HasAnyFlags(EMemoryTraceHeapAllocationFlags Contains) const
	{
		return EnumHasAnyFlags(EMemoryTraceHeapAllocationFlags(Flags), Contains);
	}

	FORCEINLINE bool IsHeap() const
	{
		return (Flags & uint32(EMemoryTraceHeapAllocationFlags::Heap)) != 0;
	}

	FORCEINLINE bool IsSwap() const
	{
		return (Flags & uint32(EMemoryTraceHeapAllocationFlags::Swap)) != 0;
	}

	FORCEINLINE void SetFlags(EMemoryTraceHeapAllocationFlags InFlags)
	{
		check((uint32(InFlags) & ~MaxFlagsValue) == 0);
		Flags = uint32(InFlags);
	}

	FORCEINLINE void AddHeapFlag()
	{
		Flags |= uint32(EMemoryTraceHeapAllocationFlags::Heap);
	}

	FORCEINLINE void RemoveHeapFlag()
	{
		Flags &= ~uint32(EMemoryTraceHeapAllocationFlags::Heap);
	}

	FORCEINLINE void AddSwapFlag()
	{
		Flags |= uint32(EMemoryTraceHeapAllocationFlags::Swap);
	}

	uint64 Address;
	uint64 Size : 56;
	uint64 AlignmentPow2 : 8;
	uint32 StartEventIndex;
	uint32 EndEventIndex;
	double StartTime;
	double EndTime;
	uint16 AllocThreadId;
	uint16 FreeThreadId;
	uint32 AllocCallstackId;
	uint32 FreeCallstackId;
	uint32 MetadataId;
	TagIdType Tag; // uint32
	uint32 Heap : 24;
	uint32 Flags : 8; // EMemoryTraceHeapAllocationFlags
};

static_assert(sizeof(FAllocationItem) == 64, "struct FAllocationItem needs packing");

} // namespace TraceServices
