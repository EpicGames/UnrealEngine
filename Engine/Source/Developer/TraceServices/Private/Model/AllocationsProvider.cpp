// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllocationsProvider.h"

#include "Containers/ArrayView.h"
#include "Containers/Utf8String.h"
#include "Misc/PathViews.h"
#include "ProfilingDebugging/MemoryTrace.h"

// TraceServices
#include "AllocationsQuery.h"
#include "Common/ProviderLock.h"
#include "Common/Utils.h"
#include "Model/MetadataProvider.h"
#include "SbTree.h"
#include "TraceServices/Containers/Allocators.h"
#include "TraceServices/Model/Callstack.h"

#include <limits>

////////////////////////////////////////////////////////////////////////////////////////////////////

#define INSIGHTS_SLOW_CHECK(expr) //check(expr)

// Initial reserved number for long living allocations.
#define INSIGHTS_LLA_RESERVE 0
//#define INSIGHTS_LLA_RESERVE (64 * 1024)

// Use optimized path for the short living allocations.
// If enabled, caches the short living allocations, as there is a very high chance to be freed in the next few "free" events.
// ~66% of all allocs are expected to have an "event distance" < 64 events
// ~70% of all allocs are expected to have an "event distance" < 512 events
#define INSIGHTS_USE_SHORT_LIVING_ALLOCS 1
#define INSIGHTS_SLA_USE_ADDRESS_MAP 0

// Use optimized path for the last alloc.
// If enabled, caches the last added alloc, as there is a high chance to be freed in the next "free" event.
// ~10% to ~30% of all allocs are expected to have an "event distance" == 1 event ("free" event follows the "alloc" event immediately)
#define INSIGHTS_USE_LAST_ALLOC 1

// Detects cases where an address is allocated multiple times (i.e. missing Free or MarkAllocAsHeap events).
// Note: This slows down analysis significantly, so it is disabled by default.
#define INSIGHTS_VALIDATE_ALLOC_EVENTS 0

// Automatically free the previous alloc when detecting addresses allocated multiple times (INSIGHTS_VALIDATE_ALLOC_EVENTS).
#define INSIGHTS_DOUBLE_ALLOC_FREE_PREVIOUS 1

// Automatically add a fake alloc when Free event detects a missing alloc.
#define INSIGHTS_VALIDATE_FREE_EVENTS 1

// Automatically add a fake alloc when MarkAllocAsHeap event detects a missing alloc.
#define INSIGHTS_VALIDATE_HEAP_MARK_ALLOC_EVENTS 1

// Automatically add a fake heap (or mark an existing alloc as heap) when UnmarkAllocAsHeap detects a missing alloc.
#define INSIGHTS_VALIDATE_HEAP_UNMARK_ALLOC_EVENTS 1

#define INSIGHTS_DEBUG_METADATA 0

// Generates warnings for alloc/free events with nullptr, for the System root heap.
// Note: Allocating nullptr with size != 0 will always generate warnings.
#define INSIGHTS_WARNINGS_FOR_NULLPTR_ALLOCS 0

////////////////////////////////////////////////////////////////////////////////////////////////////

// Debug functionality to discard some of the Alloc/Free/MarkAllocAsHeap/UnmarkAllocAsHeap events.
#define INSIGHTS_FILTER_EVENTS_ENABLED 0

#if INSIGHTS_FILTER_EVENTS_ENABLED
namespace TraceServices
{
bool FAllocationsProvider::ShouldIgnoreEvent(uint32 ThreadId, double Time, uint64 Address, HeapId RootHeapId, uint32 CallstackId)
{
	//if (RootHeapId != EMemoryTraceRootHeap::VideoMemory)
	//{
	//	return true;
	//}
	//if (Address != 0x0ull)
	//{
	//	return true;
	//}
	//if (CallstackId != 0)
	//{
	//	return true;
	//}
	//if (Time > 10.0) // stop analysis after specified time
	//{
	//	EditOnAnalysisCompleted(Time);
	//	bInitialized = false;
	//	return true;
	//}
	return false;
}
} // namespace TraceServices
#define INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeapId, CallstackId) { if (ShouldIgnoreEvent(ThreadId, Time, Address, RootHeapId, CallstackId)) return; }
#else // INSIGHTS_FILTER_EVENTS_ENABLED
#define INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeapId, CallstackId)
#endif // INSIGHTS_FILTER_EVENTS_ENABLED

////////////////////////////////////////////////////////////////////////////////////////////////////

#define INSIGHTS_DEBUG_WATCH 0

#if INSIGHTS_DEBUG_WATCH
namespace TraceServices
{
	static uint64 GWatchAddresses[] =
	{
		// add here addresses to watch
		//0x0ull,
	};
}
#endif //INSIGHTS_DEBUG_WATCH

// Action to be executed when a match is found.
#define INSIGHTS_DEBUG_WATCH_FOUND // just log the API call
//#define INSIGHTS_DEBUG_WATCH_FOUND return // ignore events
//#define INSIGHTS_DEBUG_WATCH_FOUND UE_DEBUG_BREAK()

////////////////////////////////////////////////////////////////////////////////////////////////////

#define INSIGHTS_LOGF(Format, ...) { UE_LOGF(LogTraceServices, Log, "[MemAlloc]" Format, __VA_ARGS__); }
//#define INSIGHTS_LOGF(Format, ...) { FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[MemAlloc]") TEXT(Format) TEXT("\n"), __VA_ARGS__); }

#if 0 || INSIGHTS_DEBUG_WATCH
#define INSIGHTS_API_LOGF(ThreadId, Time, Address, Format, ...) \
	INSIGHTS_LOGF("[API][%3u][%f] " Format, ThreadId, Time, __VA_ARGS__);
#else
#define INSIGHTS_API_LOGF(ThreadId, Time, Address, Format, ...)
#endif

#if 0 || INSIGHTS_DEBUG_WATCH
#define INSIGHTS_INDIRECT_API_LOGF(ApiName, ThreadId, Time, Address) \
	INSIGHTS_LOGF("[API] --> " ApiName " 0x%llX", Address);
#else
#define INSIGHTS_INDIRECT_API_LOGF(ApiName, ThreadId, Time, Address)
#endif

#if INSIGHTS_DEBUG_WATCH
	#define INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, Format, ...) \
	{\
		if (IsAddressWatched(Address))\
		{\
			INSIGHTS_DEBUG_WATCH_FOUND;\
			INSIGHTS_API_LOGF(ThreadId, Time, Address, Format, __VA_ARGS__);\
		}\
	}
	#define INSIGHTS_WATCH_INDIRECT_API_LOGF(ApiName, ThreadId, Time, Address) \
	{\
		if (IsAddressWatched(Address))\
		{\
			INSIGHTS_DEBUG_WATCH_FOUND;\
			INSIGHTS_INDIRECT_API_LOGF(ApiName, ThreadId, Time, Address);\
		}\
	}
#else
	#define INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, Format, ...) INSIGHTS_API_LOGF(ThreadId, Time, Address, Format, __VA_ARGS__)
	#define INSIGHTS_WATCH_INDIRECT_API_LOGF(ApiName, ThreadId, Time, Address) INSIGHTS_INDIRECT_API_LOGF(ApiName, ThreadId, Time, Address)
#endif // INSIGHTS_DEBUG_WATCH

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{

constexpr uint32 MaxLogMessagesPerWarningType = 100;
constexpr uint32 MaxLogMessagesPerErrorType = 100;

#if INSIGHTS_DEBUG_WATCH
static bool IsAddressWatched(uint64 InAddress)
{
	for (int32 AddrIndex = 0; AddrIndex < UE_ARRAY_COUNT(GWatchAddresses); ++AddrIndex)
	{
		if (GWatchAddresses[AddrIndex] == InAddress)
		{
			return true;
		}
	}
	return false;
}
#endif // INSIGHTS_DEBUG_WATCH

thread_local FProviderLock::FThreadLocalState GAllocationsProviderLockState;

static const TCHAR* GDefaultHeapName = TEXT("Unknown");

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTagTracker
////////////////////////////////////////////////////////////////////////////////////////////////////

FTagTracker::FTagTracker(IAnalysisSession& InSession)
	: Session(InSession)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::AddTagSpec(TagIdType InTag, TagIdType InParentTag, const TCHAR* InDisplay)
{
	if (InTag == InvalidTagId)
	{
		++NumErrors;
		if (NumErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Cannot add Tag spec ('%ls') with invalid id!", InDisplay);
		}
		return;
	}

	if (!InDisplay || *InDisplay == TEXT('\0'))
	{
		++NumErrors;
		if (NumErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Tag with id %u has invalid display name (ParentTag=%u)!", InTag, InParentTag);
		}
		InDisplay = TEXT("Unknown");
	}

	// Identify the special "CustomName" tag.
	if (FCString::Strcmp(InDisplay, TEXT("CustomName")) == 0)
	{
		CustomNameTag = InTag;
	}
	// Remove the parent tag if it is "CustomName".
	if (CustomNameTag != InvalidTagId && InParentTag == CustomNameTag)
	{
		InParentTag = InvalidTagId;
	}

	const FTagEntry* TagEntry = TagMap.Find(InTag);
	if (!TagEntry)
	{
		const TCHAR* TagDisplayName;
		const TCHAR* TagFullPath;

		FStringView DisplayName(InDisplay);
		int32 OutIndex;
		if (DisplayName.FindLastChar(TEXT('/'), OutIndex))
		{
			DisplayName.RightChopInline(OutIndex + 1);
			TagDisplayName = Session.StoreString(DisplayName);
			TagFullPath = Session.StoreString(InDisplay);

			// It is possible to define a child tag in runtime using only a string, even if the parent tag does not yet
			// exist. We need to find the correct parent or store it to the side until the parent tag is announced.
			if (InParentTag == InvalidTagId)
			{
				const FStringView Parent = FPathViews::GetPathLeaf(FPathViews::GetPath(InDisplay));
				for (const auto& EntryPair : TagMap)
				{
					const uint32 Id = EntryPair.Get<0>();
					const FTagEntry Entry = EntryPair.Get<1>();
					if (Parent.Equals(Entry.Display))
					{
						InParentTag = Id;
						break;
					}
				}
				// If parent tag is still unknown here, create a temporary entry here
				if (InParentTag == InvalidTagId)
				{
					PendingTags.Emplace(InTag, Parent);
				}
			}
		}
		else
		{
			TagDisplayName = Session.StoreString(DisplayName);
			TStringBuilder<128> FullNameBuilder;
			BuildTagPath(FullNameBuilder, DisplayName, InParentTag);
			TagFullPath = Session.StoreString(FullNameBuilder);
		}

		const FTagEntry& Entry = TagMap.Emplace(InTag, FTagEntry{ TagDisplayName, TagFullPath, InParentTag });

		// Check if this new tag has been referenced before by a child tag
		for (const TTuple<TagIdType,FString>& Pending : PendingTags)
		{
			const TagIdType ReferencingId = Pending.Get<0>();
			const FString& Name = Pending.Get<1>();
			if (DisplayName.Equals(Name))
			{
				TagMap[ReferencingId].ParentTag = InTag;
			}
		}

		UE_LOGF(LogTraceServices, Verbose, "[MemAlloc] Added Tag '%ls' ('%ls') with id %u (ParentTag=%u).", Entry.Display, Entry.FullPath, InTag, InParentTag);
	}
	else
	{
		FStringView DisplayName(InDisplay);
		int32 OutIndex;
		if (DisplayName.FindLastChar(TEXT('/'), OutIndex))
		{
			DisplayName.RightChopInline(OutIndex + 1);
		}

		if (InParentTag == TagEntry->ParentTag && DisplayName.Equals(TagEntry->Display))
		{
			++NumWarnings;
			if (NumWarnings <= MaxLogMessagesPerWarningType)
			{
				UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Tag with id %u (ParentTag=%u, Display='%ls') was already added!", InTag, InParentTag, InDisplay);
			}
		}
		else
		{
			++NumErrors;
			if (NumErrors <= MaxLogMessagesPerErrorType)
			{
				UE_LOGF(LogTraceServices, Error, "[MemAlloc] Tag with id %u (ParentTag=%u, Display='%ls') was already added (ParentTag=%u, Display='%ls')!", InTag, InParentTag, InDisplay, TagEntry->ParentTag, TagEntry->Display);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::BuildTagPath(FStringBuilderBase& OutString, FStringView Name, TagIdType ParentTagId)
{
	if (const FTagEntry* ParentEntry = TagMap.Find(ParentTagId))
	{
		BuildTagPath(OutString, ParentEntry->Display, ParentEntry->ParentTag);
		OutString << TEXT("/");
	}
	OutString << Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::PushTag(uint32 InThreadId, uint8 InTracker, TagIdType InTag)
{
	const uint32 TrackerThreadId = GetTrackerThreadId(InThreadId, InTracker);
	FThreadState& State = TrackerThreadStates.FindOrAdd(TrackerThreadId);
	if (InTag == InvalidTagId)
	{
		++NumErrors;
		if (NumErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid Tag on Thread %u (Tracker=%u)!", InThreadId, uint32(InTracker));
		}
	}
	State.TagStack.Push({ InTag, ETagStackFlags::None });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::PopTag(uint32 InThreadId, uint8 InTracker)
{
	const uint32 TrackerThreadId = GetTrackerThreadId(InThreadId, InTracker);
	FThreadState* State = TrackerThreadStates.Find(TrackerThreadId);
	if (State && !State->TagStack.IsEmpty())
	{
		INSIGHTS_SLOW_CHECK(!State->TagStack.Top().IsPtrScope());
		State->TagStack.Pop();
	}
	else
	{
		++NumErrors;
		if (NumErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Tag stack on Thread %u (Tracker=%u) is already empty!", InThreadId, uint32(InTracker));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TagIdType FTagTracker::GetCurrentTag(uint32 InThreadId, uint8 InTracker) const
{
	const uint32 TrackerThreadId = GetTrackerThreadId(InThreadId, InTracker);
	const FThreadState* State = TrackerThreadStates.Find(TrackerThreadId);
	if (!State || State->TagStack.IsEmpty())
	{
		return 0; // Untagged
	}
	return State->TagStack.Top().Tag;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FTagTracker::GetTagString(TagIdType InTag) const
{
	const FTagEntry* Entry = TagMap.Find(InTag);
	return Entry ? Entry->Display : TEXT("Unknown");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FTagTracker::GetTagFullPath(TagIdType InTag) const
{
	const FTagEntry* Entry = TagMap.Find(InTag);
	return Entry ? Entry->FullPath : TEXT("Unknown");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::EnumerateTags(TFunctionRef<void (const TCHAR*, const TCHAR*, TagIdType, TagIdType)> Callback) const
{
	for (const auto& EntryPair : TagMap)
	{
		const TagIdType Id = EntryPair.Get<0>();
		const FTagEntry& Entry = EntryPair.Get<1>();
		Callback(Entry.Display, Entry.FullPath, Id, Entry.ParentTag);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::PushTagFromPtr(uint32 InThreadId, uint8 InTracker, TagIdType InTag)
{
	const uint32 TrackerThreadId = GetTrackerThreadId(InThreadId, InTracker);
	FThreadState& State = TrackerThreadStates.FindOrAdd(TrackerThreadId);
	State.TagStack.Push({ InTag, ETagStackFlags::PtrScope });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTagTracker::PopTagFromPtr(uint32 InThreadId, uint8 InTracker)
{
	const uint32 TrackerThreadId = GetTrackerThreadId(InThreadId, InTracker);
	FThreadState* State = TrackerThreadStates.Find(TrackerThreadId);
	if (State && !State->TagStack.IsEmpty())
	{
		INSIGHTS_SLOW_CHECK(State->TagStack.Top().IsPtrScope());
		State->TagStack.Pop();
	}
	else
	{
		++NumErrors;
		if (NumErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Tag stack on Thread %u (Tracker=%u) is already empty!", InThreadId, uint32(InTracker));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTagTracker::HasTagFromPtrScope(uint32 InThreadId, uint8 InTracker) const
{
	const uint32 TrackerThreadId = GetTrackerThreadId(InThreadId, InTracker);
	const FThreadState* State = TrackerThreadStates.Find(TrackerThreadId);
	return State && (State->TagStack.Num() > 0) && State->TagStack.Top().IsPtrScope();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// IAllocationsProvider::FAllocation
////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetStartEventIndex() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->StartEventIndex;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetEndEventIndex() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->EndEventIndex;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double IAllocationsProvider::FAllocation::GetStartTime() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->StartTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double IAllocationsProvider::FAllocation::GetEndTime() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->EndTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 IAllocationsProvider::FAllocation::GetAddress() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->Address;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 IAllocationsProvider::FAllocation::GetSize() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->GetSize();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetAlignment() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->GetAlignment();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetAllocThreadId() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->AllocThreadId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetFreeThreadId() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->FreeThreadId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetAllocCallstackId() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->AllocCallstackId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocation::GetFreeCallstackId() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->FreeCallstackId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
uint32 IAllocationsProvider::FAllocation::GetMetadataId() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->MetadataId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TagIdType IAllocationsProvider::FAllocation::GetTag() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->Tag;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

HeapId IAllocationsProvider::FAllocation::GetRootHeap() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->Heap;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool IAllocationsProvider::FAllocation::IsHeap() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->IsHeap();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool IAllocationsProvider::FAllocation::IsSwap() const
{
	const auto* Inner = (const FAllocationItem*)this;
	return Inner->IsSwap();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// IAllocationsProvider::FAllocations
////////////////////////////////////////////////////////////////////////////////////////////////////

void IAllocationsProvider::FAllocations::operator delete (void* Address)
{
	auto* Inner = (const FAllocationsImpl*)Address;
	delete Inner;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 IAllocationsProvider::FAllocations::Num() const
{
	auto* Inner = (const FAllocationsImpl*)this;
	return (uint32)(Inner->Items.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAllocationsProvider::FAllocation* IAllocationsProvider::FAllocations::Get(uint32 Index) const
{
	checkSlow(Index < Num());

	auto* Inner = (const FAllocationsImpl*)this;
	return (const FAllocation*)(Inner->Items[Index]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// IAllocationsProvider::FQueryStatus
////////////////////////////////////////////////////////////////////////////////////////////////////

IAllocationsProvider::FQueryResult IAllocationsProvider::FQueryStatus::NextResult() const
{
	auto* Inner = (FAllocationsImpl*)Handle;
	if (Inner == nullptr)
	{
		return nullptr;
	}

	Handle = UPTRINT(Inner->Next);

	auto* Ret = (FAllocations*)Inner;
	return FQueryResult(Ret);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FShortLivingAllocs
////////////////////////////////////////////////////////////////////////////////////////////////////

FShortLivingAllocs::FShortLivingAllocs()
{
#if INSIGHTS_SLA_USE_ADDRESS_MAP
	AddressMap.Reserve(MaxAllocCount);
#endif

	AllNodes = new FNode[MaxAllocCount];

	// Build the "unused" simple linked list.
	FirstUnusedNode = AllNodes;
	for (int32 Index = 0; Index < MaxAllocCount - 1; ++Index)
	{
		AllNodes[Index].Next = &AllNodes[Index + 1];
	}
	AllNodes[MaxAllocCount - 1].Next = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FShortLivingAllocs::~FShortLivingAllocs()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FShortLivingAllocs::Reset()
{
#if INSIGHTS_SLA_USE_ADDRESS_MAP
	AddressMap.Reset();
#endif

	FNode* Node = LastAddedAllocNode;
	while (Node != nullptr)
	{
		delete Node->Alloc;
		Node = Node->Prev;
	}

	delete[] AllNodes;
	AllNodes = nullptr;

	LastAddedAllocNode = nullptr;
	OldestAllocNode = nullptr;
	FirstUnusedNode = nullptr;
	AllocCount = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FShortLivingAllocs::FindRef(uint64 Address) const
{
#if INSIGHTS_SLA_USE_ADDRESS_MAP
	FNode** NodePtr = AddressMap.Find(Address);
	return NodePtr ? (*NodePtr)->Alloc : nullptr;
#else
	// Search linearly the list of allocations, backward, starting with most recent one.
	// As the probability of finding a match for a "free" event is higher for the recent allocs,
	// this could actually be faster than doing a O(log n) search in AddressMap.
	FNode* Node = LastAddedAllocNode;
	while (Node != nullptr)
	{
		if (Node->Alloc->Address == Address)
		{
			return Node->Alloc;
		}
		Node = Node->Prev;
	}
	return nullptr;
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FShortLivingAllocs::FindRange(const uint64 Address) const
{
	//TODO: Can we do better than iterating all the nodes?
	FNode* Node = LastAddedAllocNode;
	while (Node != nullptr)
	{
		if (Node->Alloc->IsContained(Address))
		{
			return Node->Alloc;
		}
		Node = Node->Prev;
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FShortLivingAllocs::Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	FNode* Node = LastAddedAllocNode;
	while (Node != nullptr)
	{
		Callback(*Node->Alloc);
		Node = Node->Prev;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FShortLivingAllocs::Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	FNode* Node = LastAddedAllocNode;
	while (Node != nullptr)
	{
		const FAllocationItem& Alloc = *Node->Alloc;
		if (Alloc.Address >= StartAddress && Alloc.Address < EndAddress)
		{
			Callback(Alloc);
		}
		Node = Node->Prev;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FShortLivingAllocs::AddAndRemoveOldest(FAllocationItem* Alloc)
{
	if (FirstUnusedNode == nullptr)
	{
		// Collection is already full.
		INSIGHTS_SLOW_CHECK(AllocCount == MaxAllocCount);
		INSIGHTS_SLOW_CHECK(OldestAllocNode != nullptr);
		INSIGHTS_SLOW_CHECK(LastAddedAllocNode != nullptr);

		// Reuse the node of the oldest allocation for the new allocation.
		FNode* NewNode = OldestAllocNode;

		// Remove the oldest allocation.
		FAllocationItem* RemovedAlloc = OldestAllocNode->Alloc;
#if INSIGHTS_SLA_USE_ADDRESS_MAP
		AddressMap.Remove(RemovedAlloc->Address);
#endif
		OldestAllocNode = OldestAllocNode->Next;
		INSIGHTS_SLOW_CHECK(OldestAllocNode != nullptr);
		OldestAllocNode->Prev = nullptr;

		// Add the new node.
#if INSIGHTS_SLA_USE_ADDRESS_MAP
		AddressMap.Add(Alloc->Address, NewNode);
#endif
		NewNode->Alloc = Alloc;
		NewNode->Next = nullptr;
		NewNode->Prev = LastAddedAllocNode;
		LastAddedAllocNode->Next = NewNode;
		LastAddedAllocNode = NewNode;

		return RemovedAlloc;
	}
	else
	{
		INSIGHTS_SLOW_CHECK(AllocCount < MaxAllocCount);
		++AllocCount;

		FNode* NewNode = FirstUnusedNode;
		FirstUnusedNode = FirstUnusedNode->Next;

		// Add the new node.
#if INSIGHTS_SLA_USE_ADDRESS_MAP
		AddressMap.Add(Alloc->Address, NewNode);
#endif
		NewNode->Alloc = Alloc;
		NewNode->Next = nullptr;
		NewNode->Prev = LastAddedAllocNode;
		if (LastAddedAllocNode)
		{
			LastAddedAllocNode->Next = NewNode;
		}
		else
		{
			OldestAllocNode = NewNode;
		}
		LastAddedAllocNode = NewNode;

		return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FShortLivingAllocs::Remove(uint64 Address)
{
#if INSIGHTS_SLA_USE_ADDRESS_MAP
	FNode* Node = nullptr;
	if (!AddressMap.RemoveAndCopyValue(Address, Node))
	{
		return nullptr;
	}
	INSIGHTS_SLOW_CHECK(Node && Node->Alloc && Node->Alloc->Address == Address);
#else
	// Search linearly the list of allocations, backward, starting with most recent one.
	// As the probability of finding a match for a "free" event is higher for the recent allocs,
	// this could actually be faster than doing a O(log n) search in AddressMap.
	FNode* Node = LastAddedAllocNode;
	while (Node != nullptr)
	{
		if (Node->Alloc->Address == Address)
		{
			break;
		}
		Node = Node->Prev;
	}
	if (!Node)
	{
		return nullptr;
	}
#endif // INSIGHTS_SLA_USE_ADDRESS_MAP

	INSIGHTS_SLOW_CHECK(AllocCount > 0);
	--AllocCount;

	// Remove node.
	if (Node->Prev)
	{
		Node->Prev->Next = Node->Next;
	}
	else
	{
		INSIGHTS_SLOW_CHECK(Node == OldestAllocNode);
		OldestAllocNode = Node->Next;
	}
	if (Node->Next)
	{
		Node->Next->Prev = Node->Prev;
	}
	else
	{
		INSIGHTS_SLOW_CHECK(Node == LastAddedAllocNode)
		LastAddedAllocNode = Node->Prev;
	}

	// Add the removed node to the "unused" list.
	Node->Next = FirstUnusedNode;
	FirstUnusedNode = Node;

	return Node->Alloc;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FHeapAllocs
////////////////////////////////////////////////////////////////////////////////////////////////////

FHeapAllocs::FHeapAllocs()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FHeapAllocs::~FHeapAllocs()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FHeapAllocs::Reset()
{
	for (auto& KV : AddressMap)
	{
		FList& List = KV.Value;
		const FNode* Node = List.First;
		while (Node != nullptr)
		{
			FNode* NextNode = Node->Next;
			delete Node->Alloc;
			delete Node;
			Node = NextNode;
		}
		List.First = nullptr;
		List.Last = nullptr;
	}
	AddressMap.Reset();
	AllocCount = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FHeapAllocs::FindRef(uint64 Address) const
{
	const FList* ListPtr = AddressMap.Find(Address);
	if (ListPtr)
	{
		// Search in reverse added order.
		for (const FNode* Node = ListPtr->Last; Node != nullptr; Node = Node->Prev)
		{
			if (Address == Node->Alloc->Address)
			{
				return Node->Alloc;
			}
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FHeapAllocs::FindRange(uint64 Address) const
{
	for (const auto& KV : AddressMap)
	{
		// Search in reverse added order.
		for (const FNode* Node = KV.Value.Last; Node != nullptr; Node = Node->Prev)
		{
			if (Node->Alloc->IsContained(Address))
			{
				return Node->Alloc;
			}
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FHeapAllocs::Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	for (const auto& KV : AddressMap)
	{
		// Iterate in same order as added.
		for (const FNode* Node = KV.Value.First; Node != nullptr; Node = Node->Next)
		{
			Callback(*Node->Alloc);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FHeapAllocs::Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	for (const auto& KV : AddressMap)
	{
		if (KV.Key >= StartAddress && KV.Key < EndAddress)
		{
			// Iterate in same order as added.
			for (const FNode* Node = KV.Value.First; Node != nullptr; Node = Node->Next)
			{
				//check(Node->Alloc->Address == KV.Key);
				Callback(*Node->Alloc);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FHeapAllocs::Add(FAllocationItem* Alloc)
{
	FNode* Node = new FNode();
	Node->Alloc = Alloc;
	Node->Next = nullptr;

	FList& List = AddressMap.FindOrAdd(Alloc->Address);
	if (List.Last == nullptr)
	{
		List.First = Node;
	}
	else
	{
		List.Last->Next = Node;
	}
	Node->Prev = List.Last;
	List.Last = Node;

	++AllocCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FHeapAllocs::Remove(uint64 Address)
{
	FList* ListPtr = AddressMap.Find(Address);
	if (ListPtr)
	{
		// Remove the last added heap alloc.
		check(ListPtr->Last != nullptr);
		FAllocationItem* Alloc = ListPtr->Last->Alloc;

		if (ListPtr->First == ListPtr->Last)
		{
			// Remove the entire linked list from the map (as it has only one node).
			delete ListPtr->Last;
			AddressMap.FindAndRemoveChecked(Address);
		}
		else
		{
			// Remove only the last node from the linked list of heap allocs with specified address.
			FNode* LastNode = ListPtr->Last;
			ListPtr->Last = LastNode->Prev;
			ListPtr->Last->Next = nullptr;
			delete LastNode;
		}

		--AllocCount;
		return Alloc;
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLiveAllocCollection
////////////////////////////////////////////////////////////////////////////////////////////////////

FLiveAllocCollection::FLiveAllocCollection()
{
#if INSIGHTS_LLA_RESERVE
	LongLivingAllocs.Reserve(INSIGHTS_LLA_RESERVE);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLiveAllocCollection::~FLiveAllocCollection()
{
	HeapAllocs.Reset();
	SwapAllocs.Reset();
	LongLivingAllocs.Reset();
	ShortLivingAllocs.Reset();

#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc)
	{
		delete LastAlloc;
		LastAlloc = nullptr;
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::FindRef(uint64 Address) const
{
#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc && LastAlloc->Address == Address)
	{
		return LastAlloc;
	}
#endif

#if INSIGHTS_USE_SHORT_LIVING_ALLOCS
	FAllocationItem* FoundShortLivingAlloc = ShortLivingAllocs.FindRef(Address);
	if (FoundShortLivingAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundShortLivingAlloc->Address == Address);
		return FoundShortLivingAlloc;
	}
#endif

	FAllocationItem* FoundLongLivingAlloc = LongLivingAllocs.FindRef(Address);
	if (FoundLongLivingAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundLongLivingAlloc->Address == Address);
		return FoundLongLivingAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::FindHeapRef(uint64 Address) const
{
	FAllocationItem* FoundHeapAlloc = HeapAllocs.FindRef(Address);
	if (FoundHeapAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundHeapAlloc->Address == Address);
		return FoundHeapAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::FindSwapRef(uint64 Address) const
{
	FAllocationItem* FoundSwapAlloc = SwapAllocs.FindRef(Address);
	if (FoundSwapAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundSwapAlloc->Address == Address);
		return FoundSwapAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::FindByAddressRange(uint64 Address) const
{
#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc && LastAlloc->IsContained(Address))
	{
		return LastAlloc;
	}
#endif

#if INSIGHTS_USE_SHORT_LIVING_ALLOCS
	FAllocationItem* FoundShortLivingAlloc = ShortLivingAllocs.FindRange(Address);
	if (FoundShortLivingAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundShortLivingAlloc->IsContained(Address));
		return FoundShortLivingAlloc;
	}
#endif

	FAllocationItem* FoundLongLivingAlloc = LongLivingAllocs.FindRange(Address);
	if (FoundLongLivingAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundLongLivingAlloc->IsContained(Address));
		return FoundLongLivingAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::FindHeapByAddressRange(uint64 Address) const
{
	FAllocationItem* FoundHeapAlloc = HeapAllocs.FindRange(Address);
	if (FoundHeapAlloc)
	{
		INSIGHTS_SLOW_CHECK(FoundHeapAlloc->IsContained(Address));
		return FoundHeapAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLiveAllocCollection::Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	HeapAllocs.Enumerate(Callback);
	SwapAllocs.Enumerate(Callback);

#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc)
	{
		Callback(*LastAlloc);
	}
#endif

#if INSIGHTS_USE_SHORT_LIVING_ALLOCS
	ShortLivingAllocs.Enumerate(Callback);
#endif

	LongLivingAllocs.Enumerate(Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLiveAllocCollection::Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	HeapAllocs.Enumerate(StartAddress, EndAddress, Callback);
	SwapAllocs.Enumerate(StartAddress, EndAddress, Callback);

#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc && LastAlloc->Address >= StartAddress && LastAlloc->Address < EndAddress)
	{
		Callback(*LastAlloc);
	}
#endif

#if INSIGHTS_USE_SHORT_LIVING_ALLOCS
	ShortLivingAllocs.Enumerate(StartAddress, EndAddress, Callback);
#endif

	LongLivingAllocs.Enumerate(StartAddress, EndAddress, Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::AddNew(uint64 Address)
{
	FAllocationItem* Alloc = new FAllocationItem();
	Alloc->Address = Address;

	Add(Alloc);
	return Alloc;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLiveAllocCollection::Add(FAllocationItem* Alloc)
{
	check(!Alloc->IsHeap());

	++TotalAllocCount;
	if (TotalAllocCount > MaxAllocCount)
	{
		MaxAllocCount = TotalAllocCount;
	}

#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc)
	{
		// We have a new "last allocation".
		// The previous one will be moved to the short living allocations.
		FAllocationItem* PrevLastAlloc = LastAlloc;
		LastAlloc = Alloc;
		Alloc = PrevLastAlloc;
	}
	else
	{
		LastAlloc = Alloc;
		return;
	}
#endif

#if INSIGHTS_USE_SHORT_LIVING_ALLOCS
	Alloc = ShortLivingAllocs.AddAndRemoveOldest(Alloc);
	if (Alloc == nullptr)
	{
		return;
	}
#endif

	LongLivingAllocs.Add(Alloc);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::AddNewHeap(uint64 Address)
{
	FAllocationItem* NewAlloc = new FAllocationItem();
	NewAlloc->Address = Address;
	NewAlloc->AddHeapFlag();

	AddHeap(NewAlloc);
	return NewAlloc;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLiveAllocCollection::AddHeap(FAllocationItem* HeapAlloc)
{
	check(HeapAlloc->IsHeap());

	++TotalAllocCount;
	if (TotalAllocCount > MaxAllocCount)
	{
		MaxAllocCount = TotalAllocCount;
	}

	HeapAllocs.Add(HeapAlloc);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::Remove(uint64 Address)
{
#if INSIGHTS_USE_LAST_ALLOC
	if (LastAlloc && LastAlloc->Address == Address)
	{
		FAllocationItem* RemovedAlloc = LastAlloc;
		LastAlloc = nullptr;
		INSIGHTS_SLOW_CHECK(TotalAllocCount > 0);
		--TotalAllocCount;
		return RemovedAlloc;
	}
#endif

#if INSIGHTS_USE_SHORT_LIVING_ALLOCS
	FAllocationItem* RemovedShortLivingAlloc = ShortLivingAllocs.Remove(Address);
	if (RemovedShortLivingAlloc)
	{
		INSIGHTS_SLOW_CHECK(RemovedShortLivingAlloc->Address == Address);
		INSIGHTS_SLOW_CHECK(TotalAllocCount > 0);
		--TotalAllocCount;
		return RemovedShortLivingAlloc;
	}
#endif

	FAllocationItem* RemovedLongLivingAlloc = LongLivingAllocs.Remove(Address);
	if (RemovedLongLivingAlloc)
	{
		INSIGHTS_SLOW_CHECK(RemovedLongLivingAlloc->Address == Address);
		INSIGHTS_SLOW_CHECK(TotalAllocCount > 0);
		--TotalAllocCount;
		return RemovedLongLivingAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::RemoveHeap(uint64 Address)
{
	FAllocationItem* RemovedHeapAlloc = HeapAllocs.Remove(Address);
	if (RemovedHeapAlloc)
	{
		INSIGHTS_SLOW_CHECK(RemovedHeapAlloc->Address == Address);
		INSIGHTS_SLOW_CHECK(TotalAllocCount > 0);
		--TotalAllocCount;
		return RemovedHeapAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::AddNewSwap(uint64 Address)
{
	FAllocationItem* NewAlloc = new FAllocationItem();
	NewAlloc->Address = Address;
	NewAlloc->AddSwapFlag();

	AddSwap(NewAlloc);
	return NewAlloc;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLiveAllocCollection::AddSwap(FAllocationItem* SwapAlloc)
{
	check(SwapAlloc->IsSwap());

	++TotalAllocCount;
	if (TotalAllocCount > MaxAllocCount)
	{
		MaxAllocCount = TotalAllocCount;
	}

	SwapAllocs.Add(SwapAlloc);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationItem* FLiveAllocCollection::RemoveSwap(uint64 Address)
{
	FAllocationItem* RemovedSwapAlloc = SwapAllocs.Remove(Address);
	if (RemovedSwapAlloc)
	{
		INSIGHTS_SLOW_CHECK(RemovedSwapAlloc->Address == Address);
		INSIGHTS_SLOW_CHECK(TotalAllocCount > 0);
		--TotalAllocCount;
		return RemovedSwapAlloc;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAllocationsProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsProvider::FAllocationsProvider(IAnalysisSession& InSession, FMetadataProvider& InMetadataProvider)
	: Session(InSession)
	, MetadataProvider(InMetadataProvider)
	, TagTracker(InSession)
	, Timeline(Session.GetLinearAllocator(), 1024)
{
	// Allocate the timelines
	for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU64; ++TimelineIndex)
	{
		ValueTimelinesU64[TimelineIndex] = new TPagedArray<uint64>(Session.GetLinearAllocator(), 1024);
		SampleValuesU64[TimelineIndex] = 0;
	}
	for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU32; ++TimelineIndex)
	{
		ValueTimelinesU32[TimelineIndex] = new TPagedArray<uint32>(Session.GetLinearAllocator(), 1024);
		SampleValuesU32[TimelineIndex] = 0;
	}

	// Initial number of heap spec locations. Actual number of heap specs is unlimited (uint32).
	HeapSpecs.AddDefaulted(FMath::Max(1024u, MaxRootHeaps));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsProvider::~FAllocationsProvider()
{
	// Delete root heaps.
	for (uint32 RootHeapIdx = 0; RootHeapIdx < MaxRootHeaps; ++RootHeapIdx)
	{
		FRootHeap* RootHeap = RootHeaps[RootHeapIdx];
		if (RootHeap)
		{
			RootHeap->HeapSpec = nullptr;

			if (RootHeap->SbTree)
			{
				delete RootHeap->SbTree;
				RootHeap->SbTree = nullptr;
			}
			if (RootHeap->LiveAllocs)
			{
				delete RootHeap->LiveAllocs;
				RootHeap->LiveAllocs = nullptr;
			}

			delete RootHeap;
			RootHeaps[RootHeapIdx] = nullptr;
		}
	}

	// Delete all heap specs.
	for (FHeapSpec* HeapSpec : HeapSpecs)
	{
		if (HeapSpec)
		{
			delete HeapSpec;
		}
	}
	HeapSpecs.Reset();

	// Delete the timelines.
	for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU64; ++TimelineIndex)
	{
		if (ValueTimelinesU64[TimelineIndex])
		{
			delete ValueTimelinesU64[TimelineIndex];
			ValueTimelinesU64[TimelineIndex] = nullptr;
		}
	}
	for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU32; ++TimelineIndex)
	{
		if (ValueTimelinesU32[TimelineIndex])
		{
			delete ValueTimelinesU32[TimelineIndex];
			ValueTimelinesU32[TimelineIndex] = nullptr;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditInit(double InTime, uint8 InMinAlignment, uint64 InPlatformPageSize)
{
	EditAccessCheck();

	if (bInitialized)
	{
		// error: already initialized
		return;
	}

	INSIGHTS_API_LOGF(0u, InTime, 0ull, "Init : MinAlignment=%u)", uint32(InMinAlignment));

	InitTime = InTime;
	PlatformPageSize = InPlatformPageSize;
	MinAlignment = InMinAlignment;

	// Create system root heap structures for backwards compatibility (before heap description events)
	AddHeapSpec(EMemoryTraceRootHeap::SystemMemory, 0, TEXT("System memory"), EMemoryTraceHeapFlags::Root);

	bInitialized = true;

	AdvanceTimelines(InTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditAlloc(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeapId)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		return;
	}

	++AllocEvents;
	AllocInternal(ThreadId, Time, CallstackId, Address, Size, Alignment, RootHeapId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::AllocInternal(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeapId)
{
	INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, "Alloc 0x%llX : Size=%llu Alignment=%u RootHeap=%u CallstackId=%u", Address, Size, Alignment, RootHeapId, CallstackId);
	INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeapId, CallstackId);

	if (Address == 0 && RootHeapId == EMemoryTraceRootHeap::SystemMemory)
	{
#if !INSIGHTS_WARNINGS_FOR_NULLPTR_ALLOCS
		if (Size == 0)
		{
			return;
		}
#endif // INSIGHTS_WARNINGS_FOR_NULLPTR_ALLOCS
		++AllocWarnings;
		if (AllocWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Alloc at address 0 : Size=%llu, RootHeap=%u, Time=%f, CallstackId=%u", Size, RootHeapId, Time, CallstackId);
		}
		return;
	}

	if (!IsValidRootHeap(RootHeapId))
	{
		++AllocWarnings;
		if (AllocWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Alloc with invalid root heap id (%u).", RootHeapId);
		}
		return;
	}

	FRootHeap& RootHeap = *RootHeaps[RootHeapId];

	RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time);

	AdvanceTimelines(Time);

	const uint8 Tracker = 0;
	const TagIdType Tag = TagTracker.GetCurrentTag(ThreadId, Tracker);

#if INSIGHTS_VALIDATE_ALLOC_EVENTS
	FAllocationItem* ExistingAllocationPtr = RootHeap.LiveAllocs->FindRef(Address);
	if (ExistingAllocationPtr)
	{
#if INSIGHTS_DOUBLE_ALLOC_FREE_PREVIOUS

		++AllocErrors;
		if (AllocErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid ALLOC event (Address=0x%llX, Size=%llu, Tag=%u, RootHeap=%u, Time=%f, CallstackId=%u)! The previous alloc will be freed.", Address, Size, Tag, RootHeapId, Time, CallstackId);
		}

		// Free the previous allocation.
		INSIGHTS_WATCH_INDIRECT_API_LOGF("Free", ThreadId, Time, Address);
		constexpr uint32 FreeCallstackId = 0; // no callstack
		FreeInternal(ThreadId, Time, FreeCallstackId, Address, RootHeapId);
		RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time); // for the case where the next event is first event in a new SbTree column after changing the heap alloc

#else // INSIGHTS_DOUBLE_ALLOC_FREE_PREVIOUS

		++AllocErrors;
		if (AllocErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid ALLOC event (Address=0x%llX, Size=%llu, Tag=%u, RootHeap=%u, Time=%f, CallstackId=%u)!", Address, Size, Tag, RootHeapId, Time, CallstackId);
		}

#endif // INSIGHTS_DOUBLE_ALLOC_FREE_PREVIOUS
	}
#endif // INSIGHTS_VALIDATE_ALLOC_EVENTS

	FAllocationItem* Alloc = RootHeap.LiveAllocs->AddNew(Address);
	FAllocationItem& Allocation = *Alloc;

	uint32 MetadataId = MetadataProvider.InvalidMetadataId;
	{
		FProviderEditScopeLock _(MetadataProvider);
		MetadataId = MetadataProvider.PinAndGetId(ThreadId);
#if INSIGHTS_DEBUG_METADATA
		if (MetadataId != FMetadataProvider::InvalidMetadataId &&
			!TagTracker.HasTagFromPtrScope(ThreadId, Tracker))
		{
			const uint32 MetadataStackSize = MetadataProvider.GetMetadataStackSize(ThreadId, MetadataId);
			check(MetadataStackSize > 0);
			uint16 MetaType;
			const void* MetaData;
			uint32 MetaDataSize;
			MetadataProvider.GetMetadata(ThreadId, MetadataId, MetadataStackSize - 1, MetaType, MetaData, MetaDataSize);
			if (MetaType == 0) // "MemTagId"
			{
				TagIdType MetaMemTag = *(TagIdType*)MetaData;
				ensure(MetaMemTag == Tag);
			}
		}
#endif
	}

	INSIGHTS_SLOW_CHECK(Allocation.Address == Address);
	Allocation.SetSize(Size);
	Allocation.SetAlignment(Alignment);
	Allocation.StartEventIndex = RootHeap.EventIndex;
	Allocation.EndEventIndex = (uint32)-1;
	Allocation.StartTime = Time;
	Allocation.EndTime = std::numeric_limits<double>::infinity();
	Allocation.SetAllocThreadId(ThreadId);
	Allocation.FreeThreadId = 0;
	Allocation.AllocCallstackId = CallstackId;
	Allocation.FreeCallstackId = 0; // no callstack yet
	Allocation.MetadataId = MetadataId;
	Allocation.Tag = Tag;
	Allocation.SetHeapId(RootHeapId);
	Allocation.SetFlags(EMemoryTraceHeapAllocationFlags::None);

	RootHeap.UpdateHistogramByAllocSize(Size);

	// Update the timeline stats (Alloc added to LiveAllocs as a new normal "alloc").
	TotalAllocatedMemory += Size;
	++TotalLiveAllocations;
	SetMaxSampleU64(ETimelineU64::MaxTotalAllocatedMemory, TotalAllocatedMemory);
	SetMaxSampleU32(ETimelineU32::MaxLiveAllocations, TotalLiveAllocations);
	IncSampleU32(ETimelineU32::AllocEvents);

	++AllocCount;
	if (RootHeap.EventIndex != ~0u)
	{
		RootHeap.EventIndex++;
	}
	else
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] Too many events!");
		bInitialized = false; // ignore further events
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditFree(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId RootHeapId)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		return;
	}

	++FreeEvents;
	FreeInternal(ThreadId, Time, CallstackId, Address, RootHeapId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::FreeInternal(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId RootHeapId)
{
	INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, "Free 0x%llX : RootHeap=%u CallstackId=%u", Address, RootHeapId, CallstackId);
	INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeapId, CallstackId);

	if (Address == 0 && RootHeapId == EMemoryTraceRootHeap::SystemMemory)
	{
#if INSIGHTS_WARNINGS_FOR_NULLPTR_ALLOCS
		++FreeWarnings;
		if (FreeWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Free for address 0 : RootHeap=%u, Time=%f, CallstackId=%u", RootHeapId, Time, CallstackId);
		}
#endif // INSIGHTS_WARNINGS_FOR_NULLPTR_ALLOCS
		return;
	}

	if (!IsValidRootHeap(RootHeapId))
	{
		++FreeWarnings;
		if (FreeWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Free with invalid root heap id (%u).", RootHeapId);
		}
		return;
	}

	FRootHeap& RootHeap = *RootHeaps[RootHeapId];

	RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time);

	AdvanceTimelines(Time);

	FAllocationItem* Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc
	if (!Alloc)
	{
		// Try to free a heap allocation.
		Alloc = RootHeap.LiveAllocs->FindHeapRef(Address);
		if (Alloc)
		{
			INSIGHTS_WATCH_INDIRECT_API_LOGF("UnmarkAllocationAsHeap", ThreadId, Time, Address);
			EditUnmarkAllocationAsHeap(ThreadId, Time, CallstackId, Address, Alloc->Heap);
			RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time); // for the case where the next event is first event in a new SbTree column after changing the heap alloc
			Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc
			check(Alloc != nullptr);
		}
	}

#if INSIGHTS_VALIDATE_FREE_EVENTS
	if (!Alloc)
	{
		++FreeErrors;
		if (FreeErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid FREE event (Address=0x%llX, RootHeap=%u, Time=%f, CallstackId=%u)! A fake alloc will be created with size 0.", Address, RootHeapId, Time, CallstackId);
		}

		// Fake the missing alloc.
		constexpr uint64 FakeAllocSize = 0;
		constexpr uint32 FakeAllocAlignment = 0;
		INSIGHTS_WATCH_INDIRECT_API_LOGF("Alloc", ThreadId, Time, Address);
		AllocInternal(ThreadId, Time, CallstackId, Address, FakeAllocSize, FakeAllocAlignment, RootHeapId);
		RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time); // for the case where the next event is first event in a new SbTree column after adding the fake alloc
		Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc
		check(Alloc != nullptr);
	}
#endif // INSIGHTS_VALIDATE_FREE_EVENTS

	if (Alloc)
	{
		ensure(!Alloc->IsHeap());

		Alloc->SetEndEventIndex(RootHeap.EventIndex);
		Alloc->SetEndTime(Time);
		Alloc->SetFreeThreadId(ThreadId);
		Alloc->FreeCallstackId = CallstackId;

		const uint64 AllocSize = Alloc->GetSize();
		const uint32 EventDistance = Alloc->GetEventDistance();

		RootHeap.SbTree->AddAlloc(Alloc); // SbTree takes ownership of Alloc
		Alloc = nullptr;

		RootHeap.UpdateHistogramByEventDistance(EventDistance);

		// Update stats for the current timeline sample (Alloc removed from LiveAllocs).
		TotalAllocatedMemory -= AllocSize;
		--TotalLiveAllocations;
		SetMinSampleU64(ETimelineU64::MinTotalAllocatedMemory, TotalAllocatedMemory);
		SetMinSampleU32(ETimelineU32::MinLiveAllocations, TotalLiveAllocations);
		IncSampleU32(ETimelineU32::FreeEvents);
	}
	else
	{
		++FreeErrors;
		if (FreeErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid FREE event (Address=0x%llX, RootHeap=%u, Time=%f, CallstackId=%u)!", Address, RootHeapId, Time, CallstackId);
		}
	}

	++FreeCount;
	if (RootHeap.EventIndex != ~0u)
	{
		RootHeap.EventIndex++;
	}
	else
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] Too many events!");
		bInitialized = false; // ignore further events
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditUpdateAlloc(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId RootHeapId)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		return;
	}

	INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, "UpdateAlloc 0x%llX : RootHeap=%u CallstackId=%u", Address, RootHeapId, CallstackId);
	INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeapId, CallstackId);

	if (!IsValidRootHeap(RootHeapId))
	{
		++MiscWarnings;
		if (MiscWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Update alloc with invalid root heap id (%u).", RootHeapId);
		}
		return;
	}

	FRootHeap& RootHeap = *RootHeaps[RootHeapId];

	// Try to update a regular alloc.
	FAllocationItem* Alloc = RootHeap.LiveAllocs->FindRef(Address); // alloc owned by LiveAllocs
	if (Alloc)
	{
		const uint32 AllocCallstackId = Alloc->AllocCallstackId;
		const uint64 AllocSize = Alloc->GetSize();
		const uint32 AllocAlignment = Alloc->GetAlignment();
		FreeInternal(ThreadId, Time, CallstackId, Address, RootHeapId);
		AllocInternal(ThreadId, Time, AllocCallstackId, Address, AllocSize, AllocAlignment, RootHeapId);
		return;
	}

	// Try to update a "heap" alloc.
	Alloc = RootHeap.LiveAllocs->FindHeapRef(Address); // alloc owned by LiveAllocs
	if (Alloc)
	{
		const uint32 AllocCallstackId = Alloc->AllocCallstackId;
		const uint64 AllocSize = Alloc->GetSize();
		const uint32 AllocAlignment = Alloc->GetAlignment();
		const HeapId AllocHeapId = Alloc->GetHeapId();
		const EMemoryTraceHeapAllocationFlags AllocHeapFlags = Alloc->GetFlags();
		FreeInternal(ThreadId, Time, CallstackId, Address, RootHeapId);
		AllocInternal(ThreadId, Time, AllocCallstackId, Address, AllocSize, AllocAlignment, RootHeapId);
		EditMarkAllocationAsHeap(ThreadId, Time, AllocCallstackId, Address, AllocHeapId, AllocHeapFlags);
		return;
	}

	++MiscErrors;
	if (MiscErrors <= MaxLogMessagesPerErrorType)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] UpdateAlloc: Address 0x%llX not found (RootHeap=%u, Time=%f, CallstackId=%u)!", Address, RootHeapId, Time, CallstackId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditSwapOp(uint32 ThreadId, double Time, uint64 UnmaskedPageAddress, EMemoryTraceSwapOperation SwapOp, uint64 CompressedPageSize, uint32 CallstackId)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		return;
	}

	++SwapOpEvents;

	HeapId RootHeapId = EMemoryTraceRootHeap::SystemMemory;

	if (!IsValidRootHeap(RootHeapId))
	{
		return;
	}

	FRootHeap& RootHeap = *RootHeaps[RootHeapId];

	const uint64 MaskedPagedAddress = UnmaskedPageAddress & (~(PlatformPageSize - 1));

	// Emit fake "page in" if there was a previous "page out" with the same address.
	if (SwapOp == EMemoryTraceSwapOperation::PageOut &&
		RootHeap.LiveAllocs->FindSwapRef(MaskedPagedAddress) != nullptr)
	{
		++SwapErrors;
		if (SwapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] EditSwapOp: Trying to swap out page 0x%llX that is currently already in swap (Time=%f, CallstackId=%u). A fake swap page in will be created.", MaskedPagedAddress, Time, CallstackId);
		}

		EditSwapOp(ThreadId, Time, UnmaskedPageAddress, EMemoryTraceSwapOperation::PageIn, CompressedPageSize, CallstackId);
	}

	RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time);

	AdvanceTimelines(Time);

	switch(SwapOp)
	{
		case EMemoryTraceSwapOperation::PageOut:
		{
			IncSampleU32(ETimelineU32::PageOutEvents);
			break;
		}
		case EMemoryTraceSwapOperation::PageIn:
		{
			IncSampleU32(ETimelineU32::PageInEvents);
			break;
		}
		case EMemoryTraceSwapOperation::FreeInSwap:
		{
			IncSampleU32(ETimelineU32::SwapFreeEvents);
			break;
		}
		default:
			break;
	}

	if (MaskedPagedAddress != 0)
	{
		switch(SwapOp)
		{
		case EMemoryTraceSwapOperation::PageOut:
		{
			TotalSwapMemory += PlatformPageSize;
			TotalCompressedSwapMemory += CompressedPageSize;
			SetMaxSampleU64(ETimelineU64::MaxTotalSwapMemory, TotalSwapMemory);
			SetMaxSampleU64(ETimelineU64::MaxTotalCompressedSwapMemory, TotalCompressedSwapMemory);

			FAllocationItem* Alloc = RootHeap.LiveAllocs->AddNewSwap(MaskedPagedAddress);
			FAllocationItem& Allocation = *Alloc;

			INSIGHTS_SLOW_CHECK(Allocation.Address == Address);
			Allocation.SetSize(CompressedPageSize);
			Allocation.SetAlignment(0);
			Allocation.StartEventIndex = RootHeap.EventIndex;
			Allocation.EndEventIndex = (uint32)-1;
			Allocation.StartTime = Time;
			Allocation.EndTime = std::numeric_limits<double>::infinity();
			Allocation.SetAllocThreadId(ThreadId);
			Allocation.FreeThreadId = 0;
			Allocation.AllocCallstackId = CallstackId;
			Allocation.FreeCallstackId = 0; // no callstack yet
			Allocation.MetadataId = MetadataProvider.InvalidMetadataId;
			Allocation.Tag = 0;
			Allocation.SetHeapId(RootHeapId);
			Allocation.SetFlags(EMemoryTraceHeapAllocationFlags::Swap);

			RootHeap.UpdateHistogramByAllocSize(CompressedPageSize);
			break;
		}
		case EMemoryTraceSwapOperation::PageIn:
		case EMemoryTraceSwapOperation::FreeInSwap:
		{
			if (TotalSwapMemory >= PlatformPageSize) // in case if we have imbalanced events
			{
				TotalSwapMemory -= PlatformPageSize;
			}
			SetMinSampleU64(ETimelineU64::MinTotalSwapMemory, TotalSwapMemory);

			FAllocationItem* Alloc = RootHeap.LiveAllocs->RemoveSwap(MaskedPagedAddress); // we take ownership of Alloc

			if (Alloc)
			{
				Alloc->SetEndEventIndex(RootHeap.EventIndex);
				Alloc->SetEndTime(Time);
				Alloc->SetFreeThreadId(ThreadId);
				Alloc->FreeCallstackId = CallstackId;

				const uint64 AllocSize = Alloc->GetSize();
				const uint32 EventDistance = Alloc->GetEventDistance();

				RootHeap.SbTree->AddAlloc(Alloc); // SbTree takes ownership of Alloc
				Alloc = nullptr;

				RootHeap.UpdateHistogramByEventDistance(EventDistance);

				// Update stats for the current timeline sample.
				CompressedPageSize = AllocSize;
				if (TotalCompressedSwapMemory >= CompressedPageSize)
				{
					TotalCompressedSwapMemory -= CompressedPageSize;
				}
				SetMinSampleU64(ETimelineU64::MinTotalCompressedSwapMemory, TotalCompressedSwapMemory);
			}
			else
			{
				++SwapErrors;
				if (SwapErrors <= MaxLogMessagesPerErrorType)
				{
					UE_LOGF(LogTraceServices, Error, "[MemAlloc] EditSwapOp: Trying to page in swap page 0x%llX that is currently not in swap (Time=%f, CallstackId=%u). Ignoring page in. Swap total compressed memory value will be invalid.", MaskedPagedAddress, Time, CallstackId);
				}
			}

			break;
		}
		default:
			break;
		}
	}

	if (RootHeap.EventIndex != ~0u)
	{
		RootHeap.EventIndex++;
	}
	else
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] Too many events!");
		bInitialized = false; // ignore further events
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditHeapSpec(HeapId Id, HeapId ParentId, const FStringView& Name, EMemoryTraceHeapFlags Flags)
{
	EditAccessCheck();

	INSIGHTS_API_LOGF(0u, 0.0, 0ull, "HeapSpec : Id=%u ParentId=%u Name=\"%.*ls\" Flags=0x%X", Id, ParentId, Name.Len(), Name.GetData(), uint32(Flags));

	AddHeapSpec(Id, ParentId, Name, Flags);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IAllocationsProvider::FHeapSpec& FAllocationsProvider::GetOrCreateHeapSpec(HeapId Id)
{
	if (Id < (uint32)HeapSpecs.Num())
	{
		if (HeapSpecs[Id] != nullptr)
		{
			return *HeapSpecs[Id];
		}
	}
	else
	{
		HeapSpecs.AddDefaulted(Id - HeapSpecs.Num() + 1);
	}

	FHeapSpec* HeapSpec = new FHeapSpec();

	HeapSpec->Id = Id;
	HeapSpec->Parent = nullptr;
	HeapSpec->Name = GDefaultHeapName;
	HeapSpec->Flags = EMemoryTraceHeapFlags::None;

	HeapSpecs[Id] = HeapSpec;
	return *HeapSpec;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsProvider::FRootHeap& FAllocationsProvider::FindParentRootHeapUnchecked(HeapId Id) const
{
	const FHeapSpec* HeapSpec = HeapSpecs[Id];
	while (HeapSpec->Parent != nullptr)
	{
		HeapSpec = HeapSpec->Parent;
	}
	return *RootHeaps[HeapSpec->Id];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::CreateRootHeap(HeapId Id)
{
	FRootHeap* RootHeapPtr = new FRootHeap();

	RootHeapPtr->HeapSpec = &GetOrCreateHeapSpec(Id);

	constexpr uint32 ColumnShift = 17; // 1<<17 = 128K
	RootHeapPtr->SbTree = new FSbTree(Session.GetLinearAllocator(), ColumnShift);

	RootHeapPtr->LiveAllocs = new FLiveAllocCollection();

	RootHeaps[Id] = RootHeapPtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::AddHeapSpec(HeapId Id, HeapId ParentId, const FStringView& Name, EMemoryTraceHeapFlags Flags)
{
	const TCHAR* HeapName = Session.StoreString(Name);

	if (Id < MaxRootHeaps)
	{
		FHeapSpec* HeapSpec = HeapSpecs[Id];
		FRootHeap* RootHeapPtr = RootHeaps[Id];

		if (RootHeapPtr != nullptr)
		{
			check(HeapSpec != nullptr)
			check(HeapSpec->Id == Id);

			check(RootHeapPtr->HeapSpec == HeapSpec);
			check(RootHeapPtr->SbTree != nullptr);
			check(RootHeapPtr->LiveAllocs != nullptr);

			// "System" root heap is already created. See class constructor.
			if (Id != 0)
			{
				++HeapWarnings;
				if (HeapWarnings <= MaxLogMessagesPerWarningType)
				{
					UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Root heap %u has changed from (\"%ls\", flags=%u) to (\"%ls\", flags=%u).",
						Id,
						HeapSpec->Name,
						int(HeapSpec->Flags),
						HeapName,
						int(Flags));
				}
			}
		}
		else
		{
			check(HeapSpec == nullptr);

			CreateRootHeap(Id);

			HeapSpec = HeapSpecs[Id];
			check(HeapSpec != nullptr);
		}

		HeapSpec->Name = HeapName;
		HeapSpec->Flags = Flags;
	}
	else
	{
		FHeapSpec& ParentHeapSpec = GetOrCreateHeapSpec(ParentId);
		if (ParentHeapSpec.Name == GDefaultHeapName)
		{
			if (ParentId < MaxRootHeaps && RootHeaps[ParentId] == nullptr)
			{
				CreateRootHeap(ParentId);
			}

			++HeapWarnings;
			if (HeapWarnings <= MaxLogMessagesPerWarningType)
			{
				UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Heap %u (\"%ls\") is used before its parent heap %u was announced.", Id, HeapName, ParentId);
			}
		}

		if (IsValidHeap(Id))
		{
			FHeapSpec& PreviousHeap = GetHeapSpecUnchecked(Id);
			if (PreviousHeap.Name != GDefaultHeapName)
			{
				++HeapWarnings;
				if (HeapWarnings <= MaxLogMessagesPerWarningType)
				{
					UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Heap %u has changed from (\"%ls\", parent=\"%ls\", flags=%u) to (\"%ls\", parent=\"%ls\", flags=%u).",
						Id,
						PreviousHeap.Name,
						PreviousHeap.Parent && PreviousHeap.Parent->Name ? PreviousHeap.Parent->Name : TEXT(""),
						int(PreviousHeap.Flags),
						HeapName,
						ParentHeapSpec.Name ? ParentHeapSpec.Name : TEXT(""),
						int(Flags));
				}
			}
		}

		FHeapSpec& HeapSpec = GetOrCreateHeapSpec(Id);
		HeapSpec.Parent = &ParentHeapSpec;
		HeapSpec.Name = HeapName;
		HeapSpec.Flags = Flags;

		ParentHeapSpec.Children.Add(&HeapSpec);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditMarkAllocationAsHeap(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId Heap, EMemoryTraceHeapAllocationFlags Flags)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		return;
	}

	++HeapEvents;

	INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, "MarkAllocAsHeap 0x%llX : Heap=%u Flags=0x%X CallstackId=%u", Address, Heap, uint32(Flags), CallstackId);

	if (!IsValidHeap(Heap))
	{
		++HeapErrors;
		if (HeapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapMarkAlloc: Heap %u is not valid (Address=0x%llX, Time=%f, CallstackId=%u)!", Heap, Address, Time, CallstackId);
		}
		return;
	}

	const FHeapSpec& HeapSpec = GetHeapSpecUnchecked(Heap);
	const FRootHeap& RootHeap = FindParentRootHeapUnchecked(Heap);

	INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeap.HeapSpec->Id, CallstackId);

#if 0 // TODO
	if (Heap == RootHeap.HeapSpec->Id)
	{
		++HeapWarnings;
		if (HeapWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] HeapMarkAlloc: Heap %u is a root heap (Address=0x%llX, Time=%f, CallstackId=%u)!", Heap, Address, Time, CallstackId);
		}
	}
#endif

	// Remove the allocation from the Live allocs.
	FAllocationItem* Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc

#if INSIGHTS_VALIDATE_HEAP_MARK_ALLOC_EVENTS
	if (!Alloc)
	{
		++HeapErrors;
		if (HeapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapMarkAlloc: Could not find alloc with address 0x%llX (Heap=%u \"%ls\", Flags=%u, Time=%f, CallstackId=%u)!", Address, Heap, HeapSpec.Name, uint32(Flags), Time, CallstackId);
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapMarkAlloc: A fake alloc will be created with size 0.");
		}

		// Fake the missing alloc.
		constexpr uint64 FakeAllocSize = 0;
		constexpr uint32 FakeAllocAlignment = 0;
		INSIGHTS_WATCH_INDIRECT_API_LOGF("Alloc", ThreadId, Time, Address);
		AllocInternal(ThreadId, Time, CallstackId, Address, FakeAllocSize, FakeAllocAlignment, RootHeap.HeapSpec->Id);
		RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time); // for the case where the next event is first event in a new SbTree column after adding the fake alloc
		Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc
		check(Alloc != nullptr);
	}
#endif // INSIGHTS_VALIDATE_HEAP_MARK_ALLOC_EVENTS

	if (Alloc)
	{
		check(Address == Alloc->Address);

		// Mark allocation as a "heap" allocation.
		ensure(EnumHasAnyFlags(Flags, EMemoryTraceHeapAllocationFlags::Heap));
		ensure(!Alloc->IsHeap());
		Alloc->AddHeapFlag();
		Alloc->SetHeapId(Heap);
		if (CallstackId != 0)
		{
			Alloc->AllocCallstackId = CallstackId;
		}

		// Re-add it to the Live allocs as a heap allocation.
		RootHeap.LiveAllocs->AddHeap(Alloc); // the Live allocs takes ownership of Alloc

		// Update the timeline stats (Alloc removed from LiveAllocs).
		TotalAllocatedMemory -= Alloc->GetSize();
		--TotalLiveAllocations;
		SetMinSampleU64(ETimelineU64::MinTotalAllocatedMemory, TotalAllocatedMemory);
		SetMinSampleU32(ETimelineU32::MinLiveAllocations, TotalLiveAllocations);
		IncSampleU32(ETimelineU32::FreeEvents);
	}
	else
	{
		++HeapErrors;
		if (HeapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapMarkAlloc: Could not find alloc with address 0x%llX (Heap=%u \"%ls\", Flags=%u, Time=%f, CallstackId=%u)!", Address, Heap, HeapSpec.Name, uint32(Flags), Time, CallstackId);
		}
	}

	++HeapCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditUnmarkAllocationAsHeap(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId Heap)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		return;
	}

	++HeapEvents;

	INSIGHTS_WATCH_API_LOGF(ThreadId, Time, Address, "UnmarkAllocAsHeap 0x%llX : Heap=%u CallstackId=%u", Address, Heap, CallstackId);

	if (!IsValidHeap(Heap))
	{
		++HeapErrors;
		if (HeapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapUnmarkAlloc: Heap %u is not valid (Address=0x%llX, Time=%f, CallstackId=%u)!", Heap, Address, Time, CallstackId);
		}
		return;
	}

	const FHeapSpec& HeapSpec = GetHeapSpecUnchecked(Heap);
	const FRootHeap& RootHeap = FindParentRootHeapUnchecked(Heap);

	INSIGHTS_FILTER_EVENT(ThreadId, Time, Address, RootHeap.HeapSpec->Id, CallstackId);

#if 0 // TODO
	if (Heap == RootHeap.HeapSpec->Id)
	{
		++HeapWarnings;
		if (HeapWarnings <= MaxLogMessagesPerWarningType)
		{
			UE_LOGF(LogTraceServices, Warning, "[MemAlloc] HeapUnmarkAlloc: Heap %u is a root heap (Address=0x%llX, Time=%f, CallstackId=%u)!", Heap, Address, Time, CallstackId);
		}
	}
#endif

	// Remove the heap allocation from the Live allocs.
	FAllocationItem* Alloc = RootHeap.LiveAllocs->RemoveHeap(Address); // we take ownership of Alloc

#if INSIGHTS_VALIDATE_HEAP_UNMARK_ALLOC_EVENTS
	if (!Alloc)
	{
		++HeapErrors;
		if (HeapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapUnmarkAlloc: Could not find heap with address 0x%llX (Heap=%u \"%ls\", Time=%f, CallstackId=%u)!", Address, Heap, HeapSpec.Name, Time, CallstackId);
		}

		// Remove the allocation from the Live allocs.
		Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc

		if (!Alloc)
		{
			if (HeapErrors <= MaxLogMessagesPerErrorType)
			{
				UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapUnmarkAlloc: A fake heap alloc will be created with size 0.");
			}

			// Fake the missing alloc.
			constexpr uint64 FakeAllocSize = 0;
			constexpr uint32 FakeAllocAlignment = 0;
			INSIGHTS_WATCH_INDIRECT_API_LOGF("Alloc", ThreadId, Time, Address);
			AllocInternal(ThreadId, Time, CallstackId, Address, FakeAllocSize, FakeAllocAlignment, RootHeap.HeapSpec->Id);
			RootHeap.SbTree->SetTimeForEvent(RootHeap.EventIndex, Time); // for the case where the next event is first event in a new SbTree column after adding the fake alloc
			Alloc = RootHeap.LiveAllocs->Remove(Address); // we take ownership of Alloc
			check(Alloc != nullptr);
		}
		else
		{
			if (HeapErrors <= MaxLogMessagesPerErrorType)
			{
				UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapUnmarkAlloc: An alloc (Size=%llu, CallstackId=%u) with this address exists. It will be temporary marked as heap.", Alloc->GetSize(), Alloc->AllocCallstackId);
			}
		}

		// Update the timeline stats (Alloc removed from LiveAllocs).
		TotalAllocatedMemory -= Alloc->GetSize();
		--TotalLiveAllocations;
		SetMinSampleU64(ETimelineU64::MinTotalAllocatedMemory, TotalAllocatedMemory);
		SetMinSampleU32(ETimelineU32::MinLiveAllocations, TotalLiveAllocations);

#if INSIGHTS_VALIDATE_ALLOC_EVENTS && INSIGHTS_DOUBLE_ALLOC_FREE_PREVIOUS
		// This can fail if Address is allocated multiple times. See INSIGHTS_VALIDATE_ALLOC_EVENTS.
		ensure(RootHeap.LiveAllocs->FindRef(Address) == nullptr);
#endif

		// Mark allocation as a "heap" allocation.
		ensure(!Alloc->IsHeap());
		Alloc->AddHeapFlag();
		Alloc->SetHeapId(Heap);
		Alloc->AllocCallstackId = CallstackId;
	}
#endif // INSIGHTS_VALIDATE_HEAP_UNMARK_ALLOC_EVENTS

	if (Alloc)
	{
		check(Address == Alloc->Address);

		//constexpr bool bSearchLiveAllocs = false; // never
		const bool bSearchLiveAllocs = EnumHasAnyFlags(HeapSpec.Flags, EMemoryTraceHeapFlags::NeverFrees); // only for heaps with NeverFrees flag
		//constexpr bool bSearchLiveAllocs = true; // for all heaps

		if (bSearchLiveAllocs)
		{
			// Find all live allocs in this heap.
			uint64 AllocatedSizeInHeap = 0;
			struct FAllocInHeap
			{
				uint64 Address;
				uint64 Size;
			};
			TArray<FAllocInHeap> AllocsInHeap;
			const uint64 HeapSize = Alloc->GetSize();
			const uint64 EndAddress = Alloc->Address + HeapSize;
			RootHeap.LiveAllocs->Enumerate(Address, EndAddress,
				[&AllocatedSizeInHeap, &AllocsInHeap]
				(const FAllocationItem& ChildAlloc)
				{
					const uint64 ChildAllocSize = ChildAlloc.GetSize();
					AllocatedSizeInHeap += ChildAllocSize;
					AllocsInHeap.Add({ ChildAlloc.Address, ChildAllocSize });
				});

			if (AllocsInHeap.Num() > 0)
			{
				if (!EnumHasAnyFlags(HeapSpec.Flags, EMemoryTraceHeapFlags::NeverFrees))
				{
					++HeapWarnings;
					if (HeapWarnings <= MaxLogMessagesPerWarningType)
					{
						// For heaps that do not have NeverFrees flag, we report live allocs as leaks.
						UE_LOGF(LogTraceServices, Warning, "[MemAlloc] HeapUnmarkAlloc: %d memory leaks (%llu bytes) detected for heap %u (\"%ls\", Address=0x%llX, Size=%llu, Time=%f, CallstackId=%u)",
							AllocsInHeap.Num(), AllocatedSizeInHeap, Heap, HeapSpec.Name, Address, HeapSize, Time, CallstackId);
						UE_LOGF(LogTraceServices, Warning, "[MemAlloc] Top memory leaks:");
						AllocsInHeap.Sort([this](const FAllocInHeap& A, const FAllocInHeap& B) -> bool { return A.Size > B.Size; });
						int NumAllocsInTop = 10; // only show first 10 allocs
						for (const FAllocInHeap& AllocInHeap : AllocsInHeap)
						{
							UE_LOGF(LogTraceServices, Warning, "[MemAlloc]     alloc 0x%llX (%llu bytes)", AllocInHeap.Address, AllocInHeap.Size);
							if (--NumAllocsInTop <= 0)
							{
								break;
							}
						}
					}
				}
#if 0 // debug
				else
				{
					UE_LOGF(LogTraceServices, Log, "[MemAlloc] HeapUnmarkAlloc: %d memory allocs (%llu bytes) freed for heap %u (\"%ls\", Address=0x%llX, Size=%llu, Time=%f, CallstackId=%u)",
						AllocsInHeap.Num(), AllocatedSizeInHeap, Heap, HeapSpec.Name, Address, HeapSize, Time, CallstackId);
				}
#endif

				// Free automatically all allocs in this heap.
				for (const FAllocInHeap& AllocInHeap : AllocsInHeap)
				{
					INSIGHTS_WATCH_INDIRECT_API_LOGF("Free", ThreadId, Time, AllocInHeap.Address);
					FreeInternal(ThreadId, Time, CallstackId, AllocInHeap.Address, RootHeap.HeapSpec->Id);
				}
			}
		}

		const uint64 AllocSize = Alloc->GetSize();
		const uint32 AllocAlignment = Alloc->GetAlignment();
		const uint32 AllocCallstackId = Alloc->AllocCallstackId;
		const TagIdType AllocTag = Alloc->Tag;

		// Re-add this allocation to the Live allocs.
		ensure(Alloc->IsHeap());
		Alloc->RemoveHeapFlag();
		RootHeap.LiveAllocs->Add(Alloc); // the Live allocs takes ownership of Alloc
		Alloc = nullptr;

		// Update the timeline stats (Alloc re-added to LiveAllocs as a normal "alloc").
		TotalAllocatedMemory += AllocSize;
		++TotalLiveAllocations;
		SetMaxSampleU64(ETimelineU64::MaxTotalAllocatedMemory, TotalAllocatedMemory);
		SetMaxSampleU32(ETimelineU32::MaxLiveAllocations, TotalLiveAllocations);
		IncSampleU32(ETimelineU32::AllocEvents);

		// We cannot just unmark the allocation as heap, there is no timestamp support, instead fake a "free"
		// event and an "alloc" event. Make sure the new allocation retains the tag from the original.
		const uint8 Tracker = 1;
		TagTracker.PushTagFromPtr(ThreadId, Tracker, AllocTag);
		INSIGHTS_WATCH_INDIRECT_API_LOGF("Free", ThreadId, Time, Address);
		FreeInternal(ThreadId, Time, CallstackId, Address, RootHeap.HeapSpec->Id);
		INSIGHTS_WATCH_INDIRECT_API_LOGF("Alloc", ThreadId, Time, Address);
		AllocInternal(ThreadId, Time, AllocCallstackId, Address, AllocSize, AllocAlignment, RootHeap.HeapSpec->Id);
		TagTracker.PopTagFromPtr(ThreadId, Tracker);
	}
	else
	{
		++HeapErrors;
		if (HeapErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] HeapUnmarkAlloc: Could not find heap with address 0x%llX (Heap=%u \"%ls\", Time=%f, CallstackId=%u)!", Address, Heap, HeapSpec.Name, Time, CallstackId);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::FRootHeap::UpdateHistogramByAllocSize(uint64 Size)
{
	if (Size > MaxAllocSize)
	{
		MaxAllocSize = Size;
	}

	// HistogramIndex : Value Range
	// 0 : [0]
	// 1 : [1]
	// 2 : [2]
	// 3 : [3 .. 4]
	// 4 : [5 .. 8]
	// 5 : [9 .. 16]
	// ...
	// i : [2^(i-2)+1 .. 2^(i-1)], i = 3 .. 64
	// ...
	// 64 : [2^62+1 .. 2^63]
	// 65 : [2^63+1 .. 2^64-1]

	uint32 Index = 0;
	if (Size != 0)
	{
		Index = 65 - static_cast<uint32>(FMath::CountLeadingZeros64(Size - 1));
		check(Index < NumAllocSizeHistogramPow2);
	}
	++AllocSizeHistogramPow2[Index];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::FRootHeap::UpdateHistogramByEventDistance(uint32 EventDistance)
{
	if (EventDistance > MaxEventDistance)
	{
		MaxEventDistance = EventDistance;
	}

	// HistogramIndex : Value Range
	//  0 : [0]
	//  1 : [1]
	//  2 : [2 .. 10]
	//  3 : [11 .. 100]
	// ...
	//  i : [10^(i-2)+1 .. 10^(i-1)], i = 2 .. 11
	// ...
	// 11 : [10^9+1 .. 10^10]
	// 12 : [MAX_uint32]

	uint32 Index;
	if (EventDistance == 0)
	{
		Index = 0;
	}
	else if (EventDistance != uint32(-1))
	{
		--EventDistance;
		Index = 1;
		while (EventDistance > 0)
		{
			++Index;
			EventDistance /= 10;
		}
		check(Index < NumEventDistanceHistogramPow10 - 1);
	}
	else // EventDistance == uint32(-1)
	{
		Index = NumEventDistanceHistogramPow10 - 1;
	}
	++EventDistanceHistogramPow10[Index];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::FRootHeap::LogStats()
{
	if (EventIndex == 0)
	{
		return;
	}

	check(HeapSpec != nullptr);

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] RootHeap %u (%ls) : %u events, %u live allocs, max alloc size = %llu bytes, max event distance = %u",
		uint32(HeapSpec->Id), HeapSpec->Name, EventIndex, LiveAllocs->Num(), MaxAllocSize, MaxEventDistance);

	LogAllocSizeHistogram();
	LogEventDistanceHistogram();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FUtf8String Pow2ToString(uint32 Pow2, const UTF8CHAR* UnitName = UTF8TEXT(""), bool bAddSpaceBeforeUnit = false)
{
	const ANSICHAR* UnitPrefixes[] = { "", "Ki", "Mi", "Gi", "Ti", "Pi", "Ei" };
	constexpr int32 NumPrefixes = UE_ARRAY_COUNT(UnitPrefixes);

	uint32 Pow = 10;
	for (int32 Index = 0; Index < NumPrefixes; ++Index)
	{
		if (Pow2 < Pow)
		{
			if (bAddSpaceBeforeUnit)
			{
				if (UnitPrefixes[Index][0] == 0 && UnitName[0] == 0)
				{
					return FUtf8String::Printf("%u", 1u << (Pow2 + 10 - Pow));
				}
				else
				{
					return FUtf8String::Printf("%u %s%s", 1u << (Pow2 + 10 - Pow), UnitPrefixes[Index], UnitName);
				}
			}
			else
			{
				return FUtf8String::Printf("%u%s%s", 1u << (Pow2 + 10 - Pow), UnitPrefixes[Index], UnitName);
			}
		}
		Pow += 10;
	}
	return "inf";
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FUtf8String Pow2BytesToString(uint32 Pow2)
{
	return Pow2ToString(Pow2, UTF8TEXT("B"), true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FUtf8String Pow10ToString(uint32 Pow10, const UTF8CHAR* UnitName = UTF8TEXT(""), bool bAddSpaceBeforeUnit = false)
{
	const ANSICHAR* UnitPrefixes[] = { "", "K", "M", "G", "T", "P", "E" };
	constexpr int32 NumPrefixes = UE_ARRAY_COUNT(UnitPrefixes);

	uint32 Pow = 3;
	for (int32 Index = 0; Index < NumPrefixes; ++Index)
	{
		if (Pow10 < Pow)
		{
			uint64 Value = 1ull;
			const uint32 ShiftCount = Pow10 + 3 - Pow;
			for (uint32 Shift = 0; Shift < ShiftCount; ++Shift)
			{
				Value *= 10;
			}
			if (bAddSpaceBeforeUnit)
			{
				if (UnitPrefixes[Index][0] == 0 && UnitName[0] == 0)
				{
					return FUtf8String::Printf("%llu", Value);
				}
				else
				{
					return FUtf8String::Printf("%llu %s%s", Value, UnitPrefixes[Index], UnitName);
				}
			}
			else
			{
				return FUtf8String::Printf("%llu%s%s", Value, UnitPrefixes[Index], UnitName);
			}
		}
		Pow += 3;
	}
	return "inf";
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FUtf8String Pow10EventsToString(uint32 Pow10)
{
	return Pow10ToString(Pow10, UTF8TEXT(""), false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static FUtf8String CountToString(uint64 Count)
{
	FUtf8String Out = FUtf8String::Printf("%llu", Count);
	for (int32 Index = Out.Len() - 3; Index > 0; Index -= 3)
	{
		Out.InsertAt(Index, ',');
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::FRootHeap::LogAllocSizeHistogram()
{
	uint32 StartIndex = 0;
	while (StartIndex < NumAllocSizeHistogramPow2 && AllocSizeHistogramPow2[StartIndex] == 0)
	{
		++StartIndex;
	}

	uint32 EndIndex = NumAllocSizeHistogramPow2;
	while (EndIndex > 0 && AllocSizeHistogramPow2[EndIndex - 1] == 0)
	{
		--EndIndex;
	}

	if (StartIndex >= EndIndex)
	{
		return;
	}

	check(HeapSpec != nullptr);
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Alloc Size Histogram (%ls):", HeapSpec->Name);

	// HistogramIndex : Value Range
	//  0 : [0 bytes]
	//  1 : [1 byte]
	//  2 : [2 bytes]
	//  3 : (2 .. 4]
	//  4 : (4 .. 8]
	//  5 : (8 .. 16]
	// ...
	//  i : (2^(i-2) .. 2^(i-1)], i = 3 .. 64
	// ...
	// 64 : (2^62 .. 2^63]
	// 65 : (2^63 .. 2^64)

	uint64 Sum = 0;
	for (uint32 Index = StartIndex; Index < EndIndex; ++Index)
	{
		Sum += AllocSizeHistogramPow2[Index];
	}
	uint64 PartialSum = 0;
	double PercentFactor = 100.0 / (double)Sum;

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] -----------------------------------------------------------");
	for (uint32 Index = StartIndex; Index < EndIndex; ++Index)
	{
		const uint64 Count = AllocSizeHistogramPow2[Index];
		if (Count == 0)
		{
			continue;
		}
		PartialSum += Count;
		const double Percent = (double)Count * PercentFactor;
		const double CumulativePercent = (double)PartialSum * PercentFactor;
		if (Index == 65)
		{
			// 65 : (2^63 .. 2^64)
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] (%8s .. %-8s) %11s %4.01f%% %11s %5.01f%%",
				*Pow2BytesToString(Index - 2), *Pow2BytesToString(Index - 1), *CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
		else if (Index > 2)
		{
			// i : (2^(i-2) .. 2^(i-1)], i = 3 .. 64
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] (%8s .. %-8s] %11s %4.01f%% %11s %5.01f%%",
				*Pow2BytesToString(Index - 2), *Pow2BytesToString(Index - 1), *CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
		else if (Index == 1)
		{
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] [     1 byte         ] %11s %4.01f%% %11s %5.01f%%",
				*CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
		else // if (Index == 0 || Index == 2)
		{
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] [     %u bytes        ] %11s %4.01f%% %11s %5.01f%%",
				Index, *CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
	}
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] -----------------------------------------------------------");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::FRootHeap::LogEventDistanceHistogram()
{
	uint32 StartIndex = 0;
	while (StartIndex < NumEventDistanceHistogramPow10 && EventDistanceHistogramPow10[StartIndex] == 0)
	{
		++StartIndex;
	}

	uint32 EndIndex = NumEventDistanceHistogramPow10;
	while (EndIndex > 0 && EventDistanceHistogramPow10[EndIndex - 1] == 0)
	{
		--EndIndex;
	}

	if (StartIndex >= EndIndex)
	{
		return;
	}

	check(HeapSpec != nullptr);
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Event Distance Histogram (%ls):", HeapSpec->Name);

	// HistogramIndex : Value Range
	//  0 : [0]
	//  1 : [1]
	//  2 : (1 .. 10]
	//  3 : (10 .. 100]
	// ...
	//  i : (10^(i-2) .. 10^(i-1)], i = 2 .. 11
	// ...
	// 11 : (10^9 .. 10^10]
	// 12 : [MAX_uint32]

	uint64 Sum = LiveAllocs->Num();
	for (uint32 Index = StartIndex; Index < EndIndex; ++Index)
	{
		Sum += EventDistanceHistogramPow10[Index];
	}
	uint64 PartialSum = 0;
	double PercentFactor = 100.0 / (double)Sum;

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] -----------------------------------------------------------");
	for (uint32 Index = StartIndex; Index < EndIndex; ++Index)
	{
		const uint64 Count = EventDistanceHistogramPow10[Index];
		if (Count == 0)
		{
			continue;
		}
		PartialSum += Count;
		const double Percent = (double)Count * PercentFactor;
		const double CumulativePercent = (double)PartialSum * PercentFactor;
		if (Index == NumEventDistanceHistogramPow10 - 1)
		{
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] [     MAX_uint32     ] %11s %4.01f%% %11s %5.01f%%",
				*CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
		else if (Index > 1)
		{
			// i : (10^(i-2) .. 10^(i-1)], i = 2 .. 11
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] (%8s .. %-8s] %11s %4.01f%% %11s %5.01f%%",
				*Pow10EventsToString(Index - 2), *Pow10EventsToString(Index - 1), *CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
		else if (Index == 1)
		{
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] [       1 event      ] %11s %4.01f%% %11s %5.01f%%",
				*CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
		else if (Index == 0)
		{
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] [       0 events     ] %11s %4.01f%% %11s %5.01f%%",
				*CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
		}
	}
	if (LiveAllocs->Num() > 0)
	{
		const uint64 Count = LiveAllocs->Num();
		PartialSum += Count;
		const double Percent = (double)Count * PercentFactor;
		const double CumulativePercent = (double)PartialSum * PercentFactor;
		UE_LOGF(LogTraceServices, Log, "[MemAlloc] (   \u221E (live allocs)  ) %11s %4.01f%% %11s %5.01f%%",
			*CountToString(Count), Percent, *CountToString(PartialSum), CumulativePercent);
	}
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] -----------------------------------------------------------");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::AdvanceTimelines(double Time)
{
	// If enough time has passed (since the current sample is started)...
	if (Time - SampleStartTimestamp > DefaultTimelineSampleGranularity)
	{
		// Add the current sample to the timelines.
		Timeline.EmplaceBack(SampleStartTimestamp);
		for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU64; ++TimelineIndex)
		{
			ValueTimelinesU64[TimelineIndex]->EmplaceBack(SampleValuesU64[TimelineIndex]);
		}
		for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU32; ++TimelineIndex)
		{
			ValueTimelinesU32[TimelineIndex]->EmplaceBack(SampleValuesU32[TimelineIndex]);
		}

		// Start a new sample.
		SampleStartTimestamp = Time;
		SetSampleU64(ETimelineU64::MinTotalAllocatedMemory, TotalAllocatedMemory);
		SetSampleU64(ETimelineU64::MaxTotalAllocatedMemory, TotalAllocatedMemory);
		SetSampleU32(ETimelineU32::MinLiveAllocations, TotalLiveAllocations);
		SetSampleU32(ETimelineU32::MaxLiveAllocations, TotalLiveAllocations);
		SetSampleU64(ETimelineU64::MinTotalSwapMemory, TotalSwapMemory);
		SetSampleU64(ETimelineU64::MaxTotalSwapMemory, TotalSwapMemory);
		SetSampleU64(ETimelineU64::MinTotalCompressedSwapMemory, TotalCompressedSwapMemory);
		SetSampleU64(ETimelineU64::MaxTotalCompressedSwapMemory, TotalCompressedSwapMemory);
		SetSampleU32(ETimelineU32::AllocEvents, 0);
		SetSampleU32(ETimelineU32::FreeEvents, 0);
		SetSampleU32(ETimelineU32::PageInEvents, 0);
		SetSampleU32(ETimelineU32::PageOutEvents, 0);
		SetSampleU32(ETimelineU32::SwapFreeEvents, 0);

		// If the previous sample is well distanced in time...
		if (Time - SampleEndTimestamp > DefaultTimelineSampleGranularity)
		{
			// Add an intermediate "flat region" sample.
			Timeline.EmplaceBack(SampleEndTimestamp);
			for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU64; ++TimelineIndex)
			{
				ValueTimelinesU64[TimelineIndex]->EmplaceBack(SampleValuesU64[TimelineIndex]);
			}
			for (int32 TimelineIndex = 0; TimelineIndex < NumTimelinesU32; ++TimelineIndex)
			{
				ValueTimelinesU32[TimelineIndex]->EmplaceBack(SampleValuesU32[TimelineIndex]);
			}
		}
	}

	SampleEndTimestamp = Time;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditPushTag(uint32 ThreadId, uint8 Tracker, TagIdType Tag)
{
	EditAccessCheck();

	TagTracker.PushTag(ThreadId, Tracker, Tag);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditPopTag(uint32 ThreadId, uint8 Tracker)
{
	EditAccessCheck();

	TagTracker.PopTag(ThreadId, Tracker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditPushTagFromPtr(uint32 ThreadId, uint8 Tracker, uint64 Ptr, HeapId RootHeapId)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		// No errors if the "memallocs" channel was not enabled.
		TagTracker.PushTagFromPtr(ThreadId, Tracker, 0);
		return;
	}

	const FAllocationItem* Alloc = nullptr;

	if (IsValidRootHeap(RootHeapId))
	{
		const FRootHeap& RootHeap = GetRootHeapUnchecked(RootHeapId);
		Alloc = RootHeap.LiveAllocs->FindRef(Ptr);
	}
	else
	{
		++MiscErrors;
		if (MiscErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid root heap (%u) for MemoryScopePtr or ReallocFree event!", RootHeapId);
		}
	}

	const TagIdType Tag = Alloc ? Alloc->Tag : 0; // If ptr is not found use "Untagged"
	TagTracker.PushTagFromPtr(ThreadId, Tracker, Tag);

	INSIGHTS_FILTER_EVENT(ThreadId, 0.0, Ptr, RootHeapId, 0);

	if (!Alloc)
	{
		++MiscErrors;
		if (MiscErrors <= MaxLogMessagesPerErrorType)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] Invalid address (0x%llX) for MemoryScopePtr or ReallocFree event!", Ptr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditPopTagFromPtr(uint32 ThreadId, uint8 Tracker)
{
	EditAccessCheck();

	TagTracker.PopTagFromPtr(ThreadId, Tracker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EditOnAnalysisCompleted(double Time)
{
	EditAccessCheck();

	if (!bInitialized)
	{
		if (TagTracker.GetNumErrors() > 0)
		{
			UE_LOGF(LogTraceServices, Error, "[MemAlloc] TagTracker errors: %u", TagTracker.GetNumErrors());
		}
		return;
	}

	// Flush the last cached timeline sample.
	AdvanceTimelines(std::numeric_limits<double>::infinity());

#if 0
	DebugPrint();
#endif

	for (uint32 RootHeapIdx = 0; RootHeapIdx < MaxRootHeaps; ++RootHeapIdx)
	{
		FRootHeap* RootHeap = RootHeaps[RootHeapIdx];
		if (RootHeap)
		{
			RootHeap->SbTree->Validate();
		}
	}

	//TODO: shrink live allocs buffers

	if (AllocWarnings > 0 || FreeWarnings > 0 || HeapWarnings > 0 || SwapWarnings > 0 || MiscWarnings > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[MemAlloc] %llu warnings (%llu ALLOC + %llu FREE + %llu HEAP + %llu SWAP + %llu other)",
			AllocWarnings + FreeWarnings + HeapWarnings + SwapWarnings + MiscWarnings,
			AllocWarnings, FreeWarnings, HeapWarnings, SwapWarnings, MiscWarnings);
	}
	if (TagTracker.GetNumWarnings() > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[MemAlloc] TagTracker warnings: %u", TagTracker.GetNumWarnings());
	}

	if (AllocErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] ALLOC event errors: %llu", AllocErrors);
	}
	if (FreeErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] FREE event errors: %llu", FreeErrors);
	}
	if (HeapErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] HEAP event errors: %llu", HeapErrors);
	}
	if (SwapErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] SWAP event errors: %llu", SwapErrors);
	}
	if (MiscErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] Other errors: %llu", MiscErrors);
	}
	if (TagTracker.GetNumErrors() > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[MemAlloc] TagTracker errors: %u", TagTracker.GetNumErrors());
	}

	uint64 TotalEventCount = 0;
	for (uint32 RootHeapIdx = 0; RootHeapIdx < MaxRootHeaps; ++RootHeapIdx)
	{
		FRootHeap* RootHeap = RootHeaps[RootHeapIdx];
		if (RootHeap)
		{
			RootHeap->LogStats();
			TotalEventCount += RootHeap->EventIndex;
		}
	}

	uint32 NumHeapSpecs = 0;
	for (FHeapSpec* HeapSpec : HeapSpecs)
	{
		if (HeapSpec)
		{
			++NumHeapSpecs;
		}
	}

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Total Indexed: %llu events", TotalEventCount);
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Traced: %llu alloc events, %llu free events, %llu heap events, %llu swap op events",
		AllocEvents, FreeEvents, HeapEvents, SwapOpEvents);
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Analysis completed (%llu allocs, %llu frees, %llu heaps, %u heap specs).",
		AllocCount, FreeCount, HeapCount, NumHeapSpecs);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EnumerateRootHeaps(TFunctionRef<void(HeapId Id, const FHeapSpec&)> Callback) const
{
	ReadAccessCheck();

	for (uint32 RootHeapIdx = 0; RootHeapIdx < MaxRootHeaps; ++RootHeapIdx)
	{
		FRootHeap* RootHeap = RootHeaps[RootHeapIdx];
		if (RootHeap)
		{
			check(RootHeap->HeapSpec != nullptr);
			const FHeapSpec& HeapSpec = *RootHeap->HeapSpec;
			check(HeapSpec.Parent == nullptr);
			if (HeapSpec.Name != nullptr)
			{
				Callback(HeapSpec.Id, HeapSpec);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EnumerateHeaps(TFunctionRef<void(HeapId Id, const FHeapSpec&)> Callback) const
{
	ReadAccessCheck();

	const uint32 NumHeapSpecs = HeapSpecs.Num();
	for (uint32 HeapSpecIdx = 0; HeapSpecIdx < NumHeapSpecs; ++HeapSpecIdx)
	{
		const FHeapSpec* HeapSpecPtr = HeapSpecs[HeapSpecIdx];
		if (HeapSpecPtr)
		{
			const FHeapSpec& HeapSpec = *HeapSpecPtr;
			if (HeapSpec.Name != nullptr)
			{
				Callback(HeapSpec.Id, HeapSpec);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::GetTimelineIndexRange(double StartTime, double EndTime, int32& StartIndex, int32& EndIndex) const
{
	using PageType = const TPagedArrayPage<double>;

	PageType* PageData = Timeline.GetPages();
	if (PageData)
	{
		const int32 NumPoints = static_cast<int32>(Timeline.Num());

		const int32 NumPages = static_cast<int32>(Timeline.NumPages());
		TArrayView<PageType, int32> Pages(PageData, NumPages);

		const int32 StartPageIndex = Algo::UpperBoundBy(Pages, StartTime, [](PageType& Page) { return Page.Items[0]; }) - 1;
		if (StartPageIndex < 0)
		{
			StartIndex = -1;
		}
		else
		{
			PageType& Page = PageData[StartPageIndex];
			TArrayView<double> PageValues(Page.Items, static_cast<int32>(Page.Count));
			const int32 Index = Algo::UpperBound(PageValues, StartTime) - 1;
			check(Index >= 0);
			StartIndex = StartPageIndex * static_cast<int32>(Timeline.GetPageSize()) + Index;
			check(Index < NumPoints);
		}

		const int32 EndPageIndex = Algo::UpperBoundBy(Pages, EndTime, [](PageType& Page) { return Page.Items[0]; }) - 1;
		if (EndPageIndex < 0)
		{
			EndIndex = -1;
		}
		else
		{
			PageType& Page = PageData[EndPageIndex];
			TArrayView<double> PageValues(Page.Items, static_cast<int32>(Page.Count));
			const int32 Index = Algo::UpperBound(PageValues, EndTime) - 1;
			check(Index >= 0);
			EndIndex = EndPageIndex * static_cast<int32>(Timeline.GetPageSize()) + Index;
			check(Index < NumPoints);
		}
	}
	else
	{
		StartIndex = -1;
		EndIndex = -1;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EnumerateTimeline(ETimelineU64 TimelineIndex, int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
{
	ReadAccessCheck();

	if (int32(TimelineIndex) < 0 ||
		int32(TimelineIndex) >= NumTimelinesU64 ||
		ValueTimelinesU64[int32(TimelineIndex)] == nullptr)
	{
		return;
	}
	const TPagedArray<uint64>& ValueTimeline = *ValueTimelinesU64[int32(TimelineIndex)];

	const int32 NumPoints = static_cast<int32>(Timeline.Num());
	StartIndex = FMath::Max(StartIndex, 0);
	EndIndex = FMath::Min(EndIndex + 1, NumPoints); // make it exclusive
	if (StartIndex >= EndIndex)
	{
		return;
	}

	auto TimeIt = Timeline.GetIteratorFromItem(StartIndex);
	auto ValueIt = ValueTimeline.GetIteratorFromItem(StartIndex);

	double PrevTime = *TimeIt;
	uint64 PrevValue = *ValueIt;
	++TimeIt;
	++ValueIt;

	for (int32 Index = StartIndex + 1; Index < EndIndex; ++Index, ++TimeIt, ++ValueIt)
	{
		const double Time = *TimeIt;
		Callback(PrevTime, Time - PrevTime, PrevValue);
		PrevTime = *TimeIt;
		PrevValue = *ValueIt;
	}

	if (EndIndex < NumPoints)
	{
		const double Time = *TimeIt;
		Callback(PrevTime, Time - PrevTime, PrevValue);
	}
	else
	{
		Callback(PrevTime, std::numeric_limits<double>::infinity(), PrevValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EnumerateTimeline(ETimelineU32 TimelineIndex, int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
{
	ReadAccessCheck();

	if (int32(TimelineIndex) < 0 ||
		int32(TimelineIndex) >= NumTimelinesU32 ||
		ValueTimelinesU32[int32(TimelineIndex)] == nullptr)
	{
		return;
	}
	const TPagedArray<uint32>& ValueTimeline = *ValueTimelinesU32[int32(TimelineIndex)];

	const int32 NumPoints = static_cast<int32>(Timeline.Num());
	StartIndex = FMath::Max(StartIndex, 0);
	EndIndex = FMath::Min(EndIndex + 1, NumPoints); // make it exclusive
	if (StartIndex >= EndIndex)
	{
		return;
	}

	auto TimeIt = Timeline.GetIteratorFromItem(StartIndex);
	auto ValueIt = ValueTimeline.GetIteratorFromItem(StartIndex);

	double PrevTime = *TimeIt;
	uint32 PrevValue = *ValueIt;
	++TimeIt;
	++ValueIt;

	for (int32 Index = StartIndex + 1; Index < EndIndex; ++Index, ++TimeIt, ++ValueIt)
	{
		const double Time = *TimeIt;
		Callback(PrevTime, Time - PrevTime, PrevValue);
		PrevTime = *TimeIt;
		PrevValue = *ValueIt;
	}

	if (EndIndex < NumPoints)
	{
		const double Time = *TimeIt;
		Callback(PrevTime, Time - PrevTime, PrevValue);
	}
	else
	{
		Callback(PrevTime, std::numeric_limits<double>::infinity(), PrevValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EnumerateTags(TFunctionRef<void(const TCHAR*, const TCHAR*, TagIdType, TagIdType)> Callback) const
{
	ReadAccessCheck();

	TagTracker.EnumerateTags(Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::DebugPrint() const
{
	ReadAccessCheck();

	for (uint32 RootHeapIdx = 0; RootHeapIdx < MaxRootHeaps; ++RootHeapIdx)
	{
		FRootHeap* RootHeap = RootHeaps[RootHeapIdx];
		if (RootHeap)
		{
			RootHeap->SbTree->DebugPrint();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IAllocationsProvider::FQueryHandle FAllocationsProvider::StartQuery(const IAllocationsProvider::FQueryParams& Params) const
{
	auto* Inner = new FAllocationsQuery(*this, Params);
	return IAllocationsProvider::FQueryHandle(Inner);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::CancelQuery(FQueryHandle Query) const
{
	auto* Inner = (FAllocationsQuery*)Query;
	return Inner->Cancel();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAllocationsProvider::FQueryStatus FAllocationsProvider::PollQuery(FQueryHandle Query) const
{
	auto* Inner = (FAllocationsQuery*)Query;
	return Inner->Poll();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FAllocationsProvider::GetPlatformPageSize() const
{
	return bInitialized ? PlatformPageSize : 4096;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsProvider::EnumerateLiveAllocs(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const
{
	ReadAccessCheck();

	for (uint32 RootHeapIdx = 0; RootHeapIdx < MaxRootHeaps; ++RootHeapIdx)
	{
		FRootHeap* RootHeap = RootHeaps[RootHeapIdx];
		if (RootHeap)
		{
			RootHeap->LiveAllocs->Enumerate(Callback);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FAllocationsProvider::GetNumLiveAllocs() const
{
	ReadAccessCheck();

	return TotalLiveAllocations;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName GetAllocationsProviderName()
{
	static const FName Name("AllocationsProvider");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAllocationsProvider* ReadAllocationsProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IAllocationsProvider>(GetAllocationsProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef INSIGHTS_SLOW_CHECK
#undef INSIGHTS_LLA_RESERVE
#undef INSIGHTS_USE_SHORT_LIVING_ALLOCS
#undef INSIGHTS_SLA_USE_ADDRESS_MAP
#undef INSIGHTS_USE_LAST_ALLOC
#undef INSIGHTS_VALIDATE_ALLOC_EVENTS
#undef INSIGHTS_DOUBLE_ALLOC_FREE_PREVIOUS
#undef INSIGHTS_VALIDATE_FREE_EVENTS
#undef INSIGHTS_VALIDATE_HEAP_MARK_ALLOC_EVENTS
#undef INSIGHTS_VALIDATE_HEAP_UNMARK_ALLOC_EVENTS
#undef INSIGHTS_DEBUG_METADATA
#undef INSIGHTS_WARNINGS_FOR_NULLPTR_ALLOCS
#undef INSIGHTS_FILTER_EVENTS_ENABLED
#undef INSIGHTS_FILTER_EVENT
#undef INSIGHTS_DEBUG_WATCH
#undef INSIGHTS_DEBUG_WATCH_FOUND
#undef INSIGHTS_LOGF
#undef INSIGHTS_API_LOGF
#undef INSIGHTS_INDIRECT_API_LOGF
#undef INSIGHTS_WATCH_API_LOGF
#undef INSIGHTS_WATCH_INDIRECT_API_LOGF
