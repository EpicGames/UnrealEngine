// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "ProfilingDebugging/MemoryTrace.h" // for EMemoryTraceHeapFlags and EMemoryTraceHeapAllocationFlags

#include "TraceServices/Model/AllocationsProvider.h"
#include "AllocationItem.h"
#include "AllocMap.h"

namespace TraceServices
{

class IAnalysisSession;
class ILinearAllocator;
class FMetadataProvider;
class FSbTree;

extern thread_local FProviderLock::FThreadLocalState GAllocationsProviderLockState;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTagTracker
{
private:
	static constexpr uint32 TrackerIdShift = 24;
	static constexpr uint32 TrackerIdMask = 0xFF000000;
	static constexpr TagIdType UntaggedTagId = 0;
	static constexpr TagIdType InvalidTagId = ~0;

	enum class ETagStackFlags : uint32
	{
		None = 0,
		PtrScope = 1 << 0,
	};

	struct FTagStackEntry
	{
		TagIdType Tag;
		ETagStackFlags Flags;

		inline bool IsPtrScope() const { return ((uint32(Flags) & uint32(ETagStackFlags::PtrScope)) != 0); }
	};

	struct FThreadState
	{
		TArray<FTagStackEntry> TagStack;
	};

	struct FTagEntry
	{
		const TCHAR* Display;
		const TCHAR* FullPath;
		TagIdType ParentTag;
	};

public:
	explicit FTagTracker(IAnalysisSession& Session);
	void AddTagSpec(TagIdType Tag, TagIdType ParentTag, const TCHAR* Display);
	void PushTag(uint32 ThreadId, uint8 Tracker, TagIdType Tag);
	void PopTag(uint32 ThreadId, uint8 Tracker);
	TagIdType GetCurrentTag(uint32 ThreadId, uint8 Tracker) const;
	const TCHAR* GetTagString(TagIdType Tag) const;
	const TCHAR* GetTagFullPath(TagIdType Tag) const;
	void EnumerateTags(TFunctionRef<void(const TCHAR*, const TCHAR*, TagIdType, TagIdType)> Callback) const;

	void PushTagFromPtr(uint32 ThreadId, uint8 Tracker, TagIdType Tag);
	void PopTagFromPtr(uint32 ThreadId, uint8 Tracker);
	bool HasTagFromPtrScope(uint32 ThreadId, uint8 Tracker) const;

	uint32 GetNumErrors() const { return NumErrors; }
	uint32 GetNumWarnings() const { return NumWarnings; }

private:
	void BuildTagPath(FStringBuilderBase& OutString, FStringView Name, TagIdType ParentTagId);
	static inline uint32 GetTrackerThreadId(uint32 ThreadId, uint8 Tracker)
	{
		return (Tracker << TrackerIdShift) | (~TrackerIdMask & ThreadId);
	}

	IAnalysisSession& Session;
	TMap<uint32, FThreadState> TrackerThreadStates;
	TMap<TagIdType, FTagEntry> TagMap;
	TArray<TTuple<TagIdType, FString>> PendingTags;
	TagIdType CustomNameTag = InvalidTagId;
	uint32 NumErrors = 0;
	uint32 NumWarnings = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FShortLivingAllocs
{
private:
	struct FNode
	{
		FAllocationItem* Alloc;
		FNode* Next;
		FNode* Prev;
	};

	static constexpr int32 MaxAllocCount = 64; // max number of short living allocations

public:
	FShortLivingAllocs();
	~FShortLivingAllocs();

	void Reset();

	bool IsFull() const { return AllocCount == MaxAllocCount; }
	int32 Num() const { return AllocCount; }

	// Finds the allocation with the specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRef(uint64 Address) const;

	// Finds an allocation containing the specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRange(uint64 Address) const;

	FORCEINLINE void Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;
	FORCEINLINE void Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;

	// The collection keeps ownership of FAllocationItem* until Remove is called or until the oldest allocation is removed.
	// Returns the removed oldest allocation if collection is already full; nullptr otherwise.
	// The caller receives ownership of the removed oldest allocation, if a valid pointer is returned.
	FORCEINLINE FAllocationItem* AddAndRemoveOldest(FAllocationItem* Alloc);

	// The caller takes ownership of FAllocationItem*. Returns nullptr if Address is not found.
	FORCEINLINE FAllocationItem* Remove(uint64 Address);

private:
	//TMap<uint64, FNode*> AddressMap; // map for short living allocations: Address -> FNode* // #if INSIGHTS_SLA_USE_ADDRESS_MAP
	FNode* AllNodes = nullptr; // preallocated array of nodes (MaxAllocCount nodes)
	FNode* LastAddedAllocNode = nullptr; // the last added alloc; double linked list: Prev -> .. -> OldestAlloc
	FNode* OldestAllocNode = nullptr; // the oldest alloc; double linked list: Next -> .. -> LastAddedAlloc
	FNode* FirstUnusedNode = nullptr; // simple linked list with unused nodes (uses Next)
	int32 AllocCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FHeapAllocs
{
private:
	struct FNode
	{
		FAllocationItem* Alloc;
		FNode* Next;
		FNode* Prev;
	};

	struct FList
	{
		FNode* First = nullptr;
		FNode* Last = nullptr;
	};

public:
	FHeapAllocs();
	~FHeapAllocs();

	void Reset();

	int32 Num() const { return AllocCount; }

	// Finds the last heap allocation with specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRef(uint64 Address) const;

	// Finds a heap allocation containing the specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRange(uint64 Address) const;

	FORCEINLINE void Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;
	FORCEINLINE void Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;

	// The collection keeps ownership of FAllocationItem* until Remove is called.
	FORCEINLINE void Add(FAllocationItem* Alloc);

	// The caller takes ownership of FAllocationItem*. Returns nullptr if Address is not found.
	FORCEINLINE FAllocationItem* Remove(uint64 Address);

private:
	TMap<uint64, FList> AddressMap; // map heap allocs: Address -> list of heap allocs
	int32 AllocCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLiveAllocCollection
{
public:
	FLiveAllocCollection();
	~FLiveAllocCollection();

	uint32 Num() const { return TotalAllocCount; }
	uint32 PeakCount() const { return MaxAllocCount; }

	// Finds the allocation with specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindRef(uint64 Address) const;

	// Finds the heap allocation with specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindHeapRef(uint64 Address) const;

	// Finds the swap allocation with specified address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindSwapRef(uint64 Address) const;

	// Finds an allocation containing the address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindByAddressRange(uint64 Address) const;

	// Finds the heap allocation containing the address.
	// Returns the found allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* FindHeapByAddressRange(uint64 Address) const;

	// Enumerates all allocations (including heap allocations).
	FORCEINLINE void Enumerate(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;

	// Enumerates allocations in a specified address range (including heap allocations).
	FORCEINLINE void Enumerate(uint64 StartAddress, uint64 EndAddress, TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;

	// Adds a new allocation with specified address.
	// The collection keeps ownership of FAllocationItem* until Remove is called.
	// Returns the new added allocation.
	FORCEINLINE FAllocationItem* AddNew(uint64 Address);

	// Adds a new allocation.
	// The collection keeps ownership of FAllocationItem* until Remove is called.
	FORCEINLINE void Add(FAllocationItem* Alloc);

	// Adds a new heap allocation with specified address.
	// The collection keeps ownership of FAllocationItem* until RemoveHeap is called.
	// Returns the new added allocation.
	FORCEINLINE FAllocationItem* AddNewHeap(uint64 Address);

	// Adds a new heap allocation.
	// The collection keeps ownership of FAllocationItem* until RemoveHeap is called.
	FORCEINLINE void AddHeap(FAllocationItem* HeapAlloc);

	// Removes an allocation with specified address.
	// The caller takes ownership of FAllocationItem*.
	// Returns the removed allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* Remove(uint64 Address);

	// Removes a heap allocation with specified address.
	// The caller takes ownership of FAllocationItem*.
	// Returns the removed heap allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* RemoveHeap(uint64 Address);

	// Adds a new swap allocation with specified address (a memory page that was moved to swap).
	// The collection keeps ownership of FAllocationItem* until RemoveSwap is called.
	// Returns the new added allocation.
	FORCEINLINE FAllocationItem* AddNewSwap(uint64 Address);

	// Adds a new swap allocation.
	// The collection keeps ownership of FAllocationItem* until RemoveSwap is called.
	FORCEINLINE void AddSwap(FAllocationItem* SwapAlloc);

	// Removes a swap allocation with specified address.
	// The caller takes ownership of FAllocationItem*.
	// Returns the removed swap allocation or nullptr if not found.
	FORCEINLINE FAllocationItem* RemoveSwap(uint64 Address);

private:
	FAllocationItem* LastAlloc = nullptr; // last allocation
	FShortLivingAllocs ShortLivingAllocs; // short living allocations
	FAllocMap LongLivingAllocs; // long living allocations
	FHeapAllocs HeapAllocs; // heap allocations
	FHeapAllocs SwapAllocs; // swap allocations (memory pages stored in swap)

	uint32 TotalAllocCount = 0;
	uint32 MaxAllocCount = 0; // debug stats
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAllocationsProvider
	: public IAllocationsProvider
	, public IEditableProvider
{
private:
	static constexpr double DefaultTimelineSampleGranularity = 0.0001; // 0.1ms

	// Number of supported root heaps
	static constexpr uint32 MaxRootHeaps = 16;

	struct FRootHeap
	{
		void UpdateHistogramByAllocSize(uint64 Size);
		void UpdateHistogramByEventDistance(uint32 EventDistance);

		void LogStats();
		void LogAllocSizeHistogram();
		void LogEventDistanceHistogram();

		FHeapSpec* HeapSpec = nullptr;

		FLiveAllocCollection* LiveAllocs = nullptr;

		FSbTree* SbTree = nullptr;

		uint32 EventIndex = 0;

		static constexpr uint32 NumAllocSizeHistogramPow2 = 66;
		uint64 AllocSizeHistogramPow2[NumAllocSizeHistogramPow2] = { 0 };
		uint64 MaxAllocSize = 0;

		static constexpr uint32 NumEventDistanceHistogramPow10 = 13;
		uint32 EventDistanceHistogramPow10[NumEventDistanceHistogramPow10] = { 0 };
		uint32 MaxEventDistance = 0;
	};

public:
	explicit FAllocationsProvider(IAnalysisSession& InSession, FMetadataProvider& InMetadataProvider);
	virtual ~FAllocationsProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override       { Lock.BeginRead(GAllocationsProviderLockState); }
	virtual void EndRead() const override         { Lock.EndRead(GAllocationsProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GAllocationsProviderLockState); }

	virtual bool IsInitialized() const override { ReadAccessCheck(); return bInitialized; }

	virtual bool HasAllocationEvents() const { ReadAccessCheck(); return AllocEvents > 0 || FreeEvents > 0 || HeapEvents > 0; }

	virtual bool HasSwapOpEvents() const { ReadAccessCheck(); return SwapOpEvents > 0; }

	virtual void EnumerateTags(TFunctionRef<void(const TCHAR*, const TCHAR*, TagIdType, TagIdType)> Callback) const override;

	virtual const TCHAR* GetTagName(TagIdType Tag) const override { ReadAccessCheck(); return TagTracker.GetTagString(Tag); }
	virtual const TCHAR* GetTagFullPath(TagIdType Tag) const override { ReadAccessCheck(); return TagTracker.GetTagFullPath(Tag); }
	bool HasTagFromPtrScope(uint32 ThreadId, uint8 Tracker) const { ReadAccessCheck(); return TagTracker.HasTagFromPtrScope(ThreadId, Tracker); }

	virtual void EnumerateRootHeaps(TFunctionRef<void(HeapId Id, const FHeapSpec&)> Callback) const override;
	virtual void EnumerateHeaps(TFunctionRef<void(HeapId Id, const FHeapSpec&)> Callback) const override;

	virtual int32 GetTimelineNumPoints() const override { ReadAccessCheck(); return static_cast<int32>(Timeline.Num()); }
	virtual void GetTimelineIndexRange(double StartTime, double EndTime, int32& StartIndex, int32& EndIndex) const override;

	virtual void EnumerateTimeline(ETimelineU64 Timeline, int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const override;
	virtual void EnumerateTimeline(ETimelineU32 Timeline, int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const override;

	virtual FQueryHandle StartQuery(const FQueryParams& Params) const override;
	virtual void CancelQuery(FQueryHandle Query) const override;
	virtual const FQueryStatus PollQuery(FQueryHandle Query) const override;

	virtual uint64 GetPlatformPageSize() const override;

	const FSbTree* GetSbTreeUnchecked(HeapId Heap) const { ReadAccessCheck(); return RootHeaps[Heap]->SbTree; }

	void EnumerateLiveAllocs(TFunctionRef<void(const FAllocationItem& Alloc)> Callback) const;
	uint32 GetNumLiveAllocs() const;

	void DebugPrint() const;

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override       { Lock.BeginWrite(GAllocationsProviderLockState); }
	virtual void EndEdit() const override         { Lock.EndWrite(GAllocationsProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GAllocationsProviderLockState); }

	void EditInit(double Time, uint8 MinAlignment, uint64 PlatformPageSize);

	void EditAlloc(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeap);
	void EditFree(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId RootHeap);
	void EditUpdateAlloc(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId RootHeap);
	void EditSwapOp(uint32 ThreadId, double Time, uint64 UnmaskedPageAddress, EMemoryTraceSwapOperation SwapOp, uint64 CompressedPageSize, uint32 CallstackId);

	void EditHeapSpec(HeapId Id, HeapId ParentId, const FStringView& Name, EMemoryTraceHeapFlags Flags);
	void EditMarkAllocationAsHeap(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId Heap, EMemoryTraceHeapAllocationFlags Flags);
	void EditUnmarkAllocationAsHeap(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId Heap);

	void EditAddTagSpec(TagIdType Tag, TagIdType ParentTag, const TCHAR* Display) { EditAccessCheck(); TagTracker.AddTagSpec(Tag, ParentTag, Display); }
	void EditPushTag(uint32 ThreadId, uint8 Tracker, TagIdType Tag);
	void EditPopTag(uint32 ThreadId, uint8 Tracker);
	void EditPushTagFromPtr(uint32 ThreadId, uint8 Tracker, uint64 Ptr, HeapId RootHeapId);
	void EditPopTagFromPtr(uint32 ThreadId, uint8 Tracker);

	void EditOnAnalysisCompleted(double Time);

	//////////////////////////////////////////////////

private:
	void AllocInternal(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, uint64 Size, uint32 Alignment, HeapId RootHeap);
	void FreeInternal(uint32 ThreadId, double Time, uint32 CallstackId, uint64 Address, HeapId RootHeap);

	bool ShouldIgnoreEvent(uint32 ThreadId, double Time, uint64 Address, HeapId RootHeapId, uint32 CallstackId);
	void AdvanceTimelines(double Time);
	void AddHeapSpec(HeapId Id, HeapId ParentId, const FStringView& Name, EMemoryTraceHeapFlags Flags);

	bool IsValidHeap(HeapId HeapId) const
	{
		return HeapId < (uint32)HeapSpecs.Num() && HeapSpecs[HeapId] != nullptr;
	}

	FHeapSpec& GetHeapSpecUnchecked(HeapId Id) const
	{
		return *HeapSpecs[Id];
	}

	FHeapSpec& GetOrCreateHeapSpec(HeapId Id);

	bool IsValidRootHeap(HeapId RootHeapId) const
	{
		return RootHeapId < MaxRootHeaps && RootHeaps[RootHeapId] != nullptr;
	}

	FRootHeap& GetRootHeapUnchecked(HeapId RootHeapId) const
	{
		return *RootHeaps[RootHeapId];
	}

	FRootHeap& FindParentRootHeapUnchecked(HeapId HeapId) const;
	void CreateRootHeap(HeapId Id);

	void SetSampleU64(ETimelineU64 TimelineIndex, uint64 Value)
	{
		uint64& Sample = SampleValuesU64[int32(TimelineIndex)];
		Sample = Value;
	}
	void SetMinSampleU64(ETimelineU64 TimelineIndex, uint64 Value)
	{
		uint64& Sample = SampleValuesU64[int32(TimelineIndex)];
		Sample = FMath::Min(Sample, Value);
	}
	void SetMaxSampleU64(ETimelineU64 TimelineIndex, uint64 Value)
	{
		uint64& Sample = SampleValuesU64[int32(TimelineIndex)];
		Sample = FMath::Max(Sample, Value);
	}
	void SetSampleU32(ETimelineU32 TimelineIndex, uint32 Value)
	{
		uint32& Sample = SampleValuesU32[int32(TimelineIndex)];
		Sample = Value;
	}
	void SetMinSampleU32(ETimelineU32 TimelineIndex, uint32 Value)
	{
		uint32& Sample = SampleValuesU32[int32(TimelineIndex)];
		Sample = FMath::Min(Sample, Value);
	}
	void SetMaxSampleU32(ETimelineU32 TimelineIndex, uint32 Value)
	{
		uint32& Sample = SampleValuesU32[int32(TimelineIndex)];
		Sample = FMath::Max(Sample, Value);
	}
	void IncSampleU32(ETimelineU32 TimelineIndex)
	{
		uint32& Sample = SampleValuesU32[int32(TimelineIndex)];
		++Sample;
	}

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;
	FMetadataProvider& MetadataProvider;

	double InitTime = 0;
	uint64 PlatformPageSize = 0;
	uint8 MinAlignment = 0;
	uint8 SizeShift = 0;
	uint8 SummarySizeShift = 0;
	bool bInitialized = false;

	FTagTracker TagTracker;

	TArray<FHeapSpec*> HeapSpecs;
	FRootHeap* RootHeaps[MaxRootHeaps] = { nullptr };

	uint64 AllocCount = 0;
	uint64 FreeCount = 0;
	uint64 HeapCount = 0;

	uint64 AllocEvents = 0;
	uint64 FreeEvents = 0;
	uint64 HeapEvents = 0;
	uint64 SwapOpEvents = 0;

	uint64 MiscWarnings = 0;
	uint64 MiscErrors = 0;
	uint64 HeapWarnings = 0;
	uint64 HeapErrors = 0;
	uint64 AllocWarnings = 0;
	uint64 AllocErrors = 0;
	uint64 FreeWarnings = 0;
	uint64 FreeErrors = 0;
	uint64 SwapWarnings = 0;
	uint64 SwapErrors = 0;

	uint64 TotalAllocatedMemory = 0;
	uint32 TotalLiveAllocations = 0;
	uint64 TotalSwapMemory = 0;
	uint64 TotalCompressedSwapMemory = 0;

	static constexpr int32 NumTimelinesU64 = int32(IAllocationsProvider::ETimelineU64::Count);
	static constexpr int32 NumTimelinesU32 = int32(IAllocationsProvider::ETimelineU32::Count);

	double SampleStartTimestamp = 0.0;
	double SampleEndTimestamp = 0.0;
	uint64 SampleValuesU64[NumTimelinesU64];
	uint32 SampleValuesU32[NumTimelinesU32];

	TPagedArray<double> Timeline;
	TPagedArray<uint64>* ValueTimelinesU64[NumTimelinesU64];
	TPagedArray<uint32>* ValueTimelinesU32[NumTimelinesU32];
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
