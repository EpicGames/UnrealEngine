// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllocationsQuery.h"

#include "SbTree.h"
#include "Common/Utils.h"
#include "Model/AllocationItem.h"
#include "Model/AllocationsProvider.h"
#include "TraceServices/Containers/Allocators.h"

#include <limits>

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAllocationsQueryAsyncTask
////////////////////////////////////////////////////////////////////////////////////////////////////

class FAllocationsQueryAsyncTask
{
public:
	FAllocationsQueryAsyncTask(FAllocationsQuery* InQuery)
	{
		QueryPtr = InQuery;
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FAllocationsQueryAsyncTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (QueryPtr)
		{
			QueryPtr->Run();
		}
	}

private:
	FAllocationsQuery* QueryPtr = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAllocationsQuery
////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsQuery::FAllocationsQuery(const FAllocationsProvider& InAllocationProvider, const IAllocationsProvider::FQueryParams& InParams)
	: AllocationsProvider(InAllocationProvider)
	, Params(InParams)
	, IsWorking(true)
	, IsCanceling(false)
{
	// Start the async task.
	CompletedEvent = TGraphTask<FAllocationsQueryAsyncTask>::CreateTask().ConstructAndDispatchWhenReady(this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsQuery::Cancel()
{
	if (CompletedEvent.IsValid())
	{
		// Cancel the async task.
		IsCanceling = true;
		CompletedEvent->Wait();
	}
	delete this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IAllocationsProvider::FQueryStatus FAllocationsQuery::Poll()
{
	IAllocationsProvider::FQueryStatus Status = {};

	FAllocationsImpl* Allocations = nullptr;
	if (Results.Dequeue(Allocations))
	{
		Status.Status = IAllocationsProvider::EQueryStatus::Available;
		Status.Handle = UPTRINT(Allocations);
		return Status;
	}

	if (IsWorking)
	{
		Status.Status = IAllocationsProvider::EQueryStatus::Working;
	}
	else
	{
		Status.Status = IAllocationsProvider::EQueryStatus::Done;
	}
	Status.Handle = 0;
	return Status;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsQuery::Run()
{
	// Note: This function is called from the async task (FAllocationsQueryAsyncTask),
	//       so no assumption can be made on which thread is running.

	uint64 StartTime = FPlatformTime::Cycles64();

	uint32 CellCount = 0;
	uint32 TotalAllocationCount = 0;

	AllocationsProvider.BeginRead();

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Processing %d live allocs...", AllocationsProvider.GetNumLiveAllocs());

	FAllocationsImpl* LiveAllocsResult = new FAllocationsImpl();
	QueryLiveAllocs(LiveAllocsResult->Items);

	const uint32 NumLiveAllocs = static_cast<uint32>(LiveAllocsResult->Items.Num());
	if (NumLiveAllocs != 0)
	{
		UE_LOGF(LogTraceServices, Log, "[MemAlloc] Enqueue %u live allocs...", NumLiveAllocs);
		TotalAllocationCount += NumLiveAllocs;

		Results.Enqueue(LiveAllocsResult);
	}
	else
	{
		delete LiveAllocsResult;
	}

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Detecting cells...");

	TArray<HeapId> Heaps;
	TArray<const FSbTreeCell*> Cells;

	AllocationsProvider.EnumerateRootHeaps([&Heaps](HeapId Heap, const IAllocationsProvider::FHeapSpec& Spec)
	{
		Heaps.Add(Heap);
	});

	for (HeapId Heap : Heaps)
	{
		if (!IsCanceling)
		{
			const FSbTree* SbTree = AllocationsProvider.GetSbTreeUnchecked(Heap);
			SbTree->Query(Cells, Params);
		}
	}

	CellCount += Cells.Num();

	UE_LOGF(LogTraceServices, Log, "[MemAlloc] %d cells to process", Cells.Num());
	uint32 CellIndex = 0;

	for (const FSbTreeCell* Cell : Cells)
	{
		if (IsCanceling)
		{
			break;
		}

		++CellIndex;
		UE_LOGF(LogTraceServices, Log, "[MemAlloc] Processing cell %u (%u allocs)...", CellIndex, Cell->GetAllocCount());

		FAllocationsImpl* CellResult = new FAllocationsImpl();

		Cell->Query(CellResult->Items, Params);

		const uint32 NumAllocs = static_cast<uint32>(CellResult->Items.Num());
		if (NumAllocs != 0)
		{
			UE_LOGF(LogTraceServices, Log, "[MemAlloc] Enqueue %u allocs...", NumAllocs);
			TotalAllocationCount += NumAllocs;

			Results.Enqueue(CellResult);
		}
		else
		{
			delete CellResult;
		}
	}

	AllocationsProvider.EndRead();

	IsWorking = false;

	uint64 EndTime = FPlatformTime::Cycles64();
	const double TotalTime = static_cast<double>(EndTime - StartTime) * FPlatformTime::GetSecondsPerCycle64();
	UE_LOGF(LogTraceServices, Log, "[MemAlloc] Allocations query completed in %.3fs (%u cells, %u allocations)",
		TotalTime, CellCount, TotalAllocationCount);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsQuery::QueryLiveAllocs(TArray<const FAllocationItem*>& OutAllocs) const
{
	// TODO: Live allocs have the end time set to std::numeric_limits<double>::infinity(),
	//       so the conditions below could be simplified to remove the end time checks.
	//       For now, the code remains "unoptimized" for debugging purposes.

	switch (Params.Rule)
	{
	case IAllocationsProvider::EQueryRule::aAf: // active allocs at A
	{
		const double Time = Params.TimeA;
		AllocationsProvider.EnumerateLiveAllocs([Time, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime <= Time && Time <= Alloc.EndTime)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::afA: // before
	{
		const double Time = Params.TimeA;
		AllocationsProvider.EnumerateLiveAllocs([Time, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.EndTime <= Time)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::Aaf: // after
	{
		const double Time = Params.TimeA;
		AllocationsProvider.EnumerateLiveAllocs([Time, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= Time)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::aAfB: // decline
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime <= TimeA && Alloc.EndTime >= TimeA && Alloc.EndTime <= TimeB)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBf: // growth
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeB)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::aAfaBf: // decline + growth
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if ((Alloc.StartTime <= TimeA && Alloc.EndTime >= TimeA && Alloc.EndTime <= TimeB) || // decline
				(Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeB)) // growth
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AfB: // free events
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.EndTime >= TimeA && Alloc.EndTime <= TimeB)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AaB: // alloc events
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AafB: // short living allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= TimeA && Alloc.EndTime <= TimeB)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::aABf: // long living allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime <= TimeA && Alloc.EndTime >= TimeB)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBCf: // memory leaks
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, TimeC, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeC)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBfC: // limited lifetime
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, TimeC, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeB && Alloc.EndTime <= TimeC)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::aABfC: // decline of long living allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, TimeC, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime <= TimeA && Alloc.EndTime >= TimeB && Alloc.EndTime <= TimeC)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBCfD: // specific lifetime
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		const double TimeD = Params.TimeD;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, TimeC, TimeD, &OutAllocs](const FAllocationItem& Alloc)
		{
			if (Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeC && Alloc.EndTime <= TimeD)
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AoB:
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if ((Alloc.IsSwap() && Alloc.StartTime >= TimeA && Alloc.StartTime <= TimeB) ||
				(!Alloc.IsSwap() && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeA)) // include allocs that were freed between A and B
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;

	case IAllocationsProvider::EQueryRule::AiB:
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		AllocationsProvider.EnumerateLiveAllocs([TimeA, TimeB, &OutAllocs](const FAllocationItem& Alloc)
		{
			if ((Alloc.IsSwap() && Alloc.EndTime >= TimeA && Alloc.EndTime <= TimeB) ||
				(!Alloc.IsSwap() && Alloc.StartTime <= TimeB && Alloc.EndTime >= TimeA)) // include allocs that were freed between A and B
			{
				OutAllocs.Add(&Alloc);
			}
		});
	}
	break;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
