// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CallstacksProvider.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Queue.h"
#include "HAL/PlatformAtomics.h"
#include "TraceServices/Model/AllocationsProvider.h"

#include <atomic>

namespace TraceServices
{

struct FAllocationItem;
class FAllocationsProvider;
class ILinearAllocator;

struct FAllocationsImpl
{
	FAllocationsImpl* Next = nullptr;
	TArray<const FAllocationItem*> Items;
};

class FAllocationsQuery
{
	friend class FAllocationsQueryAsyncTask;

public:
	FAllocationsQuery(const FAllocationsProvider& InAllocationsProvider, const IAllocationsProvider::FQueryParams& InParams);

	void Cancel();
	IAllocationsProvider::FQueryStatus Poll();

private:
	void Run();
	void QueryLiveAllocs(TArray<const FAllocationItem*>& OutAllocs) const;

private:
	const FAllocationsProvider& AllocationsProvider;

	IAllocationsProvider::FQueryParams Params;

	TQueue<FAllocationsImpl*> Results;

	std::atomic<bool> IsWorking;
	std::atomic<bool> IsCanceling;

	FGraphEventRef CompletedEvent;
};

} // namespace TraceServices
