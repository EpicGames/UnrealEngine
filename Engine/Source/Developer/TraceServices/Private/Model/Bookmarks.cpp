// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Bookmarks.h"
#include "Model/BookmarksPrivate.h"

#include "AnalysisServicePrivate.h"
#include "Common/FormatArgs.h"
#include "Common/ProviderLock.h"
#include "RegionsPrivate.h"
#include "TraceServices/Model/Regions.h"

namespace TraceServices
{

static constexpr FStringView RegionStartToken = TEXTVIEW("RegionStart:");
static constexpr FStringView RegionEndToken = TEXTVIEW("RegionEnd:");

FBookmarkProvider::FBookmarkProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
}

FBookmarkSpec& FBookmarkProvider::GetSpec(uint64 BookmarkPoint)
{
	Session.WriteAccessCheck();

	TSharedPtr<FBookmarkSpec>* Found = SpecMap.Find(BookmarkPoint);
	if (Found)
	{
		return *Found->Get();
	}

	TSharedPtr<FBookmarkSpec> Spec = MakeShared<FBookmarkSpec>();
	Spec->File = TEXT("<unknown>");
	Spec->FormatString = TEXT("<unknown>");
	SpecMap.Add(BookmarkPoint, Spec);
	return *Spec.Get();
}

void FBookmarkProvider::UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line)
{
	Session.WriteAccessCheck();

	FBookmarkSpec& BookmarkSpec = GetSpec(BookmarkPoint);
	BookmarkSpec.FormatString = FormatString;
	BookmarkSpec.File = File;
	BookmarkSpec.Line = Line;
}

void FBookmarkProvider::AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatArgs)
{
	Session.WriteAccessCheck();

	FBookmarkSpec Spec = GetSpec(BookmarkPoint);
	FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, Spec.FormatString, FormatArgs);
	TSharedRef<FBookmarkInternal> Bookmark = MakeShared<FBookmarkInternal>();
	Bookmark->Time = Time;
	Bookmark->Text = Session.StoreString(FormatBuffer);
	Bookmark->CallstackId = CallstackId;
	Bookmarks.Add(Bookmark);

	CheckBookmarkForRegion(Bookmark);

	Session.UpdateDurationSeconds(Time);
}

void FBookmarkProvider::AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text)
{
	Session.WriteAccessCheck();

	TSharedRef<FBookmarkInternal> Bookmark = MakeShared<FBookmarkInternal>();
	Bookmark->Time = Time;
	Bookmark->Text = Text;
	Bookmark->CallstackId = CallstackId;
	Bookmarks.Add(Bookmark);

	CheckBookmarkForRegion(Bookmark);

	Session.UpdateDurationSeconds(Time);
}

void FBookmarkProvider::CheckBookmarkForRegion(const TSharedRef<FBookmarkInternal> Bookmark) const
{
	IEditableRegionProvider* EditableRegionProvider = Session.EditProvider<IEditableRegionProvider>(GetRegionProviderName());
	if (!EditableRegionProvider)
	{
		return;
	}

	const FStringView Text = Bookmark->Text;

	if (Text.StartsWith(RegionStartToken))
	{
		// StringView.GetData() is not necessarily null-terminated. Since we started from a null terminated string
		// and only called RightChop() we should still be fine.

		FProviderEditScopeLock _(*EditableRegionProvider);
		EditableRegionProvider->AppendRegionBegin(Text.RightChop(RegionStartToken.Len()).GetData(), Bookmark->Time);
	}
	if (Text.StartsWith(RegionEndToken))
	{
		FProviderEditScopeLock _(*EditableRegionProvider);
		EditableRegionProvider->AppendRegionEnd(Text.RightChop(RegionEndToken.Len()).GetData(), Bookmark->Time);
	}
}

void FBookmarkProvider::EnumerateBookmarks(double IntervalStart, double IntervalEnd, TFunctionRef<void(const FBookmark &)> Callback) const
{
	Session.ReadAccessCheck();

	if (IntervalStart > IntervalEnd)
	{
		return;
	}
	int32 FirstBookmarkIndex = Algo::LowerBoundBy(Bookmarks, IntervalStart, [](const TSharedRef<FBookmarkInternal>& B)
	{
		return B->Time;
	});
	int32 BookmarkCount = Bookmarks.Num();
	if (FirstBookmarkIndex >= BookmarkCount)
	{
		return;
	}
	int32 LastBookmarkIndex = Algo::UpperBoundBy(Bookmarks, IntervalEnd, [](const TSharedRef<FBookmarkInternal>& B)
	{
		return B->Time;
	});
	if (LastBookmarkIndex == 0)
	{
		return;
	}
	--LastBookmarkIndex;
	for (int32 Index = FirstBookmarkIndex; Index <= LastBookmarkIndex; ++Index)
	{
		const FBookmarkInternal& InternalBookmark = Bookmarks[Index].Get();
		FBookmark Bookmark;
		Bookmark.Time = InternalBookmark.Time;
		Bookmark.Text = InternalBookmark.Text;
		Bookmark.CallstackId = InternalBookmark.CallstackId;
		Callback(Bookmark);
	}
}

FName GetBookmarkProviderName()
{
	static const FName Name("BookmarkProvider");
	return Name;
}

const IBookmarkProvider& ReadBookmarkProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<IBookmarkProvider>(GetBookmarkProviderName());
}

IEditableBookmarkProvider& EditBookmarkProvider(IAnalysisSession& Session)
{
	return *Session.EditProvider<IEditableBookmarkProvider>(GetBookmarkProviderName());
}

} // namespace TraceServices
