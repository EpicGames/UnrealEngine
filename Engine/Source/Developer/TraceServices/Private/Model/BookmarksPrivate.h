// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/Bookmarks.h"
#include "Templates/SharedPointer.h"

namespace TraceServices
{

class FAnalysisSessionLock;
class FStringStore;

struct FBookmarkSpec
{
	const TCHAR* File = nullptr;
	const TCHAR* FormatString = nullptr;
	int32 Line = 0;
};

struct FBookmarkInternal
{
	double Time = 0.0;
	const TCHAR* Text = nullptr;
	uint32 CallstackId = 0;
};

class FBookmarkProvider
	: public IBookmarkProvider
	, public IEditableBookmarkProvider
{
public:
	explicit FBookmarkProvider(IAnalysisSession& Session);
	virtual ~FBookmarkProvider() = default;

	FBookmarkSpec& GetSpec(uint64 BookmarkPoint);
	virtual uint64 GetBookmarkCount() const override { return Bookmarks.Num(); }
	virtual void EnumerateBookmarks(double IntervalStart, double IntervalEnd, TFunctionRef<void(const FBookmark&)> Callback) const override;

	// implement IEditableBookmarkProvider
	virtual void UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line) override;
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatArgs) override;
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text) override;

private:
	// check whether a bookmark is actually a region marker from the deprecated/5.1 regions implementation.
	// will forward to IEditableRegionProvider in this case
	void CheckBookmarkForRegion(const TSharedRef<FBookmarkInternal> Shared) const;
	
	IAnalysisSession& Session;
	TMap<uint64, TSharedPtr<FBookmarkSpec>> SpecMap;
	TArray<TSharedRef<FBookmarkInternal>> Bookmarks;

	enum
	{
		FormatBufferSize = 65536
	};
	TCHAR FormatBuffer[FormatBufferSize];
	TCHAR TempBuffer[FormatBufferSize];
};

} // namespace TraceServices
