// Copyright Epic Games, Inc. All Rights Reserved.

#include "CallstacksProvider.h"

#include "Algo/Unique.h"
#include "Containers/ArrayView.h"
#include "Misc/ScopeRWLock.h"
#include "ModuleProvider.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

static const FResolvedSymbol GNeverResolveSymbol(ESymbolQueryResult::NotLoaded, nullptr, nullptr, nullptr, 0, EResolvedSymbolFilterStatus::NotFiltered);
static const FResolvedSymbol GNotFoundSymbol(ESymbolQueryResult::NotFound, TEXT("Unknown"), nullptr, nullptr, 0, EResolvedSymbolFilterStatus::NotFiltered);
static constexpr FStackFrame GNotFoundStackFrame = { 0, &GNotFoundSymbol };
static const FCallstack GNotFoundCallstack(&GNotFoundStackFrame, 1);

////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef TRACE_CALLSTACK_STATS
static struct FCallstackProviderStats
{
	uint64 Callstacks;
	uint64 Frames;
	uint64 FrameCountHistogram[256];
} GCallstackStats;
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

FCallstacksProvider::FCallstacksProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, ModuleProvider(nullptr)
	, Callstacks(InSession.GetLinearAllocator(), CallstacksPerPage)
	, Frames(InSession.GetLinearAllocator(), FramesPerPage)
{
	// Let the first callstack to be the default empty callstack (i.e. CallstackId == 0, "callstack not recorded").
	FCallstack& FirstCallstack = Callstacks.PushBack();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCallstacksProvider::AddCallstack(uint32 InCallstackId, const uint64* InFrames, uint8 InFrameCount)
{
	if (InCallstackId == 0)
	{
		return;
	}

#ifdef TRACE_CALLSTACK_STATS
	GCallstackStats.Callstacks++;
	GCallstackStats.Frames += InFrameCount;
	GCallstackStats.FrameCountHistogram[InFrameCount]++;
#endif

	// The module provider is created on the fly so we want to cache it
	// once it's available. Note that the module provider is conditionally
	// created so EditProvider() may return a null pointer.
	if (!ModuleProvider)
	{
		ModuleProvider = Session.EditProvider<IModuleProvider>(GetModuleProviderName());
	}

	if (InFrameCount > 0)
	{
		// Make sure all the frames fit on one page by appending dummy entries.
		const uint64 PageHeadroom = Frames.GetPageSize() - (Frames.Num() % Frames.GetPageSize());
		if (PageHeadroom < InFrameCount)
		{
			FRWScopeLock WriteLock(EntriesLock, SLT_Write);
			uint64 EntriesToAdd = PageHeadroom + 1; // Fill page and allocate one on next
			do { Frames.PushBack(); } while (--EntriesToAdd);
		}

		// Append the incoming frames.
		for (uint32 FrameIdx = 0; FrameIdx < InFrameCount; ++FrameIdx)
		{
			FStackFrame& F = Frames.PushBack();
			F.Addr = InFrames[FrameIdx];

			if (ModuleProvider)
			{
				// This will return immediately. The result will be empty if the symbol
				// has not been encountered before, and resolution has been queued up.
				F.Symbol = ModuleProvider->GetSymbol(InFrames[FrameIdx]);
			}
			else
			{
				F.Symbol = &GNeverResolveSymbol;
			}
		}
	}

	{
		FRWScopeLock WriteLock(EntriesLock, SLT_Write);
		FCallstack* Callstack = nullptr;
		if (InCallstackId < Callstacks.Num())
		{
			Callstack = &Callstacks[InCallstackId];
		}
		else
		{
			while (InCallstackId >= Callstacks.Num())
			{
				Callstack = &Callstacks.PushBack();
				Callstack->InitEmpty(Callstacks.Num() - 1);
			}
		}
		check(Callstack);
		check(Callstack->IsEmpty() && Callstack->GetEmptyId() == uint64(InCallstackId)); // adding same callstack id twice?
		if (InFrameCount > 0)
		{
			check(InFrameCount <= Frames.Num());
			Callstack->Init(&Frames[Frames.Num() - InFrameCount], InFrameCount);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FCallstacksProvider::AddCallstackWithHash(uint64 InCallstackHash, const uint64* InFrames, uint8 InFrameCount)
{
	if (InCallstackHash == 0)
	{
		return 0;
	}

	uint32 CallstackId;
	{
		FRWScopeLock WriteLock(EntriesLock, SLT_Write);
		CallstackId = static_cast<uint32>(Callstacks.Num());
		CallstackMap.Add(InCallstackHash, CallstackId);
	}

	AddCallstack(CallstackId, InFrames, InFrameCount);
	return CallstackId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FCallstacksProvider::GetCallstackIdForHash(uint64 InCallstackHash) const
{
	if (InCallstackHash == 0)
	{
		return 0;
	}
	FRWScopeLock ReadLock(EntriesLock, SLT_ReadOnly);
	const uint32* CallstackIdPtr = CallstackMap.Find(InCallstackHash);
	if (CallstackIdPtr)
	{
		return *CallstackIdPtr;
	}
	else
	{
		return 0;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FCallstack* FCallstacksProvider::GetCallstack(uint32 CallstackId) const
{
	FRWScopeLock ReadLock(EntriesLock, SLT_ReadOnly);
	if (CallstackId < Callstacks.Num())
	{
		return &Callstacks[CallstackId];
	}
	else
	{
		return &GNotFoundCallstack;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCallstacksProvider::GetCallstacks(const TArrayView<uint32>& CallstackIds, FCallstack const** OutCallstacks) const
{
	uint64 OutIdx(0);
	check(OutCallstacks != nullptr);

	FRWScopeLock ReadLock(EntriesLock, SLT_ReadOnly);
	for (uint64 CallstackId : CallstackIds)
	{
		if (CallstackId < Callstacks.Num())
		{
			OutCallstacks[OutIdx] = &Callstacks[CallstackId];
		}
		else
		{
			OutCallstacks[OutIdx] = &GNotFoundCallstack;
		}
		OutIdx++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName GetCallstacksProviderName()
{
	static const FName Name("CallstacksProvider");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const ICallstacksProvider* ReadCallstacksProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<ICallstacksProvider>(GetCallstacksProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
