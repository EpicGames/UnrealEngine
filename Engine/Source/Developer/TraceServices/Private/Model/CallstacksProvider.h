// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/Callstack.h"
#include "HAL/CriticalSection.h"
#include "Common/PagedArray.h"
#include "Containers/Map.h"

namespace TraceServices
{

class IAnalysisSession;
class IModuleProvider;

/////////////////////////////////////////////////////////////////////
class FCallstacksProvider : public ICallstacksProvider
{
public:
	explicit FCallstacksProvider(IAnalysisSession& Session);
	virtual ~FCallstacksProvider() = default;

	const FCallstack* GetCallstack(uint32 CallstackId) const override;
	void GetCallstacks(const TArrayView<uint32>& CallstackIds, FCallstack const** OutCallstacks) const override;
	void AddCallstack(uint32 CallstackId, const uint64* Frames, uint8 FrameCount);

	// Backward compatibility with legacy memory trace format (5.0-EA).
	uint32 AddCallstackWithHash(uint64 CallstackHash, const uint64* Frames, uint8 FrameCount);
	uint32 GetCallstackIdForHash(uint64 CallstackHash) const override;

private:
	enum
	{
		FramesPerPage		= 65536, // 16 bytes/entry -> 1 Mb per page
		CallstacksPerPage	= 65536 * 2 // 8 bytes/callstack -> 1Mb per page
	};

	mutable FRWLock					EntriesLock;
	IAnalysisSession&				Session;
	mutable IModuleProvider*		ModuleProvider;
	TPagedArray<FCallstack>			Callstacks; // CallstackId is an index in this array; Callstacks[0] is an empty callstack
	TPagedArray<FStackFrame>		Frames;
	TMap<uint64, uint32>			CallstackMap; // (CallstackHash --> CallstackId) map, used for backward compatibility with legacy trace format
};

} // namespace TraceServices
