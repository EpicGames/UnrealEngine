// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channel.h"

#include "Misc/DateTime.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

FChannelProvider::FChannelProvider()
{
	TimeStamp = FDateTime::Now();
}

void FChannelProvider::AnnounceChannel(const TCHAR* InChannelName, uint32 Id, bool bReadOnly)
{
	FString ChannelName(InChannelName);

	if (*InChannelName)
	{
		ChannelName.GetCharArray()[0] = TChar<TCHAR>::ToUpper(ChannelName.GetCharArray()[0]);
	}

	Channels.Add(FChannelEntry{ Id, ChannelName, false, bReadOnly });

	TimeStamp = FDateTime::Now();
}

void FChannelProvider::UpdateChannel(uint32 Id, bool bEnabled)
{
	const auto FoundEntry = Channels.FindByPredicate(
		[Id](const FChannelEntry& Entry)
		{
			return Entry.Id == Id;
		});

	if (FoundEntry)
	{
		FoundEntry->bIsEnabled = bEnabled;
	}

	TimeStamp = FDateTime::Now();
}

uint64 FChannelProvider::GetChannelCount() const
{
	return Channels.Num();
}

const TArray<FChannelEntry>& FChannelProvider::GetChannels() const
{
	return Channels;
}

FDateTime FChannelProvider::GetTimeStamp() const
{
	return TimeStamp;
}

FName GetChannelProviderName()
{
	static const FName Name("ChannelProvider");
	return Name;
}

const IChannelProvider* ReadChannelProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IChannelProvider>(GetChannelProviderName());
}

} // namespace TraceServices
