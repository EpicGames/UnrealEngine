// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/DateTime.h"
#include "TraceServices/Model/Channel.h"

namespace TraceServices
{

class FChannelProvider : public IChannelProvider
{
public:
	FChannelProvider();
	virtual ~FChannelProvider() = default;

	void AnnounceChannel(const TCHAR* ChannelName, uint32 Id, bool bReadOnly);
	void UpdateChannel(uint32 Id, bool bEnabled);

	virtual uint64 GetChannelCount() const override;
	virtual const TArray<FChannelEntry>& GetChannels() const override;

	virtual FDateTime GetTimeStamp() const override;

private:
	TArray<FChannelEntry> Channels;
	FDateTime TimeStamp;
};

} // namespace TraceServices
