// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/ContextSwitches.h"
#include "Model/ContextSwitchesPrivate.h"

#include "Algo/Sort.h"

// TraceServices
#include "AnalysisServicePrivate.h"

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GContextSwitchesProviderLockState;

FContextSwitchesProvider::FContextSwitchesProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, NumCpuCores(0)
{
}

FContextSwitchesProvider::~FContextSwitchesProvider()
{
	for (const auto& ContextSwitches : Threads)
	{
		delete ContextSwitches.Value;
	}
	Threads.Reset();

	for (const auto& CpuCoreEvents : CpuCores)
	{
		if (CpuCoreEvents)
		{
			delete CpuCoreEvents;
		}
	}
	CpuCores.Reset();
}

bool FContextSwitchesProvider::HasData() const
{
	ReadAccessCheck();

	return Threads.Num() > 0;
}

bool FContextSwitchesProvider::GetSystemThreadId(uint32 ThreadId, uint32& OutSystemThreadId) const
{
	ReadAccessCheck();

	const uint32* SystemThreadIdPtr = TraceToSystemThreadIdMap.Find(ThreadId);
	if (SystemThreadIdPtr)
	{
		OutSystemThreadId = *SystemThreadIdPtr;
		return true;
	}
	return false;
}

bool FContextSwitchesProvider::GetThreadId(uint32 SystemThreadId, uint32& OutThreadId) const
{
	ReadAccessCheck();

	const uint32* ThreadIdPtr = SystemToTraceThreadIdMap.Find(SystemThreadId);
	if (ThreadIdPtr)
	{
		OutThreadId = *ThreadIdPtr;
		return true;
	}
	return false;
}

bool FContextSwitchesProvider::GetSystemThreadId(uint32 CoreNumber, double Time, uint32& OutSystemThreadId) const
{
	ReadAccessCheck();

	if (CoreNumber >= (uint32)CpuCores.Num())
	{
		return false;
	}

	const TPagedArray<FCpuCoreEvent>* CpuCoreEvents = CpuCores[CoreNumber];
	if (!CpuCoreEvents)
	{
		return false;
	}

	uint64 PageIndex = Algo::LowerBoundBy(*CpuCoreEvents, Time, [](const TPagedArrayPage<FCpuCoreEvent>& In) -> double
		{
			return GetFirstItem(In)->Start;
		});

	if (PageIndex < GetNum(*CpuCoreEvents))
	{
		const TPagedArrayPage<FCpuCoreEvent>& Page = GetData(*CpuCoreEvents)[PageIndex];
		uint64 Index = Algo::LowerBoundBy(Page, Time, [](const FCpuCoreEvent& In) -> double
			{
				return In.Start;
			});

		if (Index < GetNum(Page))
		{
			const FCpuCoreEvent& CpuCoreEvent = GetData(Page)[Index];
			if (CpuCoreEvent.Start >= Time && CpuCoreEvent.End < Time)
			{
				OutSystemThreadId = CpuCoreEvent.SystemThreadId;
				return true;
			}
		}
	}

	return false;
}

bool FContextSwitchesProvider::GetThreadId(uint32 CoreNumber, double Time, uint32& OutThreadId) const
{
	ReadAccessCheck();

	uint32 SystemThreadId;
	if (GetSystemThreadId(CoreNumber, Time, SystemThreadId))
	{
		return GetThreadId(SystemThreadId, OutThreadId);
	}

	return false;
}

bool FContextSwitchesProvider::GetCoreNumber(uint32 ThreadId, double Time, uint32& OutCoreNumber) const
{
	ReadAccessCheck();

	const TPagedArray<FContextSwitch>* ContextSwitches = GetContextSwitches(ThreadId);
	if (!ContextSwitches)
	{
		return false;
	}

	uint64 PageIndex = Algo::LowerBoundBy(*ContextSwitches, Time, [](const TPagedArrayPage<FContextSwitch>& In) -> double
		{
			return GetFirstItem(In)->Start;
		});

	if (PageIndex < GetNum(*ContextSwitches))
	{
		const TPagedArrayPage<FContextSwitch>& Page = GetData(*ContextSwitches)[PageIndex];
		uint64 Index = Algo::LowerBoundBy(Page, Time, [](const FContextSwitch& In) -> double
			{
				return In.Start;
			});

		if (Index < GetNum(Page))
		{
			const FContextSwitch& ContextSwitch = GetData(Page)[Index];
			if (ContextSwitch.Start >= Time && ContextSwitch.End < Time)
			{
				OutCoreNumber = ContextSwitch.CoreNumber;
				return true;
			}
		}
	}

	return false;
}

void FContextSwitchesProvider::EnumerateCpuCores(CpuCoreCallback Callback) const
{
	ReadAccessCheck();

	const uint32 CoreCount = (uint32)CpuCores.Num();
	for (uint32 CoreNumber = 0; CoreNumber < CoreCount; ++CoreNumber)
	{
		if (CpuCores[CoreNumber] != nullptr)
		{
			FCpuCoreInfo CpuCoreInfo;
			CpuCoreInfo.CoreNumber = CoreNumber;
			Callback(CpuCoreInfo);
		}
	}
}

void FContextSwitchesProvider::EnumerateCpuCoreEvents(uint32 CoreNumber, double StartTime, double EndTime, CpuCoreEventCallback Callback) const
{
	ReadAccessCheck();

	if (CoreNumber >= (uint32)CpuCores.Num())
	{
		return;
	}

	const TPagedArray<FCpuCoreEvent>* CpuCoreEvents = CpuCores[CoreNumber];
	if (CpuCoreEvents == nullptr)
	{
		return;
	}

	uint64 PageIndex = Algo::UpperBoundBy(*CpuCoreEvents, StartTime, [](const TPagedArrayPage<FCpuCoreEvent>& Page)
		{
			return Page.Items[0].Start;
		});

	if (PageIndex > 0)
	{
		--PageIndex;
	}

	auto Iterator = CpuCoreEvents->GetIteratorFromPage(PageIndex);
	const FCpuCoreEvent* CurrentCpuCoreEvent = Iterator.GetCurrentItem();
	while (CurrentCpuCoreEvent && CurrentCpuCoreEvent->Start < EndTime)
	{
		if (CurrentCpuCoreEvent->End > StartTime)
		{
			if (Callback(*CurrentCpuCoreEvent) == EContextSwitchEnumerationResult::Stop)
			{
				break;
			}
		}

		CurrentCpuCoreEvent = Iterator.NextItem();
	}
}

void FContextSwitchesProvider::EnumerateCpuCoreEventsBackwards(uint32 CoreNumber, double EndTime, double StartTime, CpuCoreEventCallback Callback) const
{
	ReadAccessCheck();

	if (CoreNumber >= (uint32)CpuCores.Num())
	{
		return;
	}

	const TPagedArray<FCpuCoreEvent>* CpuCoreEvents = CpuCores[CoreNumber];
	if (CpuCoreEvents == nullptr)
	{
		return;
	}

	uint64 PageIndex = Algo::UpperBoundBy(*CpuCoreEvents, EndTime, [](const TPagedArrayPage<FCpuCoreEvent>& Page)
		{
			return Page.Items[0].Start;
		});

	if (PageIndex == 0)
	{
		return;
	}

	auto Iterator = PageIndex < CpuCoreEvents->NumPages() ?
		CpuCoreEvents->GetIteratorFromPage(PageIndex) :
		CpuCoreEvents->GetIteratorFromItem(CpuCoreEvents->Num() - 1);
	const FCpuCoreEvent* CurrentCpuCoreEvent = Iterator.GetCurrentItem();
	while (CurrentCpuCoreEvent && CurrentCpuCoreEvent->End > StartTime)
	{
		if (CurrentCpuCoreEvent->Start < EndTime)
		{
			if (Callback(*CurrentCpuCoreEvent) == EContextSwitchEnumerationResult::Stop)
			{
				break;
			}
		}

		CurrentCpuCoreEvent = Iterator.PrevItem();
	}
}

const TPagedArray<FContextSwitch>* FContextSwitchesProvider::GetContextSwitches(uint32 ThreadId) const
{
	ReadAccessCheck();

	if (TraceToSystemThreadIdMap.Contains(ThreadId))
	{
		auto ContextSwitches = Threads.Find(TraceToSystemThreadIdMap[ThreadId]);
		return ContextSwitches ? *ContextSwitches : nullptr;
	}

	return nullptr;
}

void FContextSwitchesProvider::EnumerateContextSwitches(uint32 ThreadId, double StartTime, double EndTime, ContextSwitchCallback Callback) const
{
	ReadAccessCheck();

	const TPagedArray<FContextSwitch>* ContextSwitches = GetContextSwitches(ThreadId);
	if (ContextSwitches == nullptr)
	{
		return;
	}

	uint64 PageIndex = Algo::UpperBoundBy(*ContextSwitches, StartTime, [](const TPagedArrayPage<FContextSwitch>& Page)
		{
			return Page.Items[0].Start;
		});

	if (PageIndex > 0)
	{
		--PageIndex;
	}

	auto Iterator = ContextSwitches->GetIteratorFromPage(PageIndex);
	const FContextSwitch* CurrentContextSwitch = Iterator.NextItem();
	while (CurrentContextSwitch && CurrentContextSwitch->Start < EndTime)
	{
		if (CurrentContextSwitch->End > StartTime)
		{
			if (Callback(*CurrentContextSwitch) == EContextSwitchEnumerationResult::Stop)
			{
				break;
			}
		}

		CurrentContextSwitch = Iterator.NextItem();
	}
}

void FContextSwitchesProvider::AddTimingEvent(uint32 SystemThreadId, double Start, double End, uint32 CoreNumber)
{
	EditAccessCheck();

	//////////////////////////////////////////////////
	// Context Switch events

	TPagedArray<FContextSwitch>** ContextSwitches = Threads.Find(SystemThreadId);
	if (!ContextSwitches)
	{
		ContextSwitches = &Threads.Add(SystemThreadId, new TPagedArray<FContextSwitch>(Session.GetLinearAllocator(), 4096));
	}

	FContextSwitch& ContextSwitch = (*ContextSwitches)->PushBack();
	ContextSwitch.Start = Start;
	ContextSwitch.End = End;
	ContextSwitch.CoreNumber = CoreNumber;

	//////////////////////////////////////////////////
	// Cpu Core events

	if (CoreNumber >= (uint32)CpuCores.Num())
	{
		CpuCores.AddDefaulted(CoreNumber - CpuCores.Num() + 1);
	}
	TPagedArray<FCpuCoreEvent>* CpuCoreEvents = CpuCores[CoreNumber];
	if (!CpuCoreEvents)
	{
		CpuCoreEvents = new TPagedArray<FCpuCoreEvent>(Session.GetLinearAllocator(), 4096);
		++NumCpuCores;
		CpuCores[CoreNumber] = CpuCoreEvents;
	}

#if 0 // for debugging
	auto Page = CpuCoreEvents->GetLastPage();
	if (Page)
	{
		const FCpuCoreEvent* LastCpuCoreEvent = TraceServices::GetLastItem(*Page);
		if (LastCpuCoreEvent)
		{
			check(LastCpuCoreEvent->End <= Start);
		}
	}
#endif

	FCpuCoreEvent& CpuCoreEvent = CpuCoreEvents->PushBack();
	CpuCoreEvent.Start = Start;
	CpuCoreEvent.End = End;
	CpuCoreEvent.SystemThreadId = SystemThreadId;
}

void FContextSwitchesProvider::AddThreadInfo(uint32 ThreadId, uint32 SystemThreadId)
{
	EditAccessCheck();

	if (SystemThreadId == 0)
	{
		// At the start of a session some threads might be received with a SystemThreadId of 0.
		return;
	}

	uint32* OldSystemThreadIdPtr = TraceToSystemThreadIdMap.Find(ThreadId);
	if (OldSystemThreadIdPtr)
	{
		uint32 OldSystemThreadId = *OldSystemThreadIdPtr;
		if (ThreadId == 2 /*GameThread*/ && OldSystemThreadId != SystemThreadId)
		{
			// When -nothreading is used, multiple fake threads (FFakeThread) are created from the game thread.
			// These will have same trace ThreadId == 2 and different SystemThreadId.
			SystemToTraceThreadIdMap.Add(SystemThreadId, ThreadId);
			return;
		}
		ensure(OldSystemThreadId == SystemThreadId);
	}

	TraceToSystemThreadIdMap.Add(ThreadId, SystemThreadId);
	SystemToTraceThreadIdMap.Add(SystemThreadId, ThreadId);
}

void FContextSwitchesProvider::AddThreadName(uint32 SystemThreadId, uint32 SystemProcessId, FStringView Name)
{
	//TODO
}

FName GetContextSwitchesProviderName()
{
	static const FName Name("ContextSwitchesProvider");
	return Name;
}

const IContextSwitchesProvider* ReadContextSwitchesProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IContextSwitchesProvider>(GetContextSwitchesProviderName());
}

} // namespace TraceServices
