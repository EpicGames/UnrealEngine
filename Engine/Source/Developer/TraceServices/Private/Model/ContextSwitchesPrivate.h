// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"

// TraceServices
#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/ContextSwitches.h"

namespace TraceServices
{

extern thread_local FProviderLock::FThreadLocalState GContextSwitchesProviderLockState;

class FContextSwitchesProvider
	: public IContextSwitchesProvider
	, public IEditableProvider
{
public:
	explicit FContextSwitchesProvider(IAnalysisSession& Session);
	virtual ~FContextSwitchesProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override { Lock.BeginRead(GContextSwitchesProviderLockState); }
	virtual void EndRead() const override { Lock.EndRead(GContextSwitchesProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GContextSwitchesProviderLockState); }

	virtual bool HasData() const override;
	virtual bool GetSystemThreadId(uint32 ThreadId, uint32& OutSystemThreadId) const override;
	virtual bool GetThreadId(uint32 SystemThreadId, uint32& OutThreadId) const override;
	virtual bool GetSystemThreadId(uint32 CoreNumber, double Time, uint32& OutSystemThreadId) const override;
	virtual bool GetThreadId(uint32 CoreNumber, double Time, uint32& OutThreadId) const override;
	virtual bool GetCoreNumber(uint32 ThreadId, double Time, uint32& OutCoreNumber) const override;
	virtual uint64 GetThreadsSerial() const override { ReadAccessCheck(); return uint64(Threads.Num()); }
	virtual uint64 GetCpuCoresSerial() const override { ReadAccessCheck(); return uint64(NumCpuCores); }
	virtual void EnumerateCpuCores(CpuCoreCallback Callback) const override;
	virtual void EnumerateContextSwitches(uint32 ThreadId, double StartTime, double EndTime, ContextSwitchCallback Callback) const override;
	virtual void EnumerateCpuCoreEvents(uint32 CoreNumber, double StartTime, double EndTime, CpuCoreEventCallback Callback) const override;
	virtual void EnumerateCpuCoreEventsBackwards(uint32 CoreNumber, double EndTime, double StartTime, CpuCoreEventCallback Callback) const override;

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override { Lock.BeginWrite(GContextSwitchesProviderLockState); }
	virtual void EndEdit() const override { Lock.EndWrite(GContextSwitchesProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GContextSwitchesProviderLockState); }

	void AddTimingEvent(uint32 SystemThreadId, double StartTime, double EndTime, uint32 CoreNumber);
	void AddThreadInfo(uint32 ThreadId, uint32 SystemThreadId);
	void AddThreadName(uint32 SystemThreadId, uint32 SystemProcessId, FStringView Name);

	//////////////////////////////////////////////////

private:
	uint32 GetNumCpuCoresWithEvents() const { return NumCpuCores; }
	uint32 GetExclusiveMaxCpuCoreNumber() const { return CpuCores.Num(); }
	const TPagedArray<FContextSwitch>* GetContextSwitches(uint32 ThreadId) const;

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;

	// (Trace) Thread Id -> System Thread Id
	TMap<uint32, uint32> TraceToSystemThreadIdMap;

	// System Thread Id -> (Trace) Thread Id
	TMap<uint32, uint32> SystemToTraceThreadIdMap;

	// System Thread Id -> PagedArray
	TMap<uint32, TPagedArray<FContextSwitch>*> Threads;

	// [Core Number] -> PagedArray; some can be nullptr
	TArray<TPagedArray<FCpuCoreEvent>*> CpuCores;
	uint32 NumCpuCores;
};

} // namespace TraceServices
