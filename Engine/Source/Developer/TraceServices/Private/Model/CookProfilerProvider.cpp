// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/CookProfilerProvider.h"
#include "CookProfilerProviderPrivate.h"
#include "AnalysisServicePrivate.h"

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GCookProviderLockState;

const TCHAR* GUnknownPackage = TEXT("Unknown Package");
const TCHAR* GUnknownClass = TEXT("Unknown Class");

FPackageData::FPackageData(uint64 InId)
	: Id(InId)
	, Name(GUnknownPackage)
	, AssetClass(GUnknownClass)
{
}

FCookProfilerProvider::FCookProfilerProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FCookProfilerProvider::EnumeratePackages(double StartTime, double EndTime, EnumeratePackagesCallback Callback) const
{
	ReadAccessCheck();

	for (const FPackageData& Package : Packages)
	{
		if (Callback(Package) == false)
		{
			break;
		}
	}
}

uint32 FCookProfilerProvider::GetNumPackages() const
{
	ReadAccessCheck();

	return (uint32) Packages.Num();
}

FPackageData* FCookProfilerProvider::EditPackage(uint64 Id)
{
	EditAccessCheck();

	uint32 Index = FindOrAddPackage(Id);

	FPackageData& Package = Packages[Index];
	return &Package;
}

uint32 FCookProfilerProvider::FindOrAddPackage(uint64 Id)
{
	EditAccessCheck();

	uint32* Index = PackageIdToIndexMap.Find(Id);
	if (Index != nullptr)
	{
		return *Index;
	}
	else
	{
		PackageIdToIndexMap.Add(Id, static_cast<uint32>(Packages.Num()));
		Packages.Emplace(Id);
		return static_cast<uint32>(Packages.Num() - 1);
	}
}

TPagedArray<FPackageScope>& FCookProfilerProvider::FindOrAddScopeEntries(uint32 ThreadId)
{
	EditAccessCheck();

	TPagedArray<FPackageScope>** ThreadScopeEntriesPtr = ScopeEntries.Find(ThreadId);
	if (ThreadScopeEntriesPtr == nullptr)
	{
		return *(ScopeEntries.Add(ThreadId, new TPagedArray<FPackageScope>(Session.GetLinearAllocator(), 4096)));
	}

	return **ThreadScopeEntriesPtr;
}

void FCookProfilerProvider::AddScopeEntry(uint32 ThreadId, uint64 InPackageId, double Timestamp, EPackageEventStatType InType, bool InIsEnterScope)
{
	EditAccessCheck();

	TPagedArray<FPackageScope>& Entries = FindOrAddScopeEntries(ThreadId);
	Entries.EmplaceBack(InPackageId, Timestamp, InType, InIsEnterScope);
}

void FCookProfilerProvider::CreateAggregation(TArray64<FPackageData>& OutPackages) const
{
	struct PackageStackEntry
	{
		double StartTime;
		EPackageEventStatType Type;
		uint64 InPackageId;
		double ExclTime = 0.0f;
	};

	constexpr int MaxStackSize = 128;
	PackageStackEntry Stack[128];
	int32 Depth = 0;
	double LastTime = 0;

	OutPackages = Packages;

	for (auto& Entry : ScopeEntries)
	{
		TPagedArray<FPackageScope>& Entries = *Entry.Value;
		for (auto Iterator = Entries.begin(); Iterator != Entries.end(); ++Iterator)
		{
			const FPackageScope& CurrentScope = *Iterator.GetCurrentItem();
			if (CurrentScope.bIsEnterScope)
			{
				// Reset to default values.
				Stack[Depth] = PackageStackEntry();
				Stack[Depth].InPackageId = CurrentScope.PackageId;
				Stack[Depth].Type = CurrentScope.Type;
				Stack[Depth].StartTime = CurrentScope.Timestamp;

				if (Depth > 0)
				{
					Stack[Depth - 1].ExclTime += CurrentScope.Timestamp - LastTime;
				}

				verify(++Depth < MaxStackSize);
			}
			else
			{
				verify(--Depth >= 0);
				check(Stack[Depth].InPackageId == CurrentScope.PackageId);
				check(Stack[Depth].Type == CurrentScope.Type);

				Stack[Depth].ExclTime += CurrentScope.Timestamp - LastTime;
				double InclTime = CurrentScope.Timestamp - Stack[Depth].StartTime;

				uint32 PackageIndex;
				const uint32* Index = PackageIdToIndexMap.Find(Stack[Depth].InPackageId);
				if (Index != nullptr)
				{
					PackageIndex = *Index;
				}
				else
				{
					continue;
				}
				FPackageData& Package = OutPackages[PackageIndex];
				
				switch (Stack[Depth].Type)
				{
				case EPackageEventStatType::LoadPackage:
				{
					Package.LoadTimeIncl += InclTime;
					Package.LoadTimeExcl += Stack[Depth].ExclTime;
					break;
				}
				case EPackageEventStatType::SavePackage:
				{
					Package.SaveTimeIncl += InclTime;
					Package.SaveTimeExcl += Stack[Depth].ExclTime;
					break;
				}
				case EPackageEventStatType::BeginCacheForCookedPlatformData:
				{
					Package.BeginCacheForCookedPlatformDataIncl += InclTime;
					Package.BeginCacheForCookedPlatformDataExcl += Stack[Depth].ExclTime;
					break;
				}
				case EPackageEventStatType::IsCachedCookedPlatformDataLoaded:
				{
					Package.IsCachedCookedPlatformDataLoadedIncl += InclTime;
					Package.IsCachedCookedPlatformDataLoadedExcl += Stack[Depth].ExclTime;
					break;
				}
				}
			}

			LastTime = CurrentScope.Timestamp;
		}
	}
}

} // namespace TraceServices
