// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "Containers/Map.h"

#include "TraceServices/Model/CookProfilerProvider.h"

namespace TraceServices
{

extern thread_local FProviderLock::FThreadLocalState GCookProviderLockState;

struct FPackageScope
{
	FPackageScope()
	{}

	FPackageScope(uint64 InPackageId, double InTimestamp, EPackageEventStatType InType, bool InIsEnterScope)
		: PackageId(InPackageId)
		, Timestamp(InTimestamp)
		, Type(InType)
		, bIsEnterScope(InIsEnterScope)
	{}

	uint64 PackageId;
	double Timestamp;
	EPackageEventStatType Type;
	bool bIsEnterScope;
};

class FCookProfilerProvider
	: public ICookProfilerProvider
	, public IEditableCookProfilerProvider
{
public:
	explicit FCookProfilerProvider(IAnalysisSession& Session);
	virtual ~FCookProfilerProvider() = default;

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override       { Lock.BeginRead(GCookProviderLockState); }
	virtual void EndRead() const override         { Lock.EndRead(GCookProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GCookProviderLockState); }

	virtual uint32 GetNumPackages() const;
	virtual void EnumeratePackages(double StartTime, double EndTime, EnumeratePackagesCallback Callback) const override;
	virtual void CreateAggregation(TArray64<FPackageData>& OutPackages) const override;

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override       { Lock.BeginWrite(GCookProviderLockState); }
	virtual void EndEdit() const override         { Lock.EndWrite(GCookProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GCookProviderLockState); }

	virtual FPackageData* EditPackage(uint64 Id) override;

	virtual void AddScopeEntry(uint32 ThreadId, uint64 InPackageId, double Timestamp, EPackageEventStatType InType, bool InIsEnterScope);

	//////////////////////////////////////////////////

private:
	uint32 FindOrAddPackage(uint64 Id);
	TPagedArray<FPackageScope>& FindOrAddScopeEntries(uint32 ThreadId);

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;

	TMap<uint64, uint32> PackageIdToIndexMap;
	TArray64<FPackageData> Packages;

	TMap<uint32, TPagedArray<FPackageScope>*> ScopeEntries; // The Key is the ThreadId of the scope entries.
};

} // namespace TraceServices
