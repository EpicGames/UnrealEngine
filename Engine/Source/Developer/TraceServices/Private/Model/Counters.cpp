// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Counters.h"
#include "Model/CountersPrivate.h"

#include "AnalysisServicePrivate.h"

namespace TraceServices
{

FCounter::FCounter(ILinearAllocator& Allocator, const TArray64<double>& InFrameStartTimes)
	: FrameStartTimes(InFrameStartTimes)
	, IntCounterData(Allocator)
	, DoubleCounterData(Allocator)
{
}

void FCounter::SetIsFloatingPoint(bool bInIsFloatingPoint)
{
	bIsFloatingPoint = bInIsFloatingPoint;
}

void FCounter::SetIsResetEveryFrame(bool bInIsResetEveryFrame)
{
	bIsResetEveryFrame = bInIsResetEveryFrame;
}

template<typename CounterType, typename EnumerationType>
static void EnumerateCounterValuesInternal(const TCounterData<CounterType>& CounterData, const TArray64<double>& FrameStartTimes, bool bResetEveryFrame, double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, EnumerationType)> Callback)
{
	if (!CounterData.Num())
	{
		return;
	}

	TArray64<double> NoFrameStartTimes;
	auto CounterIterator = bResetEveryFrame ? CounterData.GetIterator(FrameStartTimes) : CounterData.GetIterator(NoFrameStartTimes);
	bool bFirstValue = true;
	bool bFirstEnumeratedValue = true;
	double LastTime = 0.0;
	CounterType LastValue = CounterType();
	while (CounterIterator)
	{
		const double Time = CounterIterator.GetCurrentTime();
		const CounterType CurrentValue = CounterIterator.GetCurrentValue();
		if (Time >= IntervalStart)
		{
			if (bFirstEnumeratedValue)
			{
				if (!bFirstValue && bIncludeExternalBounds)
				{
					Callback(LastTime, static_cast<EnumerationType>(LastValue));
				}
				bFirstEnumeratedValue = false;
			}
			if (Time > IntervalEnd)
			{
				if (bIncludeExternalBounds)
				{
					Callback(Time, static_cast<EnumerationType>(CurrentValue));
				}
				break;
			}
			Callback(Time, static_cast<EnumerationType>(CurrentValue));
		}
		LastTime = Time;
		LastValue = CurrentValue;
		bFirstValue = false;
		++CounterIterator;
	}
}

void FCounter::EnumerateValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, int64)> Callback) const
{
	if (bIsFloatingPoint)
	{
		EnumerateCounterValuesInternal<double, int64>(DoubleCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
	else
	{
		EnumerateCounterValuesInternal<int64, int64>(IntCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
}

void FCounter::EnumerateFloatValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, double)> Callback) const
{
	if (bIsFloatingPoint)
	{
		EnumerateCounterValuesInternal<double, double>(DoubleCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
	else
	{
		EnumerateCounterValuesInternal<int64, double>(IntCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
}

template<typename CounterType, typename EnumerationType>
static void EnumerateCounterOpsInternal(const TCounterData<CounterType>& CounterData, const TArray64<double>& FrameStartTimes, bool bResetEveryFrame, double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, EnumerationType)> Callback)
{
	if (!CounterData.Num())
	{
		return;
	}

	TArray64<double> NoFrameStartTimes;
	auto CounterIterator = bResetEveryFrame ? CounterData.GetIterator(FrameStartTimes) : CounterData.GetIterator(NoFrameStartTimes);
	bool bFirstValue = true;
	bool bFirstEnumeratedValue = true;
	double LastTime = 0.0;
	ECounterOpType LastOp = ECounterOpType::Set;
	CounterType LastOpArgument = CounterType();
	while (CounterIterator)
	{
		const double Time = CounterIterator.GetCurrentTime();
		const ECounterOpType CurrentOp = CounterIterator.GetCurrentOp();
		const CounterType CurrentOpArgument = CounterIterator.GetCurrentOpArgument();
		if (Time >= IntervalStart)
		{
			if (bFirstEnumeratedValue)
			{
				if (!bFirstValue && bIncludeExternalBounds)
				{
					Callback(LastTime, LastOp, static_cast<EnumerationType>(LastOpArgument));
				}
				bFirstEnumeratedValue = false;
			}
			if (Time > IntervalEnd)
			{
				if (bIncludeExternalBounds)
				{
					Callback(Time, CurrentOp, static_cast<EnumerationType>(CurrentOpArgument));
				}
				break;
			}
			Callback(Time, CurrentOp, static_cast<EnumerationType>(CurrentOpArgument));
		}
		LastTime = Time;
		LastOp = CurrentOp;
		LastOpArgument = CurrentOpArgument;
		bFirstValue = false;
		++CounterIterator;
	}
}

void FCounter::EnumerateOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, int64)> Callback) const
{
	if (bIsFloatingPoint)
	{
		EnumerateCounterOpsInternal<double, int64>(DoubleCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
	else
	{
		EnumerateCounterOpsInternal<int64, int64>(IntCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
}

void FCounter::EnumerateFloatOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, double)> Callback) const
{
	if (bIsFloatingPoint)
	{
		EnumerateCounterOpsInternal<double, double>(DoubleCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
	else
	{
		EnumerateCounterOpsInternal<int64, double>(IntCounterData, FrameStartTimes, bIsResetEveryFrame, IntervalStart, IntervalEnd, bIncludeExternalBounds, Callback);
	}
}

void FCounter::AddValue(double Time, int64 Value)
{
	if (bIsFloatingPoint)
	{
		DoubleCounterData.InsertOp(Time, ECounterOpType::Add, double(Value));
	}
	else
	{
		IntCounterData.InsertOp(Time, ECounterOpType::Add, Value);
	}
}

void FCounter::AddValue(double Time, double Value)
{
	if (bIsFloatingPoint)
	{
		DoubleCounterData.InsertOp(Time, ECounterOpType::Add, Value);
	}
	else
	{
		IntCounterData.InsertOp(Time, ECounterOpType::Add, int64(Value));
	}
}

void FCounter::SetValue(double Time, int64 Value)
{
	if (bIsFloatingPoint)
	{
		DoubleCounterData.InsertOp(Time, ECounterOpType::Set, double(Value));
	}
	else
	{
		IntCounterData.InsertOp(Time, ECounterOpType::Set, Value);
	}
}

void FCounter::SetValue(double Time, double Value)
{
	if (bIsFloatingPoint)
	{
		DoubleCounterData.InsertOp(Time, ECounterOpType::Set, Value);
	}
	else
	{
		IntCounterData.InsertOp(Time, ECounterOpType::Set, int64(Value));
	}
}

FCounterProvider::FCounterProvider(IAnalysisSession& InSession, IFrameProvider& InFrameProvider)
	: Session(InSession)
	, FrameProvider(InFrameProvider)
{
}

FCounterProvider::~FCounterProvider()
{
	for (const ICounter* Counter : Counters)
	{
		delete Counter;
	}
}

void FCounterProvider::EnumerateCounters(TFunctionRef<void(uint32, const ICounter&)> Callback) const
{
	uint32 Id = 0;
	for (const ICounter* Counter : Counters)
	{
		Callback(Id++, *Counter);
	}
}

bool FCounterProvider::ReadCounter(uint32 CounterId, TFunctionRef<void(const ICounter &)> Callback) const
{
	if (CounterId >= uint32(Counters.Num()))
	{
		return false;
	}
	Callback(*Counters[CounterId]);
	return true;
}

const ICounter* FCounterProvider::GetCounter(IEditableCounter* EditableCounter)
{
	return static_cast<FCounter*>(EditableCounter);
}

IEditableCounter* FCounterProvider::CreateEditableCounter()
{
	FCounter* Counter = new FCounter(Session.GetLinearAllocator(), FrameProvider.GetFrameStartTimes(TraceFrameType_Game));
	Counters.Add(Counter);
	return Counter;
}

void FCounterProvider::AddCounter(const ICounter* Counter)
{
	Counters.Add(Counter);
}

FName GetCounterProviderName()
{
	static const FName Name("CounterProvider");
	return Name;
}

const ICounterProvider& ReadCounterProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<ICounterProvider>(GetCounterProviderName());
}

IEditableCounterProvider& EditCounterProvider(IAnalysisSession& Session)
{
	return *Session.EditProvider<IEditableCounterProvider>(GetCounterProviderName());
}

} // namespace TraceServices
