// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Model/Frames.h"

// We need to test data pattern and measure performance for more trace sessions
// before completely switching to variable paged array.
#define USE_VARIABLE_PAGED_ARRAY 1

#if USE_VARIABLE_PAGED_ARRAY
#include "Common/VariablePagedArray.h"
#else
#include "Common/PagedArray.h"
#endif

namespace TraceServices
{

template<typename ValueType>
class TCounterData;

template<typename ValueType>
class TCounterDataIterator
{
public:
	TCounterDataIterator(const TCounterData<ValueType>& Outer, const TArray64<double>& FrameStartTimes)
		: FrameStartTimesIterator(FrameStartTimes.CreateConstIterator())
		, TimestampsIterator(Outer.Timestamps.GetIterator())
		, OpTypesIterator(Outer.OpTypes.GetIterator())
		, OpArgumentsIterator(Outer.OpArguments.GetIterator())
		, CurrentTime(0.0)
		, CurrentOp(ECounterOpType::Set)
		, CurrentOpArgument(ValueType())
		, CurrentValue(ValueType())
	{
		UpdateValue();
	}

	const ValueType operator*() const
	{
		return CurrentValue;
	}

	const double GetCurrentTime() const
	{
		return CurrentTime;
	}

	const ECounterOpType GetCurrentOp() const
	{
		return CurrentOp;
	}

	const ValueType GetCurrentOpArgument() const
	{
		return CurrentOpArgument;
	}

	const ValueType GetCurrentValue() const
	{
		return CurrentValue;
	}

	explicit operator bool() const
	{
		return bool(TimestampsIterator);
	}

	TCounterDataIterator& operator++()
	{
		++TimestampsIterator;
		++OpTypesIterator;
		++OpArgumentsIterator;
		UpdateValue();
		return *this;
	}

	TCounterDataIterator operator++(int)
	{
		TCounterDataIterator Tmp(*this);
		this->operator++();
		return Tmp;
	}

private:
	void UpdateValue()
	{
		const double* Time = TimestampsIterator.GetCurrentItem();
		if (Time)
		{
			bool bIsNewFrame = false;
			while (FrameStartTimesIterator && *FrameStartTimesIterator < *Time)
			{
				bIsNewFrame = true;
				++FrameStartTimesIterator;
			}
			CurrentTime = *Time;
			CurrentOp = bIsNewFrame ? ECounterOpType::Set : *OpTypesIterator;
			CurrentOpArgument = *OpArgumentsIterator;
			switch (CurrentOp)
			{
			case ECounterOpType::Set:
				CurrentValue = CurrentOpArgument;
				break;
			case ECounterOpType::Add:
				CurrentValue += CurrentOpArgument;
				break;
			}
		}
	}

	TArray64<double>::TConstIterator FrameStartTimesIterator;
#if USE_VARIABLE_PAGED_ARRAY
	TVariablePagedArray<double>::TIterator TimestampsIterator;
	TVariablePagedArray<ECounterOpType>::TIterator OpTypesIterator;
	typename TVariablePagedArray<ValueType>::TIterator OpArgumentsIterator;
#else
	TPagedArray<double>::TIterator TimestampsIterator;
	TPagedArray<ECounterOpType>::TIterator OpTypesIterator;
	typename TPagedArray<ValueType>::TIterator OpArgumentsIterator;
#endif
	double CurrentTime;
	ECounterOpType CurrentOp;
	ValueType CurrentOpArgument;
	ValueType CurrentValue;
};

template<typename ValueType>
class TCounterData
{
public:
	typedef TCounterDataIterator<ValueType> TIterator;

	TCounterData(ILinearAllocator& Allocator)
		: Timestamps(Allocator, 1024)
		, OpTypes(Allocator, 1024)
		, OpArguments(Allocator, 1024)
	{

	}

	void InsertOp(double Timestamp, ECounterOpType OpType, ValueType OpArgument)
	{
		uint64 InsertionIndex;
		if (Timestamps.Num() == 0 || Timestamps.Last() <= Timestamp)
		{
			InsertionIndex = Timestamps.Num();
		}
		else if (Timestamps.First() >= Timestamp)
		{
			InsertionIndex = 0;
		}
		else
		{
			auto TimestampIterator = Timestamps.GetIteratorFromItem(Timestamps.Num() - 1);
			auto CurrentPage = TimestampIterator.GetCurrentPage();
			while (*GetFirstItem(*CurrentPage) > Timestamp)
			{
				CurrentPage = TimestampIterator.PrevPage();
			}
			check(CurrentPage != nullptr);
			uint64 PageInsertionIndex = Algo::LowerBound(*CurrentPage, Timestamp);
#if USE_VARIABLE_PAGED_ARRAY
			InsertionIndex = TimestampIterator.GetCurrentItemIndex() + PageInsertionIndex + 1 - CurrentPage->ItemCount;
#else
			InsertionIndex = TimestampIterator.GetCurrentPageIndex() * Timestamps.GetPageSize() + PageInsertionIndex;
#endif
			check(InsertionIndex <= Timestamps.Num());
		}
		Timestamps.Insert(InsertionIndex) = Timestamp;
		OpTypes.Insert(InsertionIndex) = OpType;
		OpArguments.Insert(InsertionIndex) = OpArgument;
	}

	uint64 Num() const
	{
		return Timestamps.Num();
	}

	TIterator GetIterator(const TArray64<double>& FrameStartTimes) const
	{
		return TIterator(*this, FrameStartTimes);
	}

	double GetFirstTimestamp() const { return Timestamps.First(); }
	double GetLastTimestamp() const { return Timestamps.Last(); }

private:
	template<typename IteratorValueType>
	friend class TCounterDataIterator;

#if USE_VARIABLE_PAGED_ARRAY
	TVariablePagedArray<double> Timestamps;
	TVariablePagedArray<ECounterOpType> OpTypes;
	TVariablePagedArray<ValueType> OpArguments;
#else
	TPagedArray<double> Timestamps;
	TPagedArray<ECounterOpType> OpTypes;
	TPagedArray<ValueType> OpArguments;
#endif
};

class FCounter
	: public ICounter
	, public IEditableCounter
{
public:
	FCounter(ILinearAllocator& Allocator, const TArray64<double>& FrameStartTimes);

	// ICounter
	virtual const TCHAR* GetName() const override { return Name; }
	virtual const TCHAR* GetGroup() const override { return Group; }
	virtual const TCHAR* GetDescription() const override { return Description; }
	virtual bool IsFloatingPoint() const override { return bIsFloatingPoint; }
	virtual bool IsResetEveryFrame() const override { return bIsResetEveryFrame; }
	virtual ECounterDisplayHint GetDisplayHint() const { return DisplayHint; }
	virtual void EnumerateValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, int64)> Callback) const override;
	virtual void EnumerateFloatValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, double)> Callback) const override;
	virtual void EnumerateOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, int64)> Callback) const override;
	virtual void EnumerateFloatOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, double)> Callback) const override;

	// IEditableCounter
	virtual void SetName(const TCHAR* InName) override { Name = InName; }
	virtual void SetGroup(const TCHAR* InGroup) override { Group = InGroup; }
	virtual void SetDescription(const TCHAR* InDescription) override { Description = InDescription; }
	virtual void SetIsFloatingPoint(bool bInIsFloatingPoint) override;
	virtual void SetIsResetEveryFrame(bool bInIsResetEveryFrame) override;
	virtual void SetDisplayHint(ECounterDisplayHint InDisplayHint) override { DisplayHint = InDisplayHint; }
	virtual void AddValue(double Time, int64 Value) override;
	virtual void AddValue(double Time, double Value) override;
	virtual void SetValue(double Time, int64 Value) override;
	virtual void SetValue(double Time, double Value) override;

private:
	const TArray64<double>& FrameStartTimes;
	TCounterData<int64> IntCounterData;
	TCounterData<double> DoubleCounterData;
	const TCHAR* Name = nullptr;
	const TCHAR* Group = nullptr;
	const TCHAR* Description = nullptr;
	ECounterDisplayHint DisplayHint = CounterDisplayHint_None;
	bool bIsFloatingPoint = false;
	bool bIsResetEveryFrame = false;
};

class FCounterProvider
	: public ICounterProvider
	, public IEditableCounterProvider
{
public:
	explicit FCounterProvider(IAnalysisSession& Session, IFrameProvider& FrameProvider);
	virtual ~FCounterProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual uint64 GetCounterCount() const override { return Counters.Num(); }
	virtual void EnumerateCounters(TFunctionRef<void(uint32, const ICounter&)> Callback) const override;
	virtual bool ReadCounter(uint32 CounterId, TFunctionRef<void(const ICounter&)> Callback) const override;

	//////////////////////////////////////////////////
	// Edit operations

	virtual const ICounter* GetCounter(IEditableCounter* EditableCounter) override;
	virtual IEditableCounter* CreateEditableCounter() override;
	virtual void AddCounter(const ICounter* Counter) override;

	//////////////////////////////////////////////////

private:
	IAnalysisSession& Session;
	IFrameProvider& FrameProvider;
	TArray<const ICounter*> Counters;
};

} // namespace TraceServices
