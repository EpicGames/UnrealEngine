// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/CsvProfilerProvider.h"
#include "CsvProfilerProviderPrivate.h"

namespace TraceServices
{

uint64 FCsvProfilerProvider::FTableLayout::GetColumnCount() const
{
	return StatSeries.Num() + 1;
}

const TCHAR* FCsvProfilerProvider::FTableLayout::GetColumnName(uint64 ColumnIndex) const
{
	if (ColumnIndex == 0)
	{
		return TEXT("EVENTS");
	}
	else
	{
		return StatSeries[static_cast<int32>(ColumnIndex - 1)]->Name;
	}
}

ETableColumnType FCsvProfilerProvider::FTableLayout::GetColumnType(uint64 ColumnIndex) const
{
	if (ColumnIndex == 0)
	{
		return TableColumnType_CString;
	}
	else
	{
		ECsvStatSeriesType SeriesType = StatSeries[static_cast<int32>(ColumnIndex - 1)]->Type;
		if (SeriesType == CsvStatSeriesType_CustomStatInt)
		{
			return TableColumnType_Int;
		}
		else
		{
			return TableColumnType_Double;
		}
	}
}

bool FCsvProfilerProvider::FTableReader::IsValid() const
{
	return CurrentRowIndex < (Capture.EndFrame - Capture.StartFrame);
}

void FCsvProfilerProvider::FTableReader::NextRow()
{
	++CurrentRowIndex;
}

void FCsvProfilerProvider::FTableReader::SetRowIndex(uint64 RowIndex)
{
	CurrentRowIndex = IntCastChecked<uint32>(RowIndex);
}

bool FCsvProfilerProvider::FTableReader::GetValueBool(uint64 ColumnIndex) const
{
	return false;
}

const FCsvProfilerProvider::FStatSeriesValue* FCsvProfilerProvider::FTableReader::GetValue(uint64 ColumnIndex) const
{
	if (!IsValid() || ColumnIndex == 0 || Capture.StatSeries.Num() <= ColumnIndex - 1)
	{
		return nullptr;
	}
	const FStatSeriesValue* FindIt = Capture.StatSeries[static_cast<int32>(ColumnIndex - 1)]->Values.Find(Capture.StartFrame + CurrentRowIndex);
	return FindIt;
}

int64 FCsvProfilerProvider::FTableReader::GetValueInt(uint64 ColumnIndex) const
{
	const FStatSeriesValue* Value = GetValue(ColumnIndex);
	if (!Value)
	{
		return 0;
	}
	return Value->Value.AsInt;
}

float FCsvProfilerProvider::FTableReader::GetValueFloat(uint64 ColumnIndex) const
{
	return static_cast<float>(GetValueDouble(ColumnIndex));
}

double FCsvProfilerProvider::FTableReader::GetValueDouble(uint64 ColumnIndex) const
{
	const FStatSeriesValue* Value = GetValue(ColumnIndex);
	if (!Value)
	{
		return 0;
	}
	return Value->Value.AsDouble;
}

const TCHAR* FCsvProfilerProvider::FTableReader::GetValueCString(uint64 ColumnIndex) const
{
	if (ColumnIndex != 0)
	{
		return nullptr;
	}
	const FEvents* const* FindIt = Events.Find(Capture.StartFrame + CurrentRowIndex);
	if (!FindIt)
	{
		return TEXT("");
	}
	return (*FindIt)->SemiColonSeparatedEvents;
}

uint64 FCsvProfilerProvider::FTable::GetRowCount() const
{
	return Capture.EndFrame - Capture.StartFrame;
}

IUntypedTableReader* FCsvProfilerProvider::FTable::CreateReader() const
{
	return new FCsvProfilerProvider::FTableReader(Capture, Events);
}

FCsvProfilerProvider::FCsvProfilerProvider(IAnalysisSession& InSession)
	: Session(InSession)
{

}

FCsvProfilerProvider::~FCsvProfilerProvider()
{
	for (FCapture* Capture : Captures)
	{
		delete Capture;
	}
	delete CurrentCapture;
	for (FStatSeries* Series : StatSeries)
	{
		delete Series;
	}
}

void FCsvProfilerProvider::EnumerateCaptures(TFunctionRef<void(const FCaptureInfo&)> Callback) const
{
	Session.ReadAccessCheck();

	uint32 Index = 0;
	for (const FCapture* Capture : Captures)
	{
		Callback({ Capture->Metadata, Capture->Filename, Index++, uint32(Capture->EndFrame - Capture->StartFrame) });
	}
}

void FCsvProfilerProvider::StartCapture(const TCHAR* Filename, uint32 FrameNumber)
{
	Session.WriteAccessCheck();

	check(!CurrentCapture);
	CurrentCapture = new FCapture(Events);
	CurrentCapture->StartFrame = FrameNumber;
	CurrentCapture->EndFrame = FrameNumber;
	CurrentCapture->Filename = Filename;
}

void FCsvProfilerProvider::EndCapture(uint32 FrameNumber)
{
	Session.WriteAccessCheck();

	check(CurrentCapture);

	CurrentCapture->EndFrame = FrameNumber;
	check(CurrentCapture->EndFrame >= CurrentCapture->StartFrame);

	CurrentCapture->Metadata = Metadata;

	Captures.Add(CurrentCapture);
	CurrentCapture = nullptr;
}

uint64 FCsvProfilerProvider::AddSeries(const TCHAR* Name, ECsvStatSeriesType Type)
{
	Session.WriteAccessCheck();

	FStatSeries* Series = new FStatSeries();
	Series->Name = Name;
	Series->Type = Type;
	int32 ColumnIndex = StatSeries.Num();
	StatSeries.Add(Series);
	return ColumnIndex;
}

FCsvProfilerProvider::FStatSeriesValue& FCsvProfilerProvider::GetValueRef(uint64 SeriesHandle, uint32 FrameNumber)
{
	FStatSeries* Series = StatSeries[static_cast<int32>(SeriesHandle)];
	if (CurrentCapture && !Series->Captures.Contains(Captures.Num()))
	{
		CurrentCapture->StatSeries.Add(Series);
		Series->Captures.Add(Captures.Num());
	}
	return Series->Values.FindOrAdd(FrameNumber);
}

void FCsvProfilerProvider::SetValue(uint64 SeriesHandle, uint32 FrameNumber, int64 Value)
{
	Session.WriteAccessCheck();

	FStatSeriesValue& RowValue = GetValueRef(SeriesHandle, FrameNumber);
	RowValue.Value.AsInt = Value;
}

void FCsvProfilerProvider::SetValue(uint64 SeriesHandle, uint32 FrameNumber, double Value)
{
	Session.WriteAccessCheck();

	FStatSeriesValue& RowValue = GetValueRef(SeriesHandle, FrameNumber);
	RowValue.Value.AsDouble = Value;
}

void FCsvProfilerProvider::AddEvent(uint32 FrameNumber, const TCHAR* Text)
{
	Session.WriteAccessCheck();

	FEvents* FrameEvents;
	FEvents** FindIt = Events.Find(FrameNumber);
	if (!FindIt)
	{
		FrameEvents = new FEvents();
		Events.Add(FrameNumber, FrameEvents);
	}
	else
	{
		FrameEvents = *FindIt;
	}
	if (FrameEvents->Events.Num())
	{
		FrameEvents->SemiColonSeparatedEvents = Session.StoreString(*(FString(FrameEvents->SemiColonSeparatedEvents) + ";" + Text));
	}
	else
	{
		FrameEvents->SemiColonSeparatedEvents = Text;
	}
	FrameEvents->Events.Add(Text);
}

void FCsvProfilerProvider::SetMetadata(const TCHAR* Key, const TCHAR* Value)
{
	Session.WriteAccessCheck();

	Metadata.Add(Key, Value);
}

} // namespace TraceServices
