// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/CsvProfilerProvider.h"
#include "TraceServices/Containers/Tables.h"
#include "Containers/Map.h"

namespace TraceServices
{

enum ECsvStatSeriesType
{
	CsvStatSeriesType_Timer,
	CsvStatSeriesType_CustomStatInt,
	CsvStatSeriesType_CustomStatFloat
};

class FCsvProfilerProvider
	: public ICsvProfilerProvider
{
public:
	explicit FCsvProfilerProvider(IAnalysisSession& InSession);
	virtual ~FCsvProfilerProvider();
	virtual const IUntypedTable& GetTable(uint32 CaptureIndex) const override { return Captures[CaptureIndex]->Table; }
	virtual void EnumerateCaptures(TFunctionRef<void(const FCaptureInfo&)> Callback) const override;
	void StartCapture(const TCHAR* Filename, uint32 FrameNumber);
	void EndCapture(uint32 FrameNumber);
	uint64 AddSeries(const TCHAR* Name, ECsvStatSeriesType Type);
	void SetValue(uint64 SeriesHandle, uint32 FrameNumber, double Value);
	void SetValue(uint64 SeriesHandle, uint32 FrameNumber, int64 Value);
	void AddEvent(uint32 FrameNumber, const TCHAR* Text);
	void SetMetadata(const TCHAR* Key, const TCHAR* Value);

private:
	struct FStatSeriesValue
	{
		FStatSeriesValue() { Value.AsInt = 0; }
		union
		{
			int64 AsInt;
			double AsDouble;
		} Value;
	};

	struct FStatSeries
	{
		TMap<uint32, FStatSeriesValue> Values;
		TSet<uint32> Captures;
		const TCHAR* Name = nullptr;
		ECsvStatSeriesType Type;
	};

	struct FEvents
	{
		TArray<const TCHAR*> Events;
		const TCHAR* SemiColonSeparatedEvents = nullptr;
	};

	struct FCapture;

	class FTableLayout
		: public ITableLayout
	{
	public:
		FTableLayout(const TArray<FStatSeries*>& InStatSeries)
			: StatSeries(InStatSeries)
		{

		}

		virtual uint64 GetColumnCount() const override;
		virtual const TCHAR* GetColumnName(uint64 ColumnIndex) const override;
		virtual ETableColumnType GetColumnType(uint64 ColumnIndex) const override;
		virtual uint32 GetColumnDisplayHintFlags(uint64 ColumnIndex) const override
		{
			return 0;
		}

	private:
		const TArray<FStatSeries*>& StatSeries;
	};

	class FTableReader
		: public IUntypedTableReader
	{
	public:
		FTableReader(const FCapture& InCapture, const TMap<uint32, FEvents*>& InEvents)
			: Capture(InCapture)
			, Events(InEvents)
			, CurrentRowIndex(0)
		{
		}

		virtual bool IsValid() const override;
		virtual void NextRow() override;
		virtual void SetRowIndex(uint64 RowIndex) override;
		virtual bool GetValueBool(uint64 ColumnIndex) const override;
		virtual int64 GetValueInt(uint64 ColumnIndex) const override;
		virtual float GetValueFloat(uint64 ColumnIndex) const override;
		virtual double GetValueDouble(uint64 ColumnIndex) const override;
		virtual const TCHAR* GetValueCString(uint64 ColumnIndex) const override;

	private:
		const FStatSeriesValue* GetValue(uint64 ColumnIndex) const;

		const FCapture& Capture;
		const TMap<uint32, FEvents*>& Events;
		uint32 CurrentRowIndex;
	};

	class FTable
		: public IUntypedTable
	{
	public:
		FTable(const FCapture& InCapture, const TMap<uint32, FEvents*>& InEvents)
			: Layout(InCapture.StatSeries)
			, Capture(InCapture)
			, Events(InEvents)
		{

		}

		virtual const ITableLayout& GetLayout() const override { return Layout; }
		virtual uint64 GetRowCount() const override;
		virtual IUntypedTableReader* CreateReader() const override;

	private:
		FTableLayout Layout;
		const FCapture& Capture;
		const TMap<uint32, FEvents*>& Events;
	};

	struct FCapture
	{
		FCapture(const TMap<uint32, FEvents*>& Events)
			: Table(*this, Events)
		{
			
		}

		FTable Table;
		const TCHAR* Filename;
		uint32 StartFrame = 0;
		uint32 EndFrame = 0;
		TArray<FStatSeries*> StatSeries;
		TMap<const TCHAR*, const TCHAR*, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, const TCHAR*>> Metadata;
	};

	FStatSeriesValue& GetValueRef(uint64 SeriesHandle, uint32 FrameNumber);

	IAnalysisSession& Session;
	TArray<FStatSeries*> StatSeries;
	TMap<uint32, FEvents*> Events;
	TMap<const TCHAR*, const TCHAR*, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, const TCHAR*>> Metadata;
	TArray<FCapture*> Captures;
	FCapture* CurrentCapture = nullptr;
};

} // namespace TraceServices
