// Copyright Epic Games, Inc. All Rights Reserved.

#include "DbgHelpResolver.h"

#if PLATFORM_WINDOWS

#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Containers/Queue.h"
#include "Containers/StringView.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"

// TraceServices
#include "Common/SymbolHelper.h"
#include "TraceServices/Model/AnalysisSession.h"

#include <atomic>

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <DbgHelp.h>
THIRD_PARTY_INCLUDES_END

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(LogDbgHelp);

namespace TraceServices {

static const TCHAR* GUnknownModuleTextDbgHelp = TEXT("Unknown");

////////////////////////////////////////////////////////////////////////////////////////////////////

FDbgHelpResolver::FDbgHelpResolver(IAnalysisSession& InSession, IResolvedSymbolFilter& InSymbolFilter)
	: Modules(InSession.GetLinearAllocator(), 128)
	, Session(InSession)
	, SymbolFilter(InSymbolFilter)
	, bRunWorkerThread(false)
	, bDrainThenStop(false)
{
	// Setup search paths. Paths are searched in the following order:
	// 1. Any new path entered by the user this session
	// 2. Path of the executable (if available)
	// 3. Paths from UE_INSIGHTS_SYMBOL_PATH environment variable
	// 4. Paths from _NT_SYMBOL_PATH environment variable
	// 5. Paths from the user configuration file

	// Get the pre-configured symbol search paths (environment variables + user configuration file).
	FSymbolHelper::GetSymbolSearchPaths(LogDbgHelp, ConfigSymbolSearchPaths);

	Start();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FDbgHelpResolver::~FDbgHelpResolver()
{
	bRunWorkerThread = false;
	if (Thread)
	{
		Thread->WaitForCompletion();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::QueueModuleLoad(const uint8* ImageId, uint32 ImageIdSize, FModule* Module)
{
	check(Module != nullptr);
	ensure(GetModuleEntry(Module) == nullptr);

	FWriteScopeLock _(ModulesLock);

	// Add the new module entry.
	FModuleEntry* Entry = &Modules.PushBack();
	Entry->Module = Module;
	Entry->ImageId = TArrayView<const uint8>(ImageId, ImageIdSize);

	// Sort list according to base address.
	SortedModules.Add(Entry);
	Algo::Sort(SortedModules, [](const FModuleEntry* Lhs, const FModuleEntry* Rhs) { return Lhs->Module->Base < Rhs->Module->Base; });

	++ModulesDiscovered;

	// Set the Pending state before scheduling the background task (to allow calling code to wait, if needed).
	Module->Status.store(EModuleStatus::Pending);

	// Queue up module to have symbols loaded.
	LoadSymbolsQueue.Enqueue(FQueuedModule{ Module, nullptr, Entry->ImageId });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::QueueModuleReload(FModule* Module, const TCHAR* Path, TFunction<void(SymbolArray&)> ResolveOnSuccess)
{
	check(Module != nullptr);

	// Find the entry
	FModuleEntry* Entry = GetModuleEntry(Module);
	if (!Entry)
	{
		return;
	}

	// No use in trying reload already loaded modules
	if (Module->Status == EModuleStatus::Loaded)
	{
		return;
	}

	// Set the Pending state before scheduling the background task (to allow calling code to wait, if needed).
	EModuleStatus PreviousStatus = Module->Status.exchange(EModuleStatus::Pending);
	if (PreviousStatus >= EModuleStatus::FailedStatusStart)
	{
		--ModulesFailed;
	}

	FString PathStr(Path);
	FPaths::NormalizeDirectoryName(PathStr);
	const TCHAR* OverrideSearchPath = Session.StoreString(PathStr);

	LoadSymbolsQueue.Enqueue(FQueuedModule{ Module, OverrideSearchPath, TArrayView<const uint8>(Entry->ImageId) });

	SymbolArray SymbolsToResolve;
	ResolveOnSuccess(SymbolsToResolve);
	for(TTuple<uint64, FResolvedSymbol*> Pair : SymbolsToResolve)
	{
		QueueSymbolResolve(Pair.Get<0>(), Pair.Get<1>());
	}

	if (!bRunWorkerThread && Thread)
	{
		// Restart the worker thread if it has stopped.
		Start();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::QueueSymbolResolve(uint64 Address, FResolvedSymbol* Symbol)
{
	ResolveQueue.Enqueue(FQueuedAddress{ Address, Symbol });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::GetStats(IModuleProvider::FStats* OutStats) const
{
	FReadScopeLock _(ModulesLock);
	FMemory::Memzero(*OutStats);
	for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
	{
		const FModule::SymbolStats& ModuleStats = Modules[ModuleIndex].Module->Stats;
		OutStats->SymbolsDiscovered += ModuleStats.Discovered.load();
		OutStats->SymbolsCached += ModuleStats.Cached.load();
		OutStats->SymbolsResolved += ModuleStats.Resolved.load();
		OutStats->SymbolsFailed += ModuleStats.Failed.load();
	}
	OutStats->SymbolsFailed += NoModuleSymbols;
	OutStats->ModulesDiscovered = ModulesDiscovered.load();
	OutStats->ModulesFailed = ModulesFailed.load();
	OutStats->ModulesLoaded = ModulesLoaded.load();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FDbgHelpResolver::HasFinishedResolving() const
{
	return LoadSymbolsQueue.IsEmpty() && ResolveQueue.IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const
{
	{
		FReadScopeLock _(CustomSymbolSearchPathsLock);
		Algo::ForEach(CustomSymbolSearchPaths, Callback);
	}

	Algo::ForEach(ConfigSymbolSearchPaths, Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::OnAnalysisComplete()
{
	// At this point no more module loads or symbol requests will be queued,
	// we drain the current queue, then release resources and file locks.
	bDrainThenStop = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FDbgHelpResolver::SetupSyms()
{
	// Create a unique handle
	static UPTRINT BaseHandle = 0x493;
	Handle = ++BaseHandle;

	// Load DbgHelp interface
	ULONG SymOpts = 0;
	SymOpts |= SYMOPT_LOAD_LINES;
	SymOpts |= SYMOPT_OMAP_FIND_NEAREST;
	//SymOpts |= SYMOPT_DEFERRED_LOADS;
	SymOpts |= SYMOPT_EXACT_SYMBOLS;
	SymOpts |= SYMOPT_IGNORE_NT_SYMPATH;
	SymOpts |= SYMOPT_UNDNAME;

	SymSetOptions(SymOpts);

	return SymInitialize((HANDLE)Handle, NULL, FALSE);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::FreeSyms() const
{
	// This release file locks on debug files
	SymCleanup((HANDLE)Handle);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FDbgHelpResolver::Run()
{
	const bool bInitialized = SetupSyms();

	while (bInitialized && bRunWorkerThread)
	{
		// Prioritize queued module loads
		while (!LoadSymbolsQueue.IsEmpty() && bRunWorkerThread)
		{
			FQueuedModule Item;
			if (LoadSymbolsQueue.Dequeue(Item))
			{
				LoadModuleSymbols(Item.Module, Item.Path, Item.ImageId);
			}
		}

		// Resolve one symbol at a time to give way for modules
		while (!ResolveQueue.IsEmpty() && LoadSymbolsQueue.IsEmpty() && bRunWorkerThread)
		{
			FQueuedAddress Item;
			if (ResolveQueue.Dequeue(Item))
			{
				ResolveSymbol(Item.Address, *Item.Target);
			}
		}

		if (bDrainThenStop && ResolveQueue.IsEmpty() && LoadSymbolsQueue.IsEmpty())
		{
			bRunWorkerThread = false;
		}

		// ...and breathe...
		FPlatformProcess::Sleep(0.2f);
	}

	// We don't need the syms library anymore
	FreeSyms();

	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::Start()
{
	// Start the worker thread
	bRunWorkerThread = true;
	Thread = FRunnableThread::Create(this, TEXT("DbgHelpWorker"), 0, TPri_Normal);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::Stop()
{
	bRunWorkerThread = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FDbgHelpResolver::FModuleEntry* FDbgHelpResolver::GetModuleEntry(FModule* Module) const
{
	FReadScopeLock _(ModulesLock);
	for (FModuleEntry* Entry : SortedModules)
	{
		if (Entry->Module == Module)
		{
			return Entry;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FDbgHelpResolver::FModuleEntry* FDbgHelpResolver::GetModuleForAddress(uint64 Address) const
{
	FReadScopeLock _(ModulesLock);
	const int32 EntryIdx = Algo::LowerBoundBy(SortedModules, Address, [](const FModuleEntry* Entry) { return Entry->Module->Base; }) - 1;
	if (EntryIdx < 0 || EntryIdx >= SortedModules.Num())
	{
		return nullptr;
	}
	return SortedModules[EntryIdx];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::UpdateResolvedSymbol(FResolvedSymbol& Symbol, ESymbolQueryResult Result, const TCHAR* Module, const TCHAR* Name, const TCHAR* File, uint16 Line)
{
	Symbol.Module = Module;
	Symbol.Name = Name;
	Symbol.File = File;
	Symbol.Line = Line;
	Symbol.Result.store(Result, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::ResolveSymbol(uint64 Address, FResolvedSymbol& Target)
{
	if (Target.Result.load() == ESymbolQueryResult::OK)
	{
		++AlreadyResolvedSymbols;
		return;
	}

	FModuleEntry* Entry = GetModuleForAddress(Address);
	if (!Entry)
	{
		UE_LOGF(LogDbgHelp, Warning, "No module mapped to address 0x%llX.", Address);
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotFound,
			GUnknownModuleTextDbgHelp,
			GUnknownModuleTextDbgHelp,
			GUnknownModuleTextDbgHelp,
			0);
		SymbolFilter.Update(Target);
		++NoModuleSymbols;
		return;
	}
	FModule* Module = Entry->Module;

	++Module->Stats.Discovered;

	const EModuleStatus ModuleStatus = Module->Status.load();
	if (ModuleStatus != EModuleStatus::Loaded)
	{
		++Module->Stats.Failed;
		UpdateResolvedSymbol(Target,
			ModuleStatus == EModuleStatus::VersionMismatch ? ESymbolQueryResult::Mismatch : ESymbolQueryResult::NotLoaded,
			GUnknownModuleTextDbgHelp,
			GUnknownModuleTextDbgHelp,
			GUnknownModuleTextDbgHelp,
			0);
		SymbolFilter.Update(Target);
		return;
	}

	uint8 InfoBuffer[sizeof(SYMBOL_INFOW) + (MaxNameLen + 1) * sizeof(TCHAR)];
	SYMBOL_INFOW* Info = (SYMBOL_INFOW*)InfoBuffer;
	Info->SizeOfStruct = sizeof(SYMBOL_INFOW);
	Info->MaxNameLen = MaxNameLen;

	// Find and build the symbol name
	if (!SymFromAddrW((HANDLE)Handle, Address, NULL, Info))
	{
		++Module->Stats.Failed;
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotFound,
			Module->Name,
			GUnknownModuleTextDbgHelp,
			GUnknownModuleTextDbgHelp,
			0);
		SymbolFilter.Update(Target);
		return;
	}

	const TCHAR* SymbolNameStr = Session.StoreString(Info->Name);

	// Find the source file and line
	DWORD  dwDisplacement;
	IMAGEHLP_LINEW64 Line;
	Line.SizeOfStruct = sizeof(IMAGEHLP_LINEW64);

	if (!SymGetLineFromAddrW64((HANDLE)Handle, Address, &dwDisplacement, &Line))
	{
		++Module->Stats.Failed;
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::OK,
			Module->Name,
			SymbolNameStr,
			GUnknownModuleTextDbgHelp,
			0);
		SymbolFilter.Update(Target);
		return;
	}

	const TCHAR* SymbolFileStr = Session.StoreString(Line.FileName);

	++Module->Stats.Resolved;
	UpdateResolvedSymbol(Target,
		ESymbolQueryResult::OK,
		Module->Name,
		SymbolNameStr,
		SymbolFileStr,
		static_cast<uint16>(Line.LineNumber));
	SymbolFilter.Update(Target);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDbgHelpResolver::LoadModuleSymbols(FModule* Module, const TCHAR* OverrideSearchPath, const TArrayView<const uint8> ImageId)
{
	check(Module);

	const uint64 Base = Module->Base;
	const uint32 Size = Module->Size;

	// Setup symbol search path
	{
		TStringBuilder<1024> UserSearchPath;

		if (OverrideSearchPath && *OverrideSearchPath != TEXT('\0'))
		{
			UserSearchPath.Append(OverrideSearchPath);
			UserSearchPath.AppendChar(TEXT(';'));
		}
		else
		{
			// 1. Any new path entered by the user this session
			{
				FReadScopeLock _(CustomSymbolSearchPathsLock);
				Algo::ForEach(CustomSymbolSearchPaths,
					[&UserSearchPath](const FString& Path)
					{
						if (!Path.IsEmpty())
						{
							UserSearchPath.Append(Path);
							UserSearchPath.AppendChar(TEXT(';'));
						}
					});
			}

			// 2. Path of the executable (if available)
			FString ModuleNamePath = FPaths::GetPath(Module->FullName);
			FPaths::NormalizeDirectoryName(ModuleNamePath);
			if (!ModuleNamePath.IsEmpty())
			{
				UserSearchPath.Append(ModuleNamePath);
				UserSearchPath.AppendChar(TEXT(';'));
			}

			// 3. Paths from UE_INSIGHTS_SYMBOL_PATH
			// 4. Paths from _NT_SYMBOL_PATH
			// 5. Paths from the user configuration file
			Algo::ForEach(ConfigSymbolSearchPaths,
				[&UserSearchPath](const FString& Path)
				{
					if (!Path.IsEmpty())
					{
						UserSearchPath.Append(Path);
						UserSearchPath.AppendChar(TEXT(';'));
					}
				});
		}

		if (!SymSetSearchPathW((HANDLE)Handle, UserSearchPath.ToString()))
		{
			UE_LOGF(LogDbgHelp, Warning, "Unable to set symbol search path to '%ls'.", UserSearchPath.ToString());
		}
		TCHAR OutPath[1024];
		SymGetSearchPathW((HANDLE) Handle, OutPath, 1024);
		UE_LOGF(LogDbgHelp, Display, "Search path: %ls", OutPath);
	}

	// Attempt to load symbols
	const DWORD64 LoadedBaseAddress = SymLoadModuleExW((HANDLE)Handle, NULL, Module->Name, NULL, Base, Size, NULL, 0);
	const bool bModuleLoaded = Base == LoadedBaseAddress;
	bool bPdbLoaded = true;
	bool bPdbMatchesImage = true;
	IMAGEHLP_MODULEW ModuleInfo;

	if (bModuleLoaded)
	{
		ModuleInfo.SizeOfStruct = sizeof(IMAGEHLP_MODULEW);
		SymGetModuleInfoW((HANDLE)Handle, Base, &ModuleInfo);

		if (ModuleInfo.SymType != SymPdb)
		{
			bPdbLoaded = false;
		}
		// Check image checksum if it exists
		else if (!ImageId.IsEmpty())
		{
			// for Pdbs checksum is a 16 byte guid and 4 byte unsigned integer for age, but usually age is not used for matching debug file to exe
			static_assert(sizeof(FGuid) == 16, "Expected 16 byte FGuid");
			check(ImageId.Num() == 20);
			const FGuid* ModuleGuid = (FGuid*) ImageId.GetData();
			const FGuid* PdbGuid = (FGuid*) &ModuleInfo.PdbSig70;
			bPdbMatchesImage = *ModuleGuid == *PdbGuid;
		}
	}

	TStringBuilder<256> StatusMessage;
	EModuleStatus Status;
	if (!bModuleLoaded || !bPdbLoaded)
	{
		// Unload the module, otherwise any subsequent attempts to load module with another
		// path will fail.
		SymUnloadModule((HANDLE)Handle, Base);
		StatusMessage.Appendf(TEXT("Unable to load symbols for %s"), Module->Name);
		Status = EModuleStatus::Failed;
		++ModulesFailed;
	}
	else if (!bPdbMatchesImage)
	{
		// Unload the module, otherwise any subsequent attempts to load module with another
		// path will fail.
		SymUnloadModule((HANDLE)Handle, Base);
		StatusMessage.Appendf(TEXT("Unable to load symbols for %s, pdb signature does not match."), Module->Name);
		Status = EModuleStatus::VersionMismatch;
		++ModulesFailed;
	}
	else
	{
		StatusMessage.Appendf(TEXT("Loaded symbols for %s from %s."), Module->Name, ModuleInfo.LoadedPdbName);
		Status = EModuleStatus::Loaded;
		++ModulesLoaded;
	}

	// Make the status visible to the world
	Module->StatusMessage = Session.StoreString(StatusMessage.ToView());
	Module->Status.store(Status);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#include "Windows/HideWindowsPlatformTypes.h"

#endif // PLATFORM_WINDOWS
