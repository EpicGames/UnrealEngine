// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "Common/PagedArray.h"
#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "HAL/CriticalSection.h"
#include "HAL/Runnable.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Modules.h"
#include <atomic>

DECLARE_LOG_CATEGORY_EXTERN(LogDbgHelp, Log, All);

namespace TraceServices {

class FDbgHelpResolver : public FRunnable
{
public:
	typedef TArray<TTuple<uint64, FResolvedSymbol*>> SymbolArray;

	FDbgHelpResolver(IAnalysisSession& InSession, IResolvedSymbolFilter& InSymbolFilter);
	~FDbgHelpResolver();

	void Start();
	void QueueModuleLoad(const uint8* ImageId, uint32 ImageIdSize, FModule* Module);
	void QueueModuleReload(FModule* Module, const TCHAR* Path, TFunction<void(SymbolArray&)> ResolveOnSuccess);
	void QueueSymbolResolve(uint64 Address, FResolvedSymbol* Symbol);
	void OnAnalysisComplete();
	void GetStats(IModuleProvider::FStats* OutStats) const;
	bool HasFinishedResolving() const;
	void EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const;

private:
	struct FModuleEntry
	{
		FModule* Module;
		TArray<uint8> ImageId;
	};

	struct FQueuedAddress
	{
		uint64 Address;
		FResolvedSymbol* Target;
	};

	struct FQueuedModule
	{
		FModule* Module;
		const TCHAR* Path;
		TArrayView<const uint8> ImageId;
	};

	enum : uint32 {
		MaxNameLen = 512,
	};

	bool SetupSyms();
	void FreeSyms() const;
	virtual uint32 Run() override;
	virtual void Stop() override;
	FModuleEntry* GetModuleEntry(FModule* Module) const;
	FModuleEntry* GetModuleForAddress(uint64 Address) const;
	static void UpdateResolvedSymbol(FResolvedSymbol& Symbol, ESymbolQueryResult Result, const TCHAR* Module, const TCHAR* Name, const TCHAR* File, uint16 Line);
	void ResolveSymbol(uint64 Address, FResolvedSymbol& Target);
	void LoadModuleSymbols(FModule* Module, const TCHAR* Path, const TArrayView<const uint8> ImageId);

	mutable FRWLock ModulesLock;
	TPagedArray<FModuleEntry> Modules;
	TArray<FModuleEntry*> SortedModules;
	TQueue<FQueuedModule, EQueueMode::Mpsc> LoadSymbolsQueue;
	TQueue<FQueuedAddress, EQueueMode::Mpsc> ResolveQueue;

	mutable FRWLock CustomSymbolSearchPathsLock;
	TArray<FString> CustomSymbolSearchPaths; // search paths added by user from UI in the current session
	TArray<FString> ConfigSymbolSearchPaths; // search paths specified in environment variables and config files

	std::atomic<uint32> ModulesDiscovered;
	std::atomic<uint32> ModulesFailed;
	std::atomic<uint32> ModulesLoaded;
	std::atomic<uint32> AlreadyResolvedSymbols;
	std::atomic<uint32> NoModuleSymbols;

	IAnalysisSession& Session;
	IResolvedSymbolFilter& SymbolFilter;

	bool bRunWorkerThread;
	bool bDrainThenStop;
	UPTRINT Handle;
	FRunnableThread* Thread;
};

} // namespace TraceServices

#endif // PLATFORM_WINDOWS
