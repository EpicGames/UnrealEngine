// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefinitionProvider.h"

#include "HAL/UnrealMemory.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FDefinitionProvider::FDefinitionProvider(IAnalysisSession* InSession)
	: PageRemain(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FDefinitionProvider::~FDefinitionProvider()
{
	for (uint8* Page : Pages)
	{
		FMemory::Free(Page);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDefinitionProvider::AddEntry(uint64 Hash, const void* Ptr)
{
	Definitions.Add(Hash, Ptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const void* FDefinitionProvider::FindEntry(uint64 Hash) const
{
	return Definitions.FindRef(Hash);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void* FDefinitionProvider::Allocate(uint32 Size, uint32 Alignment)
{
	check(Size);
	check(Size <= PageSize);
	uint8* Dest = nullptr;
	if (PageRemain > Size)
	{
		Dest = Pages.Last() + (PageSize - PageRemain);
	}
	else
	{
		Dest = (uint8*)FMemory::MallocZeroed(PageSize);
		Pages.Add(Dest);
		PageRemain = PageSize;
	}
	PageRemain -= Size;
	return Dest;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName GetDefinitionProviderName()
{
	static const FName Name("DefinitionProvider");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IDefinitionProvider* ReadDefinitionProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IDefinitionProvider>(GetDefinitionProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IDefinitionProvider* EditDefinitionProvider(IAnalysisSession& Session)
{
	return Session.EditProvider<IDefinitionProvider>(GetDefinitionProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
