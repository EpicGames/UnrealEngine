// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/UniquePtr.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/Definitions.h"

namespace TraceServices
{

class FDefinitionProvider : public IDefinitionProvider
{
public:
	FDefinitionProvider(IAnalysisSession* InSession);
	virtual ~FDefinitionProvider();

	virtual void BeginRead() const override { Lock.ReadLock(); }
	virtual void EndRead() const override { Lock.ReadUnlock(); }
	virtual void ReadAccessCheck() const override { }

	virtual void BeginEdit() const override { Lock.WriteLock(); }
	virtual void EndEdit() const override { Lock.WriteUnlock(); }
	virtual void EditAccessCheck() const override { }

private:
	virtual void AddEntry(uint64 Hash, const void* Ptr) override;
	virtual const void* FindEntry(uint64 Hash) const override;
	virtual void* Allocate(uint32 Size, uint32 Alignment) override;

private:
	mutable FRWLock Lock;

	TArray<uint8*> Pages;
	static constexpr uint32 PageSize = 1024;
	uint32 PageRemain;

	TMap<uint64, const void*> Definitions;
};

} // namespace TraceServices
