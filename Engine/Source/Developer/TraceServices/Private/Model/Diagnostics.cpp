// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Diagnostics.h"
#include "Model/DiagnosticsPrivate.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

FDiagnosticsProvider::FDiagnosticsProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FDiagnosticsProvider::SetSessionInfo(const FSessionInfo& InSessionInfo)
{
	Session.WriteAccessCheck();
	SessionInfo = InSessionInfo;
	bIsSessionInfoAvailable = true;
}

bool FDiagnosticsProvider::IsSessionInfoAvailable() const
{
	Session.ReadAccessCheck();
	return bIsSessionInfoAvailable;
}

const FSessionInfo& FDiagnosticsProvider::GetSessionInfo() const
{
	Session.ReadAccessCheck();
	return SessionInfo;
}

FName GetDiagnosticsProviderName()
{
	static const FName Name("DiagnosticsProvider");
	return Name;
}

const IDiagnosticsProvider* ReadDiagnosticsProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IDiagnosticsProvider>(GetDiagnosticsProviderName());
}

} // namespace TraceServices
