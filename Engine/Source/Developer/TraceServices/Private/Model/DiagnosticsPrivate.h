// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/Diagnostics.h"

namespace TraceServices
{

class FDiagnosticsProvider : public IDiagnosticsProvider
{
public:
	explicit FDiagnosticsProvider(IAnalysisSession& Session);
	virtual ~FDiagnosticsProvider() = default;

	virtual bool IsSessionInfoAvailable() const override;
	virtual const FSessionInfo& GetSessionInfo() const override;
	void SetSessionInfo(const FSessionInfo& InSessionInfo);

private:
	IAnalysisSession& Session;
	FSessionInfo SessionInfo;
	bool bIsSessionInfoAvailable = false;
};

} // namespace TraceServices
