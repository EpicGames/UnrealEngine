// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/FileActivity.h"
#include "Model/IntervalTimeline.h"
#include "AnalysisServicePrivate.h"
#include <limits>

namespace TraceServices
{

FFileActivityProvider::FFileActivityProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, Files(InSession.GetLinearAllocator(), 1024)
	, FileActivities(InSession.GetLinearAllocator(), 1024)
	, FileActivityTable(FileActivities)
{
	FileActivityTable.EditLayout().
		AddColumn<const TCHAR*>([](const FFileActivity& Row)
			{
				return Row.File ? Row.File->Path : TEXT("N/A");
			},
			TEXT("File")).
		AddColumn(&FFileActivity::StartTime, TEXT("StartTime")).
		AddColumn(&FFileActivity::EndTime, TEXT("EndTime")).
		AddColumn(&FFileActivity::ThreadId, TEXT("ThreadId")).
		AddColumn<const TCHAR*>([](const FFileActivity& Row)
			{
				return GetFileActivityTypeString(Row.ActivityType);
			},
			TEXT("Type")).
		AddColumn(&FFileActivity::Offset, TEXT("Offset")).
		AddColumn(&FFileActivity::Size, TEXT("Size")).
		AddColumn(&FFileActivity::ActualSize, TEXT("ActualSize")).
		AddColumn(&FFileActivity::Failed, TEXT("Failed"));
}

void FFileActivityProvider::EnumerateFileActivity(TFunctionRef<bool(const FFileInfo&, const Timeline&)> Callback) const
{
	for (uint64 FileIndex = 0, FileCount = Files.Num(); FileIndex < FileCount; ++FileIndex)
	{
		const FFileInfoInternal& FileInfoInternal = Files[FileIndex];
		if (!Callback(FileInfoInternal.FileInfo, *FileInfoInternal.ActivityTimeline))
		{
			return;
		}
	}
}

const ITable<FFileActivity>& FFileActivityProvider::GetFileActivityTable() const
{
	return FileActivityTable;
}

uint32 FFileActivityProvider::GetFileIndex(const TCHAR* Path)
{
	uint32 FileIndex = static_cast<uint32>(Files.Num());
	FFileInfoInternal& FileInfo = Files.PushBack();
	FileInfo.FileInfo.Id = FileIndex;
	FileInfo.FileInfo.Path = Session.StoreString(Path);
	FileInfo.ActivityTimeline = MakeShared<TimelineInternal>(Session.GetLinearAllocator());
	return FileIndex;
}

uint32 FFileActivityProvider::GetUnknownFileIndex()
{
	return GetFileIndex(TEXT("Unknown"));
}

uint64 FFileActivityProvider::BeginActivity(uint32 FileIndex, EFileActivityType Type, uint32 ThreadId, uint64 Offset, uint64 Size, double Time)
{
	FFileInfoInternal& FileInfo = Files[FileIndex];
	FFileActivity& FileActivity = FileActivities.PushBack();
	FileActivity.File = &FileInfo.FileInfo;
	FileActivity.Offset = Offset;
	FileActivity.Size = Size;
	FileActivity.ActualSize = 0;
	FileActivity.StartTime = Time;
	FileActivity.EndTime = std::numeric_limits<double>::infinity();
	FileActivity.ThreadId = ThreadId;
	FileActivity.Failed = false;
	FileActivity.ActivityType = Type;
	FileActivity.FileHandle = uint64(-1);
	FileActivity.ReadWriteHandle = uint64(-1);
	return FileInfo.ActivityTimeline->AppendBeginEvent(Time, &FileActivity);
}

void FFileActivityProvider::EndActivity(uint32 FileIndex, uint64 ActivityIndex, uint64 ActualSize, double Time, bool Failed)
{
	FFileInfoInternal& FileInfo = Files[FileIndex];
	FFileActivity* Activity = FileInfo.ActivityTimeline->EndEvent(ActivityIndex, Time);
	Activity->ActualSize = ActualSize;
	Activity->EndTime = Time;
	Activity->Failed = Failed;
}

void FFileActivityProvider::SetActivityFileHandle(uint32 FileIndex, uint64 ActivityIndex, uint64 FileHandle)
{
	FFileInfoInternal& FileInfo = Files[FileIndex];
	FFileActivity* Activity = FileInfo.ActivityTimeline->GetEvent(ActivityIndex);
	Activity->FileHandle = FileHandle;
}

void FFileActivityProvider::SetActivityReadWriteHandle(uint32 FileIndex, uint64 ActivityIndex, uint64 ReadWriteHandle)
{
	FFileInfoInternal& FileInfo = Files[FileIndex];
	FFileActivity* Activity = FileInfo.ActivityTimeline->GetEvent(ActivityIndex);
	Activity->ReadWriteHandle = ReadWriteHandle;
}

void FFileActivityProvider::CheckActivityReadWriteHandle(uint32 FileIndex, uint64 ActivityIndex, uint64 ReadWriteHandle)
{
	FFileInfoInternal& FileInfo = Files[FileIndex];
	FFileActivity* Activity = FileInfo.ActivityTimeline->GetEvent(ActivityIndex);
	check(Activity->ReadWriteHandle == ReadWriteHandle);
}

const TCHAR* FFileActivityProvider::GetFilePath(uint32 FileIndex) const
{
	return Files[FileIndex].FileInfo.Path;
}

const TCHAR* GetFileActivityTypeString(EFileActivityType ActivityType)
{
	switch (ActivityType)
	{
	case FileActivityType_Open:
		return TEXT("Open");
	case FileActivityType_Close:
		return TEXT("Close");
	case FileActivityType_Read:
		return TEXT("Read");
	case FileActivityType_Write:
		return TEXT("Write");
	}
	return TEXT("Invalid");
}

} // namespace TraceServices
