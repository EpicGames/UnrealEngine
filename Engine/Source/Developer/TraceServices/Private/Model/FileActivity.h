// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"
#include "Common/PagedArray.h"
#include "Model/IntervalTimeline.h"
#include "TraceServices/Model/LoadTimeProfiler.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Model/Tables.h"

namespace TraceServices
{

class FAnalysisSessionLock;

class FFileActivityProvider
	: public IFileActivityProvider
{
public:
	struct FTimelineSettings
	{
		enum
		{
			EventsPerPage = 128
		};
	};
	typedef TIntervalTimeline<FFileActivity*, FTimelineSettings> TimelineInternal;

	explicit FFileActivityProvider(IAnalysisSession& Session);
	virtual ~FFileActivityProvider() = default;

	virtual void EnumerateFileActivity(TFunctionRef<bool(const FFileInfo&, const Timeline&)> Callback) const override;
	virtual const ITable<FFileActivity>& GetFileActivityTable() const override;
	uint32 GetFileIndex(const TCHAR* Path);
	uint32 GetUnknownFileIndex();
	uint64 BeginActivity(uint32 FileIndex, EFileActivityType Type, uint32 ThreadId, uint64 Offset, uint64 Size, double Time);
	void EndActivity(uint32 FileIndex, uint64 ActivityIndex, uint64 ActualSize, double Time, bool Failed);
	void SetActivityFileHandle(uint32 FileIndex, uint64 ActivityIndex, uint64 FileHandle);
	void SetActivityReadWriteHandle(uint32 FileIndex, uint64 ActivityIndex, uint64 ReadWriteHandle);
	void CheckActivityReadWriteHandle(uint32 FileIndex, uint64 ActivityIndex, uint64 ReadWriteHandle);
	const TCHAR* GetFilePath(uint32 FileIndex) const;

private:
	struct FFileInfoInternal
	{
		FFileInfo FileInfo;
		TSharedPtr<TimelineInternal> ActivityTimeline;
	};

	IAnalysisSession& Session;
	TPagedArray<FFileInfoInternal> Files;
	TPagedArray<FFileActivity> FileActivities;
	TTableView<FFileActivity> FileActivityTable;
};

} // namespace TraceServices
