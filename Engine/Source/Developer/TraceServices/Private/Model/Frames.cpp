// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Frames.h"
#include "Model/FramesPrivate.h"

#include "Algo/BinarySearch.h"
#include "AnalysisServicePrivate.h"

#include <limits>

namespace TraceServices
{

FFrameProvider::FFrameProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
	for (int32 FrameType = 0; FrameType < TraceFrameType_Count; ++FrameType)
	{
		Frames.Emplace(Session.GetLinearAllocator(), 65536);
	}
}

uint64 FFrameProvider::GetFrameCount(ETraceFrameType FrameType) const
{
	Session.ReadAccessCheck();

	return Frames[FrameType].Num();
}

void FFrameProvider::EnumerateFrames(ETraceFrameType FrameType, uint64 StartIndex, uint64 EndIndex, TFunctionRef<void(const FFrame&)> Callback) const
{
	Session.ReadAccessCheck();

	EndIndex = FMath::Min(EndIndex, Frames[FrameType].Num());
	if (StartIndex >= EndIndex)
	{
		return;
	}
	for (auto Iterator = Frames[FrameType].GetIteratorFromItem(StartIndex); Iterator && Iterator->Index < EndIndex; ++Iterator)
	{
		Callback(*Iterator);
	}
}

void FFrameProvider::EnumerateFrames(ETraceFrameType FrameType, double StartTime, double EndTime, TFunctionRef<void(const FFrame&)> Callback) const
{
	Session.ReadAccessCheck();

	int64 StartIndex = Algo::LowerBound(FrameStartTimes[FrameType], StartTime);
	if (StartIndex > 0 && Frames[FrameType][StartIndex - 1].EndTime > StartTime)
	{
		--StartIndex;
	}

	for (uint64 Index = StartIndex; Index < Frames[FrameType].Num(); ++Index)
	{
		const FFrame& CurrentFrame = Frames[FrameType][Index];
		if (CurrentFrame.StartTime > EndTime)
		{
			break;
		}

		Callback(CurrentFrame);
	}
}

bool FFrameProvider::GetFrameFromTime(ETraceFrameType FrameType, double Time, FFrame& OutFrame) const
{
	Session.ReadAccessCheck();

	int64 LowerBound = Algo::LowerBound(FrameStartTimes[FrameType], Time);
	if (LowerBound <= 0)
	{
		return false;
	}
	check(LowerBound <= (int64)Frames[FrameType].Num());
	OutFrame = Frames[FrameType][LowerBound - 1];
	return true;
}

const FFrame* FFrameProvider::GetFrame(ETraceFrameType FrameType, uint64 Index) const
{
	Session.ReadAccessCheck();

	TPagedArray<FFrame> FramesOfType = Frames[FrameType];
	if (Index < FramesOfType.Num())
	{
		return &FramesOfType[Index];
	}
	else
	{
		return nullptr;
	}
}

void FFrameProvider::BeginFrame(ETraceFrameType FrameType, double Time)
{
	Session.WriteAccessCheck();

	uint64 Index = Frames[FrameType].Num();
	FFrame& Frame = Frames[FrameType].PushBack();
	Frame.StartTime = Time;
	Frame.EndTime = std::numeric_limits<double>::infinity();
	Frame.Index = Index;
	Frame.FrameType = FrameType;
	FrameStartTimes[FrameType].Add(Time);

	Session.UpdateDurationSeconds(Time);
}

void FFrameProvider::EndFrame(ETraceFrameType FrameType, double Time)
{
	Session.WriteAccessCheck();

	// Ignores the EndFrame event if it is the first event that comes through.
	if (Frames[FrameType].Num() > 0)
	{
		FFrame& Frame = Frames[FrameType][Frames[FrameType].Num() - 1];
		Frame.EndTime = Time;
	}

	Session.UpdateDurationSeconds(Time);
}

uint32 FFrameProvider::GetFrameNumberForTimestamp(ETraceFrameType FrameType, double Timestamp) const
{
	Session.ReadAccessCheck();

	int64 LowerBound = Algo::LowerBound(FrameStartTimes[FrameType], Timestamp);
	if (LowerBound <= 0)
	{
		return 0;
	}
	check(LowerBound <= (int64)Frames[FrameType].Num());
	return uint32(LowerBound - 1);
}

FName GetFrameProviderName()
{
	static const FName Name("FrameProvider");
	return Name;
}

const IFrameProvider& ReadFrameProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<IFrameProvider>(GetFrameProviderName());
}

} // namespace TraceServices
