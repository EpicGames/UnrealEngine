// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/PagedArray.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "TraceServices/Model/Frames.h"

namespace TraceServices
{

class IAnalysisSession;

class FFrameProvider
	: public IFrameProvider
{
public:
	explicit FFrameProvider(IAnalysisSession& Session);
	virtual ~FFrameProvider() = default;

	//////////////////////////////////////////////////
	// Read operations

	virtual uint64 GetFrameCount(ETraceFrameType FrameType) const override;
	virtual void EnumerateFrames(ETraceFrameType FrameType, uint64 StartIndex, uint64 EndIndex, TFunctionRef<void(const FFrame&)> Callback) const override;
	virtual void EnumerateFrames(ETraceFrameType FrameType, double StartTime, double EndTime, TFunctionRef<void(const FFrame&)> Callback) const override;
	virtual const TArray64<double>& GetFrameStartTimes(ETraceFrameType FrameType) const override { return FrameStartTimes[FrameType]; }
	virtual bool GetFrameFromTime(ETraceFrameType FrameType, double Time, FFrame& OutFrame) const override;
	virtual const FFrame* GetFrame(ETraceFrameType FrameType, uint64 Index) const override;
	virtual uint32 GetFrameNumberForTimestamp(ETraceFrameType FrameType, double Time) const override;

	//////////////////////////////////////////////////
	// Edit operations

	void BeginFrame(ETraceFrameType FrameType, double Time);
	void EndFrame(ETraceFrameType FrameType, double Time);

	//////////////////////////////////////////////////

private:
	IAnalysisSession& Session;
	TArray<TPagedArray<FFrame>> Frames;
	TArray64<double> FrameStartTimes[TraceFrameType_Count];
};

} // namespace TraceServices
