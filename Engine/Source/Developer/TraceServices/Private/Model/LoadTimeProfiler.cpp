// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/LoadTimeProfiler.h"
#include "Model/LoadTimeProfilerPrivate.h"
#include "AnalysisServicePrivate.h"
#include "Common/TimelineStatistics.h"
#include "TraceServices/Common/CancellationToken.h"

namespace TraceServices
{

FLoadTimeProfilerProvider::FLoaderFrameCounter::FLoaderFrameCounter(ELoaderFrameCounterType InType, const TPagedArray<FLoaderFrame>& InFrames)
	: Frames(InFrames)
	, Type(InType)
{

}

const TCHAR* FLoadTimeProfilerProvider::FLoaderFrameCounter::GetName() const
{
	switch (Type)
	{
	case LoaderFrameCounterType_IoDispatcherThroughput:
		return TEXT("AssetLoading/IoDispatcher/Throughput");
	case LoaderFrameCounterType_LoaderThroughput:
		return TEXT("AssetLoading/AsyncLoading/Throughput");
	}
	check(false);
	return TEXT("");
}

const TCHAR* FLoadTimeProfilerProvider::FLoaderFrameCounter::GetGroup() const
{
	return nullptr;
}

const TCHAR* FLoadTimeProfilerProvider::FLoaderFrameCounter::GetDescription() const
{
	return TEXT("N/A");
}

bool FLoadTimeProfilerProvider::FLoaderFrameCounter::IsFloatingPoint() const
{
	switch (Type)
	{
	case LoaderFrameCounterType_IoDispatcherThroughput:
	case LoaderFrameCounterType_LoaderThroughput:
		return true;
	}
	return false;
}

ECounterDisplayHint FLoadTimeProfilerProvider::FLoaderFrameCounter::GetDisplayHint() const
{
	switch (Type)
	{
	case LoaderFrameCounterType_IoDispatcherThroughput:
	case LoaderFrameCounterType_LoaderThroughput:
		return CounterDisplayHint_Memory;
	}
	return CounterDisplayHint_None;
}

void FLoadTimeProfilerProvider::FLoaderFrameCounter::EnumerateValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, int64)> Callback) const
{
	unimplemented();
}

void FLoadTimeProfilerProvider::FLoaderFrameCounter::EnumerateFloatValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, double)> Callback) const
{
	if (Frames.Num() == 0)
	{
		return;
	}
	int32 FirstFrameIndex = FMath::Max(0, int32(FMath::RoundToZero(IntervalStart / FLoadTimeProfilerProvider::LoaderFrameLength)) - 1);
	int32 LastFrameIndex = FMath::Min(int32(Frames.Num()) - 1, int32(FMath::RoundToZero(IntervalEnd / FLoadTimeProfilerProvider::LoaderFrameLength)) + 1);
	for (int32 FrameIndex = FirstFrameIndex; FrameIndex <= LastFrameIndex; ++FrameIndex)
	{
		double Time = (FrameIndex + 0.5) * FLoadTimeProfilerProvider::LoaderFrameLength;
		if (bIncludeExternalBounds || (IntervalStart <= Time && Time <= IntervalEnd))
		{
			const FLoaderFrame& Frame = Frames[FrameIndex];
			switch (Type)
			{
			case LoaderFrameCounterType_LoaderThroughput:
				Callback(Time, double(Frame.HeaderLoadedBytes + Frame.ExportLoadedBytes) / FLoadTimeProfilerProvider::LoaderFrameLength);
				break;
			case LoaderFrameCounterType_IoDispatcherThroughput:
				Callback(Time, double(Frame.IoDispatcherReadBytes) / FLoadTimeProfilerProvider::LoaderFrameLength);
				break;
			}
		}
	}
}

void FLoadTimeProfilerProvider::FLoaderFrameCounter::EnumerateOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, int64)> Callback) const
{
	unimplemented();
}

void FLoadTimeProfilerProvider::FLoaderFrameCounter::EnumerateFloatOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, double)> Callback) const
{
	unimplemented();
}

FLoadTimeProfilerProvider::FLoadTimeProfilerProvider(IAnalysisSession& InSession, IEditableCounterProvider& InEditableCounterProvider)
	: Session(InSession)
	, EditableCounterProvider(InEditableCounterProvider)
	, ClassInfos(Session.GetLinearAllocator(), 16384)
	, Requests(Session.GetLinearAllocator(), 16384)
	, Packages(Session.GetLinearAllocator(), 16384)
	, IoDispatcherBatches(Session.GetLinearAllocator(), 16384)
	, Exports(Session.GetLinearAllocator(), 16384)
	, RequestsTable(Requests)
	, Frames(Session.GetLinearAllocator(), 4096)
{
	RequestsTable.EditLayout().
		AddColumn(&FLoadRequest::Name, TEXT("Name")).
		AddColumn(&FLoadRequest::ThreadId, TEXT("ThreadId")).
		AddColumn(&FLoadRequest::StartTime, TEXT("StartTime"), TableColumnDisplayHint_Time).
		AddColumn(&FLoadRequest::EndTime, TEXT("EndTime"), TableColumnDisplayHint_Time).
		AddColumn<int32>(
			[](const FLoadRequest& Row)
			{
				return Row.Packages.Num();
			},
			TEXT("PackageCount"),
			TableColumnDisplayHint_Summable).
		AddColumn(&FLoadTimeProfilerProvider::PackageSizeSum, TEXT("Size"), TableColumnDisplayHint_Memory | TableColumnDisplayHint_Summable).
		AddColumn<const TCHAR*>(
			[](const FLoadRequest& Row)
			{
				return Row.Packages.Num() ? Row.Packages[0]->Name : TEXT("N/A");
			},
			TEXT("FirstPackage"));

	AggregatedStatsTableLayout.
		AddColumn(&FLoadTimeProfilerAggregatedStats::Name, TEXT("Name")).
		AddColumn(&FLoadTimeProfilerAggregatedStats::Count, TEXT("Count"), TableColumnDisplayHint_Summable).
		AddColumn(&FLoadTimeProfilerAggregatedStats::Total, TEXT("Total"), TableColumnDisplayHint_Time | TableColumnDisplayHint_Summable).
		AddColumn(&FLoadTimeProfilerAggregatedStats::Min, TEXT("Min"), TableColumnDisplayHint_Time).
		AddColumn(&FLoadTimeProfilerAggregatedStats::Max, TEXT("Max"), TableColumnDisplayHint_Time).
		AddColumn(&FLoadTimeProfilerAggregatedStats::Average, TEXT("Avg"), TableColumnDisplayHint_Time).
		AddColumn(&FLoadTimeProfilerAggregatedStats::Median, TEXT("Med"), TableColumnDisplayHint_Time);

	PackagesTableLayout.
		AddColumn<const TCHAR*>([](const FPackagesTableRow& Row)
			{
				return Row.PackageInfo->Name;
			},
			TEXT("Package")).
		AddColumn<uint64>([](const FPackagesTableRow& Row)
			{
				return Row.PackageInfo ? Row.PackageInfo->RequestId : 0;
			},
			TEXT("RequestId")).
		AddColumn(&FPackagesTableRow::TotalSerializedSize, TEXT("SerializedSize"), TableColumnDisplayHint_Memory | TableColumnDisplayHint_Summable).
		AddColumn(&FPackagesTableRow::SerializedHeaderSize, TEXT("SerializedHeaderSize"), TableColumnDisplayHint_Memory | TableColumnDisplayHint_Summable).
		AddColumn(&FPackagesTableRow::SerializedExportsCount, TEXT("SerializedExportsCount"), TableColumnDisplayHint_Summable).
		AddColumn(&FPackagesTableRow::SerializedExportsSize, TEXT("SerializedExportsSize"), TableColumnDisplayHint_Memory | TableColumnDisplayHint_Summable).
		AddColumn(&FPackagesTableRow::MainThreadTime, TEXT("MainThreadTime"), TableColumnDisplayHint_Time | TableColumnDisplayHint_Summable).
		AddColumn(&FPackagesTableRow::AsyncLoadingThreadTime, TEXT("AsyncLoadingThreadTime"), TableColumnDisplayHint_Time | TableColumnDisplayHint_Summable).
		AddColumn<int32>([](const FPackagesTableRow& Row)
			{
				return Row.PackageInfo ? Row.PackageInfo->ImportedPackages.Num() : 0;
			},
			TEXT("ImportedPackagesCount"), TableColumnDisplayHint_Summable);

	ExportsTableLayout.
		AddColumn<const TCHAR*>([](const FExportsTableRow& Row)
			{
				return Row.ExportInfo->Package ? Row.ExportInfo->Package->Name : TEXT("[unknown]");
			},
			TEXT("Package")).
		AddColumn<uint64>([](const FExportsTableRow& Row)
			{
				return Row.ExportInfo->Package ? Row.ExportInfo->Package->RequestId : 0;
			},
			TEXT("RequestId")).
		AddColumn<const TCHAR*>([](const FExportsTableRow& Row)
			{
				return Row.ExportInfo->Class ? Row.ExportInfo->Class->Name : TEXT("[unknown]");
			},
			TEXT("Class")).
		AddColumn<const TCHAR*>([](const FExportsTableRow& Row)
			{
				return GetLoadTimeProfilerObjectEventTypeString(Row.EventType);
			},
			TEXT("EventType")).
		AddColumn(&FExportsTableRow::SerializedSize, TEXT("SerializedSize"), TableColumnDisplayHint_Memory | TableColumnDisplayHint_Summable).
		AddColumn(&FExportsTableRow::MainThreadTime, TEXT("MainThreadTime"), TableColumnDisplayHint_Time | TableColumnDisplayHint_Summable).
		AddColumn(&FExportsTableRow::AsyncLoadingThreadTime, TEXT("AsyncLoadingThreadTime"), TableColumnDisplayHint_Time | TableColumnDisplayHint_Summable);

	RequestsTableLayout.
		AddColumn(&FRequestsTableRow::Id, TEXT("Id")).
		AddColumn(&FRequestsTableRow::Name, TEXT("Name")).
		AddColumn(&FRequestsTableRow::StartTime, TEXT("StartTime"), TableColumnDisplayHint_Time).
		AddColumn(&FRequestsTableRow::Duration, TEXT("Duration"), TableColumnDisplayHint_Time | TableColumnDisplayHint_Summable).
		AddColumn<int32>(
			[](const FRequestsTableRow& Row)
			{
				return Row.Packages.Num();
			},
			TEXT("PackageCount"), TableColumnDisplayHint_Summable).
		AddColumn<uint64>([](const FRequestsTableRow& Row)
			{
				uint64 Sum = 0;
				for (const FPackageInfo* Package : Row.Packages)
				{
					Sum += Package->Summary.TotalHeaderSize;
					for (const FPackageExportInfo* Export : Package->Exports)
					{
						Sum += Export->SerialSize;
					}
				}
				return Sum;
			},
			TEXT("SerializedSize"), TableColumnDisplayHint_Memory | TableColumnDisplayHint_Summable).
		AddColumn<const TCHAR*>(
			[](const FRequestsTableRow& Row)
			{
				return Row.Packages.Num() ? Row.Packages[0]->Name : TEXT("N/A");
			},
			TEXT("FirstPackage"));
}

bool FLoadTimeProfilerProvider::GetCpuThreadTimelineIndex(uint32 ThreadId, uint32& OutTimelineIndex) const
{
	Session.ReadAccessCheck();
	const uint32* FindTimelineIndex = CpuTimelinesThreadMap.Find(ThreadId);
	if (FindTimelineIndex)
	{
		OutTimelineIndex = *FindTimelineIndex;
		return true;
	}
	else
	{
		return false;
	}
}

bool FLoadTimeProfilerProvider::ReadTimeline(uint32 Index, TFunctionRef<void(const CpuTimeline &)> Callback) const
{
	Session.ReadAccessCheck();
	if (Index >= uint32(CpuTimelines.Num()))
	{
		return false;
	}
	Callback(*CpuTimelines[Index]);
	return true;
}

ITable<FLoadTimeProfilerAggregatedStats>* FLoadTimeProfilerProvider::CreateEventAggregation(double IntervalStart, double IntervalEnd) const
{
	TArray<const CpuTimelineInternal*> Timelines;
	for (const TSharedRef<CpuTimelineInternal>& Timeline : CpuTimelines)
	{
		Timelines.Add(&Timeline.Get());
	}

	auto BucketMapper = [](const FLoadTimeProfilerCpuEvent& Event)
	{
		return Event.Package;
	};

	FTimelineStatistics::TAggregationResult<const FPackageInfo*, FAggregatedTimingStats> Aggregation;

	// Cancellation not currently implemented for LoadTimeProfilerProvider.
	TSharedPtr<TraceServices::FCancellationToken> bIsCancelRequested = MakeShared<TraceServices::FCancellationToken>();

	FTimelineStatistics::CreateAggregation(Timelines, BucketMapper, IntervalStart, IntervalEnd, bIsCancelRequested, Aggregation);

	TTable<FLoadTimeProfilerAggregatedStats>* Table = new TTable<FLoadTimeProfilerAggregatedStats>(AggregatedStatsTableLayout);
	for (const auto& KV : Aggregation.GetMap())
	{
		const FPackageInfo* Package = KV.Key;
		if (Package)
		{
			FLoadTimeProfilerAggregatedStats& Row = Table->AddRow();
			Row.Name = Package->Name;
			const FAggregatedTimingStats& Stats = KV.Value;
			Row.Count = Stats.InstanceCount;
			Row.Total = Stats.TotalExclusiveTime;
			Row.Min = Stats.MinExclusiveTime;
			Row.Max = Stats.MaxExclusiveTime;
			Row.Average = Stats.AverageExclusiveTime;
			Row.Median = Stats.MedianExclusiveTime;
		}
	}
	return Table;
}

ITable<FLoadTimeProfilerAggregatedStats>* FLoadTimeProfilerProvider::CreateObjectTypeAggregation(double IntervalStart, double IntervalEnd) const
{
	TArray<const CpuTimelineInternal*> Timelines;
	for (const TSharedRef<CpuTimelineInternal>& Timeline : CpuTimelines)
	{
		Timelines.Add(&Timeline.Get());
	}

	auto BucketMapper = [](const FLoadTimeProfilerCpuEvent& Event) -> const FClassInfo*
	{
		return Event.Export ? Event.Export->Class : nullptr;
	};

	FTimelineStatistics::TAggregationResult<const FClassInfo*, FAggregatedTimingStats> Aggregation;

	// Cancellation not currently implemented for LoadTimeProfilerProvider.
	TSharedPtr<TraceServices::FCancellationToken> CancellationToken = MakeShared<TraceServices::FCancellationToken>();

	FTimelineStatistics::CreateAggregation(Timelines, BucketMapper, IntervalStart, IntervalEnd, CancellationToken, Aggregation);
	TTable<FLoadTimeProfilerAggregatedStats>* Table = new TTable<FLoadTimeProfilerAggregatedStats>(AggregatedStatsTableLayout);
	for (const auto& KV : Aggregation.GetMap())
	{
		const FClassInfo* ClassInfo = KV.Key;
		if (ClassInfo)
		{
			FLoadTimeProfilerAggregatedStats& Row = Table->AddRow();
			const FAggregatedTimingStats& Stats = KV.Value;
			Row.Name = ClassInfo->Name;
			Row.Count = Stats.InstanceCount;
			Row.Total = Stats.TotalExclusiveTime;
			Row.Min = Stats.MinExclusiveTime;
			Row.Max = Stats.MaxExclusiveTime;
			Row.Average = Stats.AverageExclusiveTime;
			Row.Median = Stats.MedianExclusiveTime;
		}
	}
	return Table;
}

ITable<FPackagesTableRow>* FLoadTimeProfilerProvider::CreatePackageDetailsTable(double IntervalStart, double IntervalEnd) const
{
	struct FTempRow
	{
		double StartTime = 0.0;
		const FPackageInfo* PackageInfo = nullptr;
		uint64 SerializedHeaderSize = 0;
		uint64 SerializedExportsCount = 0;
		uint64 SerializedExportsSize = 0;
		double MainThreadTime = 0.0;
		double AsyncLoadingThreadTime = 0.0;
	};
	TArray<FTempRow> Rows;
	struct FStackEntry
	{
		double StartTime;
		int32 RowIndex;
	};
	TArray<FStackEntry, TInlineAllocator<64>> Stack;
	double LastTime = 0.0;
	int32 TimelineIndex = 0;
	TMap<const FPackageInfo*, int32> RowMap;
	for (const TSharedRef<CpuTimelineInternal>& Timeline : CpuTimelines)
	{
		Timeline->EnumerateEvents(IntervalStart, IntervalEnd, [&Stack, &Rows, &RowMap, IntervalStart, IntervalEnd, &LastTime, TimelineIndex](bool IsEnter, double Time, const FLoadTimeProfilerCpuEvent& Event)
		{
			if (!Event.Package)
			{
				return EEventEnumerate::Continue;
			}
			double ClampedTime = FMath::Clamp(Time, IntervalStart, IntervalEnd);
			if (Stack.Num())
			{
				FStackEntry& StackEntry = Stack.Top();
				FTempRow& Row = Rows[StackEntry.RowIndex];
				if (TimelineIndex == 0)
				{
					Row.MainThreadTime += ClampedTime - LastTime;
				}
				else
				{
					Row.AsyncLoadingThreadTime += ClampedTime - LastTime;
				}
			}
			LastTime = ClampedTime;
			if (IsEnter)
			{
				FTempRow* Row;
				int32 RowIndex = -1;
				int32* FindRowIndex = RowMap.Find(Event.Package);
				if (FindRowIndex)
				{
					Row = &Rows[*FindRowIndex];
					RowIndex = *FindRowIndex;
				}
				else
				{
					RowIndex = Rows.Num();
					Row = &Rows.AddDefaulted_GetRef();
					Row->StartTime = Time;
					Row->PackageInfo = Event.Package;
					RowMap.Add(Event.Package, RowIndex);
				}
				FStackEntry& StackEntry = Stack.AddDefaulted_GetRef();
				StackEntry.StartTime = ClampedTime;
				StackEntry.RowIndex = RowIndex;
				if (Event.Export && Event.EventType == LoadTimeProfilerObjectEventType_Serialize)
				{
					Row->SerializedExportsSize += Event.Export->SerialSize;
					++Row->SerializedExportsCount;
				}
				else if (!Event.Export && Event.Package && !Row->SerializedHeaderSize)
				{
					Row->SerializedHeaderSize = Event.Package->Summary.TotalHeaderSize;
				}
			}
			else
			{
				Stack.Pop(EAllowShrinking::No);
			}
			return EEventEnumerate::Continue;
		});
		++TimelineIndex;
	}

	Algo::Sort(Rows, [](const FTempRow& A, const FTempRow& B)
		{
			return A.StartTime < B.StartTime;
		});
	TTable<FPackagesTableRow>* Table = new TTable<FPackagesTableRow>(PackagesTableLayout);
	for (const FTempRow& TempRow : Rows)
	{
		FPackagesTableRow& Row = Table->AddRow();
		Row.PackageInfo = TempRow.PackageInfo;
		Row.TotalSerializedSize = TempRow.SerializedHeaderSize + TempRow.SerializedExportsSize;
		Row.SerializedHeaderSize = TempRow.SerializedHeaderSize;
		Row.SerializedExportsCount = TempRow.SerializedExportsCount;
		Row.SerializedExportsSize = TempRow.SerializedExportsSize;

		Row.MainThreadTime = TempRow.MainThreadTime;
		Row.AsyncLoadingThreadTime = TempRow.AsyncLoadingThreadTime;
	}

	return Table;
}

ITable<FExportsTableRow>* FLoadTimeProfilerProvider::CreateExportDetailsTable(double IntervalStart, double IntervalEnd) const
{
	struct FTempRow
	{
		double StartTime = 0.0;
		const FPackageExportInfo* ExportInfo = nullptr;
		uint64 SerializedSize = 0;
		double MainThreadTime = 0.0;
		double AsyncLoadingThreadTime = 0.0;
		ELoadTimeProfilerObjectEventType EventType;
	};
	TArray<FTempRow> Rows;
	struct FStackEntry
	{
		double StartTime;
		int32 RowIndex;
	};
	TArray<FStackEntry, TInlineAllocator<64>> Stack;
	double LastTime = 0.0;
	int32 TimelineIndex = 0;
	for (const TSharedRef<CpuTimelineInternal>& Timeline : CpuTimelines)
	{
		Timeline->EnumerateEvents(IntervalStart, IntervalEnd, [&Stack, &Rows, IntervalStart, IntervalEnd, &LastTime, TimelineIndex](bool IsEnter, double Time, const FLoadTimeProfilerCpuEvent& Event)
		{
			if (!Event.Export)
			{
				return EEventEnumerate::Continue;
			}
			double ClampedTime = FMath::Clamp(Time, IntervalStart, IntervalEnd);
			if (Stack.Num())
			{
				FStackEntry& StackEntry = Stack.Top();
				FTempRow& Row = Rows[StackEntry.RowIndex];
				if (TimelineIndex == 0)
				{
					Row.MainThreadTime += ClampedTime - LastTime;
				}
				else
				{
					Row.AsyncLoadingThreadTime += ClampedTime - LastTime;
				}
			}
			LastTime = ClampedTime;
			if (IsEnter)
			{
				FStackEntry& StackEntry = Stack.AddDefaulted_GetRef();
				StackEntry.StartTime = ClampedTime;
				StackEntry.RowIndex = Rows.Num();
				FTempRow& Row = Rows.AddDefaulted_GetRef();
				Row.StartTime = Time;
				Row.ExportInfo = Event.Export;
				Row.EventType = Event.EventType;
				if ( Event.EventType == LoadTimeProfilerObjectEventType_Serialize)
				{
					Row.SerializedSize = Event.Export->SerialSize;
				}
			}
			else
			{
				Stack.Pop(EAllowShrinking::No);
			}
			return EEventEnumerate::Continue;
		});
		++TimelineIndex;
	}

	Algo::Sort(Rows, [](const FTempRow& A, const FTempRow& B)
		{
			return A.StartTime < B.StartTime;
		});
	TTable<FExportsTableRow>* Table = new TTable<FExportsTableRow>(ExportsTableLayout);
	for (const FTempRow& TempRow : Rows)
	{
		FExportsTableRow& Row = Table->AddRow();
		Row.ExportInfo = TempRow.ExportInfo;
		Row.SerializedSize = TempRow.SerializedSize;
		Row.MainThreadTime = TempRow.MainThreadTime;
		Row.AsyncLoadingThreadTime = TempRow.AsyncLoadingThreadTime;
		Row.EventType = TempRow.EventType;
	}

	return Table;
}

ITable<FRequestsTableRow>* FLoadTimeProfilerProvider::CreateRequestsTable(double IntervalStart, double IntervalEnd) const
{
	TTable<FRequestsTableRow>* Table = new TTable<FRequestsTableRow>(RequestsTableLayout);
	for (const FLoadRequest& Request : Requests)
	{
		if (Request.EndTime < IntervalStart || Request.StartTime > IntervalEnd)
		{
			continue;
		}
		FRequestsTableRow& Row = Table->AddRow();
		Row.Id = Request.Id;
		Row.Name = Request.Name;
		Row.StartTime = Request.StartTime;
		Row.Duration = Request.EndTime - Request.StartTime;
		Row.Packages = Request.Packages;
	}

	return Table;
}

const FClassInfo& FLoadTimeProfilerProvider::AddClassInfo(const TCHAR* ClassName)
{
	Session.WriteAccessCheck();

	FClassInfo& ClassInfo = ClassInfos.PushBack();
	ClassInfo.Name = Session.StoreString(ClassName);
	return ClassInfo;
}

FLoadRequest& FLoadTimeProfilerProvider::CreateRequest()
{
	Session.WriteAccessCheck();

	FLoadRequest& RequestInfo = Requests.PushBack();
	RequestInfo.Id = Requests.Num();
	return RequestInfo;
}

FPackageInfo& FLoadTimeProfilerProvider::CreatePackage()
{
	Session.WriteAccessCheck();

	uint32 PackageId = uint32(Packages.Num());
	FPackageInfo& Package = Packages.PushBack();
	Package.Id = PackageId;
	return Package;
}

void FLoadTimeProfilerProvider::DistributeBytesAcrossFrames(uint64 ByteCount, double StartTime, double EndTime, uint64 FLoaderFrame::* FrameVariable)
{
	uint64 OriginalByteCount = ByteCount;

	int32 BeginFrameIndex = int32(FMath::RoundToZero(StartTime / LoaderFrameLength));
	int32 EndFrameIndex = int32(FMath::RoundToZero(EndTime / LoaderFrameLength));
	while (Frames.Num() <= EndFrameIndex + 1)
	{
		Frames.PushBack();
	}

	if (BeginFrameIndex == EndFrameIndex)
	{
		FLoaderFrame& Frame = Frames[BeginFrameIndex];
		Frame.*FrameVariable += ByteCount;
		return;
	}

	double TotalTime = EndTime - StartTime;
	double TimeInFirstFrame = (BeginFrameIndex + 1) * LoaderFrameLength - StartTime;
	check(TimeInFirstFrame >= 0.0);
	uint64 BytesInFirstFrame = uint64(FMath::RoundToZero(TimeInFirstFrame / TotalTime * double(ByteCount)));
	Frames[BeginFrameIndex].*FrameVariable += BytesInFirstFrame;

	uint64 TotalDistributed = BytesInFirstFrame;

	double TimeInLastFrame = EndTime - EndFrameIndex * LoaderFrameLength;
	check(TimeInLastFrame >= 0.0);
	uint64 BytesInLastFrame = uint64(FMath::RoundToZero(TimeInLastFrame / TotalTime * double(ByteCount)));

	check(BytesInFirstFrame + BytesInLastFrame <= ByteCount);
	ByteCount -= BytesInFirstFrame + BytesInLastFrame;

	int32 FullFramesLeft = EndFrameIndex - BeginFrameIndex - 1;
	for (int32 FrameIndex = BeginFrameIndex + 1; FrameIndex < EndFrameIndex; ++FrameIndex)
	{
		uint64 BytesInFrame = ByteCount / FullFramesLeft;
		Frames[FrameIndex].*FrameVariable += BytesInFrame;

		TotalDistributed += BytesInFrame;

		ByteCount -= BytesInFrame;
		--FullFramesLeft;
	}

	BytesInLastFrame += ByteCount;
	Frames[EndFrameIndex].*FrameVariable += BytesInLastFrame;
	TotalDistributed += BytesInLastFrame;

	check(TotalDistributed == OriginalByteCount);
}

uint64 FLoadTimeProfilerProvider::BeginIoDispatcherBatch(uint64 BatchId, double Time)
{
	Session.WriteAccessCheck();

	CreateCounters();

	ActiveIoDispatcherBatchesCounter->AddValue(Time, int64(1));

	uint64 BatchHandle = IoDispatcherBatches.Num();

	FIoDispatcherBatch& Batch = IoDispatcherBatches.PushBack();
	Batch.StartTime = Time;

	return BatchHandle;
}

void FLoadTimeProfilerProvider::EndIoDispatcherBatch(uint64 BatchHandle, double Time, uint64 TotalSize)
{
	Session.WriteAccessCheck();

	FIoDispatcherBatch& Batch = IoDispatcherBatches[BatchHandle];
	Batch.EndTime = Time;
	Batch.TotalSize = TotalSize;

	check(Batch.EndTime >= Batch.StartTime);

	DistributeBytesAcrossFrames(Batch.TotalSize, Batch.StartTime, Batch.EndTime, &FLoaderFrame::IoDispatcherReadBytes);

	TotalIoDispatcherBytesReadCounter->AddValue(Time, int64(Batch.TotalSize));
	ActiveIoDispatcherBatchesCounter->AddValue(Time, int64(-1));
}

FPackageExportInfo& FLoadTimeProfilerProvider::CreateExport()
{
	Session.WriteAccessCheck();

	uint32 ExportId = uint32(Exports.Num());
	FPackageExportInfo& Export = Exports.PushBack();
	Export.Id = ExportId;
	return Export;
}

FLoadTimeProfilerProvider::CpuTimelineInternal& FLoadTimeProfilerProvider::EditCpuTimeline(uint32 ThreadId)
{
	Session.WriteAccessCheck();

	uint32* FindTimelineIndex = CpuTimelinesThreadMap.Find(ThreadId);
	if (FindTimelineIndex)
	{
		return CpuTimelines[*FindTimelineIndex].Get();
	}
	else
	{
		TSharedRef<CpuTimelineInternal> Timeline = MakeShared<CpuTimelineInternal>(Session.GetLinearAllocator());
		uint32 TimelineIndex = CpuTimelines.Num();
		CpuTimelinesThreadMap.Add(ThreadId, TimelineIndex);
		CpuTimelines.Add(Timeline);
		return Timeline.Get();
	}
}

uint64 FLoadTimeProfilerProvider::PackageSizeSum(const FLoadRequest& Row)
{
	uint64 Sum = 0;
	for (const FPackageInfo* Package : Row.Packages)
	{
		Sum += Package->Summary.TotalHeaderSize;
		for (const FPackageExportInfo* Export : Package->Exports)
		{
			Sum += Export->SerialSize;
		}
	}
	return Sum;
}

void FLoadTimeProfilerProvider::CreateCounters()
{
	if (!bHasCreatedCounters)
	{
		for (int32 CounterType = 0; CounterType < FLoaderFrameCounter::LoaderFrameCounterType_Count; ++CounterType)
		{
			EditableCounterProvider.AddCounter(new FLoaderFrameCounter(FLoaderFrameCounter::ELoaderFrameCounterType(CounterType), Frames));
		}

		ActiveIoDispatcherBatchesCounter = EditableCounterProvider.CreateEditableCounter();
		ActiveIoDispatcherBatchesCounter->SetName(TEXT("AssetLoading/IoDispatcher/ActiveBatches"));

		TotalIoDispatcherBytesReadCounter = EditableCounterProvider.CreateEditableCounter();
		TotalIoDispatcherBytesReadCounter->SetName(TEXT("AssetLoading/IoDispatcher/TotalBytesRead"));
		TotalIoDispatcherBytesReadCounter->SetDisplayHint(CounterDisplayHint_Memory);

		LoadingPackagesCounter = EditableCounterProvider.CreateEditableCounter();
		LoadingPackagesCounter->SetName(TEXT("AssetLoading/AsyncLoading/ActiveLoadingPackages"));

		TotalLoaderBytesLoadedCounter = EditableCounterProvider.CreateEditableCounter();
		TotalLoaderBytesLoadedCounter->SetName(TEXT("AssetLoading/AsyncLoading/TotalBytesLoaded"));
		TotalLoaderBytesLoadedCounter->SetDisplayHint(CounterDisplayHint_Memory);

		bHasCreatedCounters = true;
	}
}

const TCHAR* GetLoadTimeProfilerObjectEventTypeString(ELoadTimeProfilerObjectEventType EventType)
{
	switch (EventType)
	{
	case LoadTimeProfilerObjectEventType_Create:
		return TEXT("Create");
	case LoadTimeProfilerObjectEventType_Serialize:
		return TEXT("Serialize");
	case LoadTimeProfilerObjectEventType_PostLoad:
		return TEXT("PostLoad");
	case LoadTimeProfilerObjectEventType_None:
		return TEXT("None");
	}
	return TEXT("[invalid]");
}

} // namespace TraceServices
