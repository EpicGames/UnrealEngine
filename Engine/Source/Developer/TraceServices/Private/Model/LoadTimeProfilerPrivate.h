// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/Counters.h"
#include "TraceServices/Model/LoadTimeProfiler.h"
#include "TraceServices/Containers/SlabAllocator.h"
#include "Common/PagedArray.h"
#include "Common/StringStore.h"
#include "Model/MonotonicTimeline.h"
#include "Model/Tables.h"

namespace TraceServices
{

class FAnalysisSessionLock;

class FLoadTimeProfilerProvider
	: public ILoadTimeProfilerProvider
{
public:
	typedef TMonotonicTimeline<FLoadTimeProfilerCpuEvent> CpuTimelineInternal;

	explicit FLoadTimeProfilerProvider(IAnalysisSession& Session, IEditableCounterProvider& EditableCounterProvider);
	virtual ~FLoadTimeProfilerProvider() = default;

	virtual uint32 GetTimelineCount() const override { return CpuTimelines.Num(); }
	virtual bool GetCpuThreadTimelineIndex(uint32 ThreadId, uint32& OutTimelineIndex) const override;
	virtual bool ReadTimeline(uint32 Index, TFunctionRef<void(const CpuTimeline&)> Callback) const override;
	virtual ITable<FLoadTimeProfilerAggregatedStats>* CreateEventAggregation(double IntervalStart, double IntervalEnd) const override;
	virtual ITable<FLoadTimeProfilerAggregatedStats>* CreateObjectTypeAggregation(double IntervalStart, double IntervalEnd) const override;
	virtual ITable<FPackagesTableRow>* CreatePackageDetailsTable(double IntervalStart, double IntervalEnd) const override;
	virtual ITable<FExportsTableRow>* CreateExportDetailsTable(double IntervalStart, double IntervalEnd) const override;
	virtual ITable<FRequestsTableRow>* CreateRequestsTable(double IntervalStart, double IntervalEnd) const override;
	virtual const ITable<FLoadRequest>& GetRequestsTable() const override { return RequestsTable; }
	const FClassInfo& AddClassInfo(const TCHAR* ClassName);
	FLoadRequest& CreateRequest();
	FPackageInfo& CreatePackage();
	FPackageExportInfo& CreateExport();
	CpuTimelineInternal& EditCpuTimeline(uint32 ThreadId);
	uint64 BeginIoDispatcherBatch(uint64 BatchId, double Time);
	void EndIoDispatcherBatch(uint64 BatchHandle, double Time, uint64 TotalSize);

private:
	struct FLoaderFrame;

	static uint64 PackageSizeSum(const FLoadRequest& Row);
	void CreateCounters();
	void DistributeBytesAcrossFrames(uint64 ByteCount, double StartTime, double EndTime, uint64 FLoaderFrame::* FrameVariable);

	static constexpr double LoaderFrameLength = 1.0 / 60.0;

	struct FPackageLoad
	{
		const FPackageInfo* Package = nullptr;
		double StartTime = 0.0;
		double EndTime = 0.0;
	};

	struct FIoDispatcherBatch
	{
		double StartTime = 0.0;
		double EndTime = 0.0;
		uint64 TotalSize = 0;
	};

	struct FLoaderFrame
	{
		uint64 IoDispatcherReadBytes = 0;
		uint64 HeaderLoadedBytes = 0;
		uint64 ExportLoadedBytes = 0;
	};

	class FLoaderFrameCounter
		: public ICounter
	{
	public:
		enum ELoaderFrameCounterType
		{
			LoaderFrameCounterType_IoDispatcherThroughput,
			LoaderFrameCounterType_LoaderThroughput,
			LoaderFrameCounterType_Count
		};

		FLoaderFrameCounter(ELoaderFrameCounterType Type, const TPagedArray<FLoaderFrame>& Frames);
		virtual ~FLoaderFrameCounter() = default;

		virtual const TCHAR* GetName() const override;
		virtual const TCHAR* GetGroup() const override;
		virtual const TCHAR* GetDescription() const override;
		virtual bool IsFloatingPoint() const override;
		virtual ECounterDisplayHint GetDisplayHint() const override;
		virtual void EnumerateValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, int64)> Callback) const override;
		virtual void EnumerateFloatValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, double)> Callback) const override;
		virtual void EnumerateOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, int64)> Callback) const override;
		virtual void EnumerateFloatOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, double)> Callback) const override;

	private:
		const TPagedArray<FLoaderFrame>& Frames;
		ELoaderFrameCounterType Type;
	};

	IAnalysisSession& Session;
	IEditableCounterProvider& EditableCounterProvider;
	TPagedArray<FClassInfo> ClassInfos;
	TPagedArray<FLoadRequest> Requests;
	TPagedArray<FPackageInfo> Packages;
	TPagedArray<FIoDispatcherBatch> IoDispatcherBatches;
	TPagedArray<FPackageExportInfo> Exports;
	TArray<TSharedRef<CpuTimelineInternal>> CpuTimelines;
	TMap<uint32, uint32> CpuTimelinesThreadMap;
	TTableView<FLoadRequest> RequestsTable;
	TTableLayout<FLoadTimeProfilerAggregatedStats> AggregatedStatsTableLayout;
	TTableLayout<FPackagesTableRow> PackagesTableLayout;
	TTableLayout<FExportsTableRow> ExportsTableLayout;
	TTableLayout<FRequestsTableRow> RequestsTableLayout;
	TPagedArray<FLoaderFrame> Frames;
	bool bHasCreatedCounters = false;
	IEditableCounter* ActiveIoDispatcherBatchesCounter = nullptr;
	IEditableCounter* TotalIoDispatcherBytesReadCounter = nullptr;
	IEditableCounter* LoadingPackagesCounter = nullptr;
	IEditableCounter* TotalLoaderBytesLoadedCounter = nullptr;
};

} // namespace TraceServices
