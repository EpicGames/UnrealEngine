// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Log.h"
#include "Model/LogPrivate.h"

#include "AnalysisServicePrivate.h"
#include "Common/FormatArgs.h"

namespace TraceServices
{

FLogProvider::FLogProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, Categories(InSession.GetLinearAllocator(), 128)
	, MessageSpecs(InSession.GetLinearAllocator(), 1024)
	, Messages(InSession.GetLinearAllocator(), 1024)
	, MessagesTable(Messages)
{
	MessagesTable.EditLayout().
		AddColumn(&FLogMessageInternal::Time, TEXT("Time")).
		AddColumn<const TCHAR*>([](const FLogMessageInternal& Message)
			{
				return ToString(Message.Spec->Verbosity);
			},
			TEXT("Verbosity")).
		AddColumn<const TCHAR*>([](const FLogMessageInternal& Message)
			{
				return Message.Spec->Category->Name;
			},
			TEXT("Category")).
		AddColumn<const TCHAR*>([](const FLogMessageInternal& Message)
			{
				return Message.Spec->File;
			},
			TEXT("File")).
		AddColumn<int32>([](const FLogMessageInternal& Message)
			{
				return Message.Spec->Line;
			},
			TEXT("Line")).
		AddColumn(&FLogMessageInternal::Message, TEXT("Message"));
}

FLogProvider::~FLogProvider()
{
	for (const auto& KV : MissingLogPoints)
	{
		if (KV.Value.FormatArgs)
		{
			FMemory::Free(KV.Value.FormatArgs);
		}
	}
	MissingLogPoints.Reset();
}

uint64 FLogProvider::RegisterCategory()
{
	static uint64 IdGenerator = 0;
	return IdGenerator++;
}

FLogCategoryInfo& FLogProvider::GetCategory(uint64 CategoryPointer)
{
	Session.WriteAccessCheck();
	FLogCategoryInfo** FoundCategory = CategoryMap.Find(CategoryPointer);
	if (FoundCategory)
	{
		return **FoundCategory;
	}
	else
	{
		FLogCategoryInfo& Category = Categories.PushBack();
		Category.Name = TEXT("N/A");
		Category.DefaultVerbosity = ELogVerbosity::All;
		CategoryMap.Add(CategoryPointer, &Category);
		return Category;
	}
}

FLogMessageSpec& FLogProvider::GetMessageSpec(uint64 LogPoint)
{
	Session.WriteAccessCheck();
	FLogMessageSpec** FoundMessageSpec = SpecMap.Find(LogPoint);
	if (FoundMessageSpec)
	{
		return **FoundMessageSpec;
	}
	else
	{
		FLogMessageSpec& Spec = MessageSpecs.PushBack();
		SpecMap.Add(LogPoint, &Spec);
		return Spec;
	}
}

void FLogProvider::UpdateMessageCategory(uint64 LogPoint, uint64 InCategoryPointer)
{
	Session.WriteAccessCheck();
	FLogMessageSpec& LogMessageSpec = GetMessageSpec(LogPoint);
	LogMessageSpec.Category = &GetCategory(InCategoryPointer);
}

void FLogProvider::UpdateMessageFormatString(uint64 LogPoint, const TCHAR* InFormatString)
{
	Session.WriteAccessCheck();
	FLogMessageSpec& LogMessageSpec = GetMessageSpec(LogPoint);
	LogMessageSpec.FormatString = InFormatString;
}

void FLogProvider::UpdateMessageFile(uint64 LogPoint, const TCHAR* InFile, int32 InLine)
{
	Session.WriteAccessCheck();
	FLogMessageSpec& LogMessageSpec = GetMessageSpec(LogPoint);
	LogMessageSpec.File = InFile;
	LogMessageSpec.Line = InLine;
}

void FLogProvider::UpdateMessageVerbosity(uint64 LogPoint, ELogVerbosity::Type InVerbosity)
{
	Session.WriteAccessCheck();
	FLogMessageSpec& LogMessageSpec = GetMessageSpec(LogPoint);
	LogMessageSpec.Verbosity = InVerbosity;
}

void FLogProvider::UpdateMessageSpec(uint64 LogPoint, uint64 InCategoryPointer, const TCHAR* InFormatString, const TCHAR* InFile, int32 InLine, ELogVerbosity::Type InVerbosity)
{
	Session.WriteAccessCheck();
	FLogMessageSpec& LogMessageSpec = GetMessageSpec(LogPoint);
	LogMessageSpec.Category = &GetCategory(InCategoryPointer);
	LogMessageSpec.FormatString = InFormatString;
	LogMessageSpec.File = InFile;
	LogMessageSpec.Line = InLine;
	LogMessageSpec.Verbosity = InVerbosity;
	UpdateMessageSpec(LogPoint);
}

void FLogProvider::UpdateMessageSpec(uint64 LogPoint)
{
	Session.WriteAccessCheck();

	if (!MissingLogPoints.Contains(LogPoint))
	{
		return;
	}

	FLogMessageSpec** FoundMessageSpec = SpecMap.Find(LogPoint);
	if (!FoundMessageSpec)
	{
		return;
	}
	const FLogMessageSpec* Spec = *FoundMessageSpec;
	check(Spec != nullptr);

	TArray<FMissingMessage> MissingMessages;
	MissingLogPoints.MultiFind(LogPoint, MissingMessages, false);
	for (const FMissingMessage& MissingMessage : MissingMessages)
	{
		check(MissingMessage.Index < Messages.Num());
		FLogMessageInternal& InternalMessage = Messages[MissingMessage.Index];
		check(InternalMessage.Spec == Spec);
		FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, Spec->FormatString, MissingMessage.FormatArgs);
		InternalMessage.Message = Session.StoreString(FormatBuffer);
		if (MissingMessage.FormatArgs)
		{
			FMemory::Free(MissingMessage.FormatArgs);
		}
	}
	MissingLogPoints.Remove(LogPoint);
}

FLogMessageInternal& FLogProvider::AppendMessageInternal(double Time, uint64* OutIndex)
{
	// Performs binary search, resulting in position of the first log message with time > provided time value.
	uint64 Index = TraceServices::PagedArrayAlgo::UpperBoundBy(Messages, Time,
			[](const FLogMessageInternal& Item) { return Item.Time; });

	if (Index < Messages.Num())
	{
		++NumInserts;
	}

	for (auto& KV : MissingLogPoints)
	{
		if (KV.Value.Index >= Index)
		{
			KV.Value.Index++;
		}
	}

	if (OutIndex)
	{
		*OutIndex = Index;
	}

	FLogMessageInternal& InternalMessage = Messages.Insert(Index);
	InternalMessage.Time = Time;
	return InternalMessage;
}

void FLogProvider::AppendUnknownMessageInternal(uint64 LogPoint, double Time, const FStringView Message)
{
	FLogMessageSpec& Spec = GetMessageSpec(~0ull);
	FLogCategoryInfo& Category = GetCategory(~0ull);
	Spec.Category = &Category;
	Spec.Line = 0;
	Spec.Verbosity = ELogVerbosity::Error;
	Spec.FormatString = TEXT("%s");

	FLogMessageInternal& InternalMessage = AppendMessageInternal(Time);
	InternalMessage.Spec = &Spec;
	InternalMessage.Message = Session.StoreString(*FString::Printf(TEXT("Unknown log message spec (LogPoint=0x%llX)! Message: \"%.*s\""), LogPoint, Message.Len(), Message.GetData()));
}

void FLogProvider::AppendMessage(uint64 LogPoint, double Time, const uint8* FormatArgs)
{
	Session.WriteAccessCheck();
	FLogMessageSpec* Spec = SpecMap.FindRef(LogPoint);
	if (Spec)
	{
		if (Spec->Verbosity != ELogVerbosity::SetColor)
		{
			FLogMessageInternal& InternalMessage = AppendMessageInternal(Time);
			InternalMessage.Spec = Spec;
			FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, Spec->FormatString, FormatArgs);
			InternalMessage.Message = Session.StoreString(FormatBuffer);
		}
	}
	else
	{
		FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, TEXT("%s"), FormatArgs);
		AppendUnknownMessageInternal(LogPoint, Time, FStringView(FormatBuffer));
	}
	Session.UpdateDurationSeconds(Time);
}

void FLogProvider::AppendMessage(uint64 LogPoint, double Time, TArrayView<const uint8> FormatArgsView)
{
	Session.WriteAccessCheck();

	FLogMessageSpec* Spec = SpecMap.FindRef(LogPoint);
	if (Spec)
	{
		if (Spec->Verbosity != ELogVerbosity::SetColor)
		{
			FLogMessageInternal& InternalMessage = AppendMessageInternal(Time);
			InternalMessage.Spec = Spec;
			FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, Spec->FormatString, FormatArgsView.GetData());
			InternalMessage.Message = Session.StoreString(FormatBuffer);
		}
	}
	else
	{
		Spec = &GetMessageSpec(LogPoint);
		FLogCategoryInfo& Category = GetCategory(~0ull);
		Spec->Category = &Category;
		Spec->Line = 0;
		Spec->Verbosity = ELogVerbosity::Error;
		Spec->FormatString = TEXT("%s");

		uint64 Index = 0;
		FLogMessageInternal& InternalMessage = AppendMessageInternal(Time, &Index);
		InternalMessage.Spec = Spec;
		FFormatArgsHelper::Format(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, TEXT("%s"), FormatArgsView.GetData());
		FStringView Message(FormatBuffer);
		InternalMessage.Message = Session.StoreString(*FString::Printf(TEXT("Unknown log message spec (LogPoint=0x%llX)! Message: \"%.*s\""), LogPoint, Message.Len(), Message.GetData()));
		uint8* FormatArgs = nullptr;
		if (FormatArgsView.Num() > 0)
		{
			FormatArgs = (uint8*)FMemory::Malloc(FormatArgsView.Num());
			FMemory::Memcpy(FormatArgs, FormatArgsView.GetData(), FormatArgsView.Num());
		}
		MissingLogPoints.Add(LogPoint, { Index, FormatArgs });
		++NumUnknownSpecs;
	}

	Session.UpdateDurationSeconds(Time);
}

void FLogProvider::AppendMessage(uint64 LogPoint, double Time, const TCHAR* StoredText)
{
	Session.WriteAccessCheck();

	FLogMessageSpec* Spec = SpecMap.FindRef(LogPoint);
	if (!Spec)
	{
		Spec = &GetMessageSpec(LogPoint);
		FLogCategoryInfo& Category = GetCategory(~0ull);
		Spec->Category = &Category;
		Spec->Line = 0;
		Spec->Verbosity = ELogVerbosity::Error;
		Spec->FormatString = TEXT("%s");
	}

	if (Spec->Verbosity != ELogVerbosity::SetColor)
	{
		FLogMessageInternal& InternalMessage = AppendMessageInternal(Time);
		InternalMessage.Spec = Spec;
		InternalMessage.Message = StoredText;
	}

	Session.UpdateDurationSeconds(Time);
}

void FLogProvider::AppendMessage(uint64 LogPoint, double Time, const FStringView Message)
{
	Session.WriteAccessCheck();

	FLogMessageSpec* Spec = SpecMap.FindRef(LogPoint);
	if (Spec)
	{
		if (Spec->Verbosity != ELogVerbosity::SetColor)
		{
			FLogMessageInternal& InternalMessage = AppendMessageInternal(Time);
			InternalMessage.Spec = Spec;
			InternalMessage.Message = Session.StoreString(Message);
		}
	}
	else
	{
		AppendUnknownMessageInternal(LogPoint, Time, Message);
	}

	Session.UpdateDurationSeconds(Time);
}

bool FLogProvider::ReadMessage(uint64 Index, TFunctionRef<void(const FLogMessageInfo&)> Callback) const
{
	Session.ReadAccessCheck();

	if (Index >= Messages.Num())
	{
		return false;
	}

	ConstructMessage(Messages[Index], Index, Callback);

	return true;
}

void FLogProvider::EnumerateMessagesByIndex(uint64 StartIndex, uint64 EndIndex, TFunctionRef<void(const FLogMessageInfo&)> Callback) const
{
	Session.ReadAccessCheck();

	uint64 Count = Messages.Num();
	if (EndIndex > Count)
	{
		EndIndex = Count;
	}
	if (StartIndex >= EndIndex)
	{
		return;
	}

	for (auto It = Messages.GetIteratorFromItem(StartIndex); It && It.GetCurrentItemIndex() < EndIndex; ++It)
	{
		ConstructMessage(*It.GetCurrentItem(), It.GetCurrentItemIndex(), Callback);
	}
}

void FLogProvider::EnumerateMessages(double StartTime, double EndTime, TFunctionRef<void(const FLogMessageInfo&)> Callback) const
{
	Session.ReadAccessCheck();

	if (StartTime > EndTime)
	{
		return;
	}

	uint64 MessageCount = Messages.Num();
	if (MessageCount == 0)
	{
		return;
	}

	// Find the first log message with Time >= StartTime.
	uint64 StartIndex = TraceServices::PagedArrayAlgo::LowerBoundBy(Messages, StartTime,
		[](const FLogMessageInternal& Item) { return Item.Time; });
	if (StartIndex >= Messages.Num())
	{
		return;
	}

	// Iterate from StartIndex and stop at first log message with Time > EndTime.
	for (auto It = Messages.GetIteratorFromItem(StartIndex); It; ++It)
	{
		double Time = It.GetCurrentItem()->Time;
		if (Time > EndTime)
		{
			break;
		}
		ConstructMessage(*It.GetCurrentItem(), It.GetCurrentItemIndex(), Callback);
	}
}

void FLogProvider::ReverseEnumerateMessages(double StartTime, double EndTime, TFunctionRef<bool(const FLogMessageInfo&)> Callback) const
{
	Session.ReadAccessCheck();

	if (StartTime < EndTime)
	{
		return;
	}

	uint64 MessageCount = Messages.Num();
	if (MessageCount == 0)
	{
		return;
	}

	// Find the first log message before the current position
	uint64 StartIndex = TraceServices::PagedArrayAlgo::LowerBoundBy(Messages, StartTime,
		[](const FLogMessageInternal& Item) { return Item.Time; });
	if (StartIndex == 0 || StartIndex >= Messages.Num())
	{
		return;
	}
	--StartIndex;

	// Iterate from StartIndex and stop if we've exceeded the StartTime
	for (auto It = Messages.GetIteratorFromItem(StartIndex); It; --It)
	{
		double Time = It.GetCurrentItem()->Time;
		if (Time < EndTime)
		{
			break;
		}
		// Once we've found an appropriate element we can stop
		if (ConstructMessage(*It.GetCurrentItem(), It.GetCurrentItemIndex(), Callback))
		{
			break;
		}
	}
}

uint64 FLogProvider::LowerBoundByTime(double Time) const
{
	Session.ReadAccessCheck();

	return TraceServices::PagedArrayAlgo::LowerBoundBy(Messages, Time,
		[](const FLogMessageInternal& Item) { return Item.Time; });
}

uint64 FLogProvider::UpperBoundByTime(double Time) const
{
	Session.ReadAccessCheck();

	return TraceServices::PagedArrayAlgo::UpperBoundBy(Messages, Time,
		[](const FLogMessageInternal& Item) { return Item.Time; });
}

uint64 FLogProvider::BinarySearchClosestByTime(double Time) const
{
	Session.ReadAccessCheck();

	return TraceServices::PagedArrayAlgo::BinarySearchClosestBy(Messages, Time,
		[](const FLogMessageInternal& Item) { return Item.Time; });
}

bool FLogProvider::ConstructMessage(const FLogMessageInternal& InternalMessage, uint64 Index, TFunctionRef<void(const FLogMessageInfo&)> Callback) const
{
	FLogMessageInfo Message;
	Message.Index = Index;
	Message.Time = InternalMessage.Time;
	Message.Category = InternalMessage.Spec->Category;
	Message.File = InternalMessage.Spec->File;
	Message.Line = InternalMessage.Spec->Line;
	Message.Verbosity = InternalMessage.Spec->Verbosity;
	Message.Message = InternalMessage.Message;
	Callback(Message);
	return true;
}

bool FLogProvider::ConstructMessage(const FLogMessageInternal& InternalMessage, uint64 Index, TFunctionRef<bool(const FLogMessageInfo&)> Callback) const
{
	FLogMessageInfo Message;
	Message.Index = Index;
	Message.Time = InternalMessage.Time;
	Message.Category = InternalMessage.Spec->Category;
	Message.File = InternalMessage.Spec->File;
	Message.Line = InternalMessage.Spec->Line;
	Message.Verbosity = InternalMessage.Spec->Verbosity;
	Message.Message = InternalMessage.Message;
	return Callback(Message);
}

void FLogProvider::EnumerateCategories(TFunctionRef<void(const FLogCategoryInfo&)> Callback) const
{
	Session.ReadAccessCheck();
	for (auto Iterator = Categories.GetIteratorFromItem(0); Iterator; ++Iterator)
	{
		Callback(*Iterator);
	}
}

FName GetLogProviderName()
{
	static const FName Name("LogProvider");
	return Name;
}

const ILogProvider& ReadLogProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<ILogProvider>(GetLogProviderName());
}

IEditableLogProvider& EditLogProvider(IAnalysisSession& Session)
{
	return *Session.EditProvider<IEditableLogProvider>(GetLogProviderName());
}

void FormatString(TCHAR* OutputString, uint32 OutputStringCount, const TCHAR* FormatString, const uint8* FormatArgs)
{
	TCHAR* TempBuffer = (TCHAR*)FMemory_Alloca(OutputStringCount * sizeof(TCHAR));
	FFormatArgsHelper::Format(OutputString, OutputStringCount - 1, TempBuffer, OutputStringCount - 1, FormatString, FormatArgs);
}

} // namespace TraceServices
