// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Misc/OutputDeviceHelper.h"

// TraceServices
#include "Common/PagedArray.h"
#include "Model/Tables.h"
#include "TraceServices/Model/Log.h"

namespace TraceServices
{

class FAnalysisSessionLock;
class FStringStore;

struct FLogMessageSpec
{
	FLogCategoryInfo* Category = nullptr;
	const TCHAR* File = nullptr;
	const TCHAR* FormatString = nullptr;
	int32 Line;
	ELogVerbosity::Type Verbosity;
};

struct FLogMessageInternal
{
	FLogMessageSpec* Spec = nullptr;
	double Time;
	const TCHAR* Message = nullptr;
};

class FLogProvider
	: public ILogProvider
	, public IEditableLogProvider
{
public:
	explicit FLogProvider(IAnalysisSession& Session);
	virtual ~FLogProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual uint64 GetMessageCount() const override { Session.ReadAccessCheck(); return Messages.Num(); }
	virtual bool ReadMessage(uint64 Index, TFunctionRef<void(const FLogMessageInfo&)> Callback) const override;
	virtual void EnumerateMessagesByIndex(uint64 StartIndex, uint64 EndIndex, TFunctionRef<void(const FLogMessageInfo&)> Callback) const override;
	virtual void EnumerateMessages(double StartTime, double EndTime, TFunctionRef<void(const FLogMessageInfo&)> Callback) const override;
	virtual void ReverseEnumerateMessages(double StartTime, double EndTime, TFunctionRef<bool(const FLogMessageInfo&)> Callback) const override;

	virtual uint64 LowerBoundByTime(double Time) const override;
	virtual uint64 UpperBoundByTime(double Time) const override;
	virtual uint64 BinarySearchClosestByTime(double Time) const override;

	virtual uint64 GetMessageSpecCount() const override { Session.ReadAccessCheck(); return MessageSpecs.Num(); }

	virtual uint64 GetCategoryCount() const override { Session.ReadAccessCheck(); return Categories.Num(); }
	virtual void EnumerateCategories(TFunctionRef<void(const FLogCategoryInfo&)> Callback) const override;

	virtual const IUntypedTable& GetMessagesTable() const override { return MessagesTable; }

	virtual uint64 GetInsertCount() const override { Session.ReadAccessCheck(); return NumInserts; }
	virtual uint64 GetUnknownSpecTotalCount() const override { Session.ReadAccessCheck(); return NumUnknownSpecs; }
	virtual uint64 GetUnresolvedSpecCount() const override { Session.ReadAccessCheck(); return uint64(MissingLogPoints.Num()); }

	//////////////////////////////////////////////////
	// Edit operations

	virtual uint64 RegisterCategory() override;
	virtual FLogCategoryInfo& GetCategory(uint64 CategoryPointer) override;

	FLogMessageSpec& GetMessageSpec(uint64 LogPoint);
	virtual void UpdateMessageCategory(uint64 LogPoint, uint64 InCategoryPointer) override;
	virtual void UpdateMessageFormatString(uint64 LogPoint, const TCHAR* InFormatString) override;
	virtual void UpdateMessageFile(uint64 LogPoint, const TCHAR* InFile, int32 InLine) override;
	virtual void UpdateMessageVerbosity(uint64 LogPoint, ELogVerbosity::Type InVerbosity) override;
	virtual void UpdateMessageSpec(uint64 LogPoint, uint64 InCategoryPointer, const TCHAR* InFormatString, const TCHAR* InFile, int32 InLine, ELogVerbosity::Type InVerbosity) override;
	virtual void UpdateMessageSpec(uint64 LogPoint) override;

	virtual void AppendMessage(uint64 LogPoint, double Time, const uint8* FormatArgs) override;
	virtual void AppendMessage(uint64 LogPoint, double Time, TArrayView<const uint8> FormatArgsView) override;
	virtual void AppendMessage(uint64 LogPoint, double Time, const TCHAR* StoredText) override;
	void AppendMessage(uint64 LogPoint, double Time, const FStringView Message);

	//////////////////////////////////////////////////

private:
	FLogMessageInternal& AppendMessageInternal(double Time, uint64* OutIndex = nullptr);
	void AppendUnknownMessageInternal(uint64 LogPoint, double Time, const FStringView Message);
	bool ConstructMessage(const FLogMessageInternal& InternalMessage, uint64 Index, TFunctionRef<void(const FLogMessageInfo&)> Callback) const;
	bool ConstructMessage(const FLogMessageInternal& InternalMessage, uint64 Index, TFunctionRef<bool(const FLogMessageInfo&)> Callback) const;

	enum
	{
		FormatBufferSize = 65536
	};

	struct FMissingMessage // message with missing spec
	{
		uint64 Index = ~0ull; // index in Messages array
		uint8* FormatArgs = nullptr;
	};

	IAnalysisSession& Session;
	TMap<uint64, FLogCategoryInfo*> CategoryMap; // CategoryPointer --> FLogCategoryInfo*
	TMap<uint64, FLogMessageSpec*> SpecMap; // LogPoint --> FLogMessageSpec*
	TPagedArray<FLogCategoryInfo> Categories;
	TPagedArray<FLogMessageSpec> MessageSpecs;
	TPagedArray<FLogMessageInternal> Messages;
	TCHAR FormatBuffer[FormatBufferSize];
	TCHAR TempBuffer[FormatBufferSize];
	TTableView<FLogMessageInternal> MessagesTable;
	TMultiMap<uint64, FMissingMessage> MissingLogPoints; // LogPoint --> FMissingMessage[]
	uint64 NumInserts = 0; // total number of inserts (out of order messages)
	uint64 NumUnknownSpecs = 0; // total number of log messages received with unknown spec
};

} // namespace TraceServices
