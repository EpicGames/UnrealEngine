// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Memory.h"
#include "Model/MemoryPrivate.h"

#include "Common/Utils.h"

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GMemoryProviderLockState;

FMemoryProvider::FMemoryProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, SnapshotTimes(Session.GetLinearAllocator(), 8 * 1024)
{
	bIsInitialized = true;
	InternalAddTagSetSpec((FMemoryTagSetId)0, TEXT("Default"));
}

FMemoryProvider::~FMemoryProvider()
{
	bIsInitialized = false;

	for (FTrackerData* Tracker : AvailableTrackers)
	{
		if (Tracker)
		{
			delete Tracker->Info;
			delete Tracker;
		}
	}
	AvailableTrackers.Reset();
	NumTrackers = 0;

	for (FTagSetData* TagSet : AvailableTagSets)
	{
		if (TagSet)
		{
			delete TagSet->Info;
			delete TagSet;
		}
	}
	AvailableTagSets.Reset();
	NumTagSets = 0;

	for (FMemoryTagInfo* TagInfo : AvailableTags)
	{
		delete TagInfo;
	}
	AvailableTags.Reset();
	TagMap.Reset();
}

void FMemoryProvider::AddTrackerSpec(FMemoryTrackerId TrackerId, const FString& Name)
{
	EditAccessCheck();
	InternalAddTrackerSpec(TrackerId, Name);
}

void FMemoryProvider::InternalAddTrackerSpec(FMemoryTrackerId TrackerId, const FString& Name)
{
	if (TrackerId < 0 || TrackerId >= FMemoryTrackerInfo::MaxTrackers)
	{
		// invalid tracker id
		return;
	}

	while (AvailableTrackers.Num() <= TrackerId)
	{
		AvailableTrackers.AddDefaulted();
	}

	FTrackerData* Tracker = AvailableTrackers[TrackerId];
	if (Tracker)
	{
		// The tracker is already registered.
		check(Tracker->Info);

		// Update name.
		Tracker->Info->Name = Name;
		return;
	}

	FMemoryTrackerInfo* TrackerInfo = new FMemoryTrackerInfo({ TrackerId, Name });
	AvailableTrackers[(int32)TrackerId] = new FTrackerData(TrackerInfo);
	++NumTrackers;
}

FMemoryProvider::FTrackerData* FMemoryProvider::GetTracker(FMemoryTrackerId TrackerId) const
{
	if (TrackerId >= 0 && TrackerId < AvailableTrackers.Num())
	{
		return AvailableTrackers[TrackerId];
	}
	return nullptr;
}

FMemoryProvider::FTrackerData* FMemoryProvider::GetOrAddTracker(FMemoryTrackerId TrackerId)
{
	FTrackerData* Tracker = (TrackerId >= 0 && TrackerId < AvailableTrackers.Num()) ? AvailableTrackers[TrackerId] : nullptr;
	if (Tracker)
	{
		return Tracker;
	}
	InternalAddTrackerSpec(TrackerId, TEXT("<unknown>"));
	return AvailableTrackers[TrackerId];
}

void FMemoryProvider::AddTagSetSpec(FMemoryTagSetId TagSetId, const FString& Name)
{
	EditAccessCheck();
	InternalAddTagSetSpec(TagSetId, Name);
}

void FMemoryProvider::InternalAddTagSetSpec(FMemoryTagSetId TagSetId, const FString& Name)
{
	if (TagSetId < 0 || TagSetId >= FMemoryTagSetInfo::MaxTagSets)
	{
		// invalid tag set id
		return;
	}

	while (AvailableTagSets.Num() <= TagSetId)
	{
		AvailableTagSets.AddDefaulted();
	}

	FTagSetData* TagSet = AvailableTagSets[TagSetId];
	if (TagSet)
	{
		// The tag set is already registered.
		check(TagSet->Info);

		// Update name.
		TagSet->Info->Name = Name;

		return;
	}

	FMemoryTagSetInfo* TagSetInfo = new FMemoryTagSetInfo({ TagSetId, Name });
	AvailableTagSets[TagSetId] = new FTagSetData(TagSetInfo);
	++NumTagSets;
}

FMemoryProvider::FTagSetData* FMemoryProvider::GetTagSet(FMemoryTagSetId TagSetId) const
{
	if (TagSetId >= 0 && TagSetId < AvailableTagSets.Num())
	{
		return AvailableTagSets[TagSetId];
	}
	return nullptr;
}

FMemoryProvider::FTagSetData* FMemoryProvider::GetOrAddTagSet(FMemoryTagSetId TagSetId)
{
	FTagSetData* TagSet = (TagSetId >= 0 && TagSetId < AvailableTagSets.Num()) ? AvailableTagSets[TagSetId] : nullptr;
	if (TagSet)
	{
		return TagSet;
	}
	InternalAddTagSetSpec(TagSetId, TEXT("<unknown>"));
	return AvailableTagSets[TagSetId];
}

void FMemoryProvider::AddTagSpec(FMemoryTagId TagId, const FString& Name, FMemoryTagId ParentTagId, FMemoryTagSetId TagSetId)
{
	EditAccessCheck();
	InternalAddTagSpec(TagId, Name, ParentTagId, TagSetId);
}

void FMemoryProvider::InternalAddTagSpec(FMemoryTagId TagId, const FString& Name, FMemoryTagId ParentTagId, FMemoryTagSetId TagSetId)
{
	if (TagId == FMemoryTagInfo::InvalidTagId || TagId == -1)
	{
		// invalid tag id
		return;
	}

	if (ParentTagId == -1) // backward compatibility with UE 4.27
	{
		ParentTagId = FMemoryTagInfo::InvalidTagId;
	}

	if (TagSetId < 0 || TagSetId >= FMemoryTagSetInfo::MaxTagSets)
	{
		// invalid tag set id
		return;
	}
	FTagSetData* TagSet = GetOrAddTagSet(TagSetId);
	check(TagSet);

	FMemoryTagInfo** FoundTagInfo = TagMap.Find(TagId);
	if (FoundTagInfo)
	{
		// The tag is already registered.
		(*FoundTagInfo)->ParentId = ParentTagId;
		(*FoundTagInfo)->TagSetId = TagSetId;
		(*FoundTagInfo)->Name = Name;
		return;
	}

	uint64 Trackers = 0; // flags for trackers using this tag
	FMemoryTagInfo* TagInfo = new FMemoryTagInfo({ TagId, ParentTagId, TagSetId, Trackers, Name });
	TagMap.Add(TagId, TagInfo);

	AvailableTags.Add(TagInfo);
	TagSet->Tags.Add(TagId, TagInfo);

	++TagsSerial;
}

void FMemoryProvider::AddTagSnapshot(FMemoryTrackerId TrackerId, double Time, const int64* Tags, const FMemoryTagSample* Values, uint32 TagCount)
{
	EditAccessCheck();

	if (TrackerId < 0 || TrackerId >= FMemoryTrackerInfo::MaxTrackers)
	{
		// invalid tracker id
		return;
	}
	FTrackerData* Tracker = GetOrAddTracker(TrackerId);
	check(Tracker);

	SnapshotTimes.EmplaceBack(Time);
	const uint64 SampleCount = SnapshotTimes.Num();

	for (uint32 TagIndex = 0; TagIndex < TagCount; ++TagIndex)
	{
		const FMemoryTagId TagId = FMemoryTagId(Tags[TagIndex]);
		const FMemoryTagSample& Value = Values[TagIndex];

		FTagSampleData* TagSamples = Tracker->Samples.FindRef(TagId);
		if (!TagSamples)
		{
			TagSamples = new FTagSampleData(Session.GetLinearAllocator());
			Tracker->Samples.Add(TagId, TagSamples);
		}

		auto& TagValues = TagSamples->Values;

		// Add "past values". If new value is added, we need to ensure we have
		// the same number of elements in the value array as in the timestamp array.
		uint64 CurrentSampleCount = TagValues.Num();
		if (CurrentSampleCount < SampleCount - 1)
		{
			const FMemoryTagSample LastValue = (CurrentSampleCount > 0) ? TagValues.Last() : FMemoryTagSample { 0 };
			for (uint64 ValueIndex = SampleCount - 1; ValueIndex > CurrentSampleCount; --ValueIndex)
			{
				TagValues.PushBack() = LastValue;
			}
		}

		TagValues.PushBack() = Values[TagIndex];
		check(int32(TagValues.Num()) == SampleCount);

		if (!TagSamples->TagInfo)
		{
			// Cache pointer to FMemoryTagInfo to avoid further lookups for this tag.
			if (FMemoryTagInfo* TagInfo = const_cast<FMemoryTagInfo*>(GetTag(TagId)))
			{
				TagSamples->TagInfo = TagInfo;
			}
		}

		if (TagSamples->TagInfo)
		{
			const uint64 TrackerFlag = 1ULL << TrackerId;
			if ((TagSamples->TagInfo->Trackers & TrackerFlag) == 0)
			{
				TagSamples->TagInfo->Trackers |= TrackerFlag;
				++TagsSerial;
			}
		}
	}
}

void FMemoryProvider::OnAnalysisCompleted()
{
	EditAccessCheck();

	bIsCompleted = true;

	UE_LOGF(LogTraceServices, Log, "[MemTags] Analysis completed (%u trackers, %u tag sets, %d tags).", NumTrackers, NumTagSets, AvailableTags.Num());
}

uint32 FMemoryProvider::GetTagSerial() const
{
	ReadAccessCheck();
	return TagsSerial;
}

uint32 FMemoryProvider::GetTagCount() const
{
	ReadAccessCheck();
	return AvailableTags.Num();
}

void FMemoryProvider::EnumerateTags(TFunctionRef<void(const FMemoryTagInfo&)> Callback) const
{
	ReadAccessCheck();

	for (const FMemoryTagInfo* Tag : AvailableTags)
	{
		Callback(*Tag);
	}
}

void FMemoryProvider::EnumerateTags(FMemoryTagSetId TagSetId, TFunctionRef<void(const FMemoryTagInfo&)> Callback) const
{
	ReadAccessCheck();

	FTagSetData* TagSet = GetTagSet(TagSetId);
	if (!TagSet)
	{
		return;
	}

	for (const auto& KV : TagSet->Tags)
	{
		Callback(*KV.Value);
	}
}

const FMemoryTagInfo* FMemoryProvider::GetTag(FMemoryTagId TagId) const
{
	ReadAccessCheck();
	return TagMap.FindRef(TagId);
}

uint32 FMemoryProvider::GetTrackerCount() const
{
	ReadAccessCheck();
	return NumTrackers;
}

void FMemoryProvider::EnumerateTrackers(TFunctionRef<void(const FMemoryTrackerInfo&)> Callback) const
{
	ReadAccessCheck();

	for (const FTrackerData* Tracker : AvailableTrackers)
	{
		if (Tracker)
		{
			Callback(*Tracker->Info);
		}
	}
}

uint32 FMemoryProvider::GetTagSetCount() const
{
	ReadAccessCheck();
	return NumTagSets;
}

void FMemoryProvider::EnumerateTagSets(TFunctionRef<void(const FMemoryTagSetInfo&)> Callback) const
{
	ReadAccessCheck();

	for (const FTagSetData* TagSet : AvailableTagSets)
	{
		if (TagSet)
		{
			Callback(*TagSet->Info);
		}
	}
}

uint64 FMemoryProvider::GetTagSampleCount(FMemoryTrackerId TrackerId, FMemoryTagId TagId) const
{
	ReadAccessCheck();

	FTrackerData* Tracker = GetTracker(TrackerId);
	if (!Tracker)
	{
		// invalid tracker id
		return 0;
	}

	const FTagSampleData* TagSamples = Tracker->Samples.FindRef(TagId);
	if (!TagSamples)
	{
		// invalid tag id
		return 0;
	}

	return TagSamples->Values.Num();
}

void FMemoryProvider::EnumerateTagSamples(FMemoryTrackerId TrackerId, FMemoryTagId TagId, double StartTime, double EndTime, bool bIncludeRangeNeighbours, TFunctionRef<void(double Time, double Duration, const FMemoryTagSample&)> Callback) const
{
	ReadAccessCheck();

	FTrackerData* Tracker = GetTracker(TrackerId);
	if (!Tracker)
	{
		// invalid tracker id
		return;
	}

	const FTagSampleData* TagSamples = Tracker->Samples.FindRef(TagId);
	if (!TagSamples)
	{
		// invalid tag id
		return;
	}

	const TPagedArray<FMemoryTagSample>& SampleValues = TagSamples->Values;

	int32 IndexStart = (int32)PagedArrayAlgo::LowerBound(SnapshotTimes, StartTime);
	int32 IndexEnd = (int32)PagedArrayAlgo::UpperBound(SnapshotTimes, EndTime);

	if (bIncludeRangeNeighbours)
	{
		IndexStart = FMath::Max(IndexStart - 1, 0);
		IndexEnd = FMath::Min(IndexEnd + 1, static_cast<int32>(SampleValues.Num()));
	}
	else
	{
		IndexStart = FMath::Max(IndexStart, 0);
		IndexEnd = FMath::Min(IndexEnd, static_cast<int32>(SampleValues.Num()));
	}

	if (IndexStart >= IndexEnd)
	{
		return;
	}

	auto TimeIt = SnapshotTimes.GetIteratorFromItem(IndexStart);
	auto ValueIt = SampleValues.GetIteratorFromItem(IndexStart);

	double Time = *TimeIt;
	for (int32 SampleIndex = IndexStart; SampleIndex < IndexEnd; ++SampleIndex)
	{
		if (TimeIt.NextItem())
		{
			double NextTime = *TimeIt;
			Callback(Time, NextTime - Time, SampleValues[SampleIndex]);
			Time = NextTime;
		}
		else
		{
			// Last sample has 0 duration.
			Callback(Time, 0.0, *ValueIt);
		}
	}
}

FName GetMemoryProviderName()
{
	static const FName Name("MemoryProvider");
	return Name;
}

const IMemoryProvider* ReadMemoryProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IMemoryProvider>(GetMemoryProviderName());
}

} // namespace TraceServices
