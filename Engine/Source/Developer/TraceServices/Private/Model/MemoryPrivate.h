// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"

#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Memory.h"

namespace TraceServices
{

extern thread_local FProviderLock::FThreadLocalState GMemoryProviderLockState;

class FMemoryProvider : public IMemoryProvider, public IEditableProvider
{
public:
	explicit FMemoryProvider(IAnalysisSession& Session);
	virtual ~FMemoryProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override { Lock.BeginRead(GMemoryProviderLockState); }
	virtual void EndRead() const override { Lock.EndRead(GMemoryProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GMemoryProviderLockState); }

	virtual bool IsInitialized() const override { return bIsInitialized; }
	virtual bool IsCompleted() const override { return bIsCompleted; }

	virtual uint32 GetTagSerial() const override;
	virtual uint32 GetTagCount() const override;
	virtual const FMemoryTagInfo* GetTag(FMemoryTagId Id) const override;

	virtual void EnumerateTags(TFunctionRef<void(const FMemoryTagInfo&)> Callback) const override;
	virtual void EnumerateTags(FMemoryTagSetId TagSetId, TFunctionRef<void(const FMemoryTagInfo&)> Callback) const override;

	virtual uint32 GetTrackerCount() const override;
	virtual void EnumerateTrackers(TFunctionRef<void(const FMemoryTrackerInfo&)> Callback) const override;

	virtual uint32 GetTagSetCount() const override;
	virtual void EnumerateTagSets(TFunctionRef<void(const FMemoryTagSetInfo&)> Callback) const override;

	virtual uint64 GetTagSampleCount(FMemoryTrackerId Tracker, FMemoryTagId Tag) const override;
	virtual void EnumerateTagSamples(
		FMemoryTrackerId Tracker,
		FMemoryTagId Tag,
		double StartTime,
		double EnddTime,
		bool bIncludeRangeNeighbours,
		TFunctionRef<void(double Time, double Duration, const FMemoryTagSample&)> Callback) const override;

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override { Lock.BeginWrite(GMemoryProviderLockState); }
	virtual void EndEdit() const override { Lock.EndWrite(GMemoryProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GMemoryProviderLockState); }

	void AddTrackerSpec(FMemoryTrackerId TrackerId, const FString& Name);
	void AddTagSetSpec(FMemoryTagSetId TagSetId, const FString& Name);
	void AddTagSpec(FMemoryTagId Tag, const FString& Name, FMemoryTagId ParentTag, FMemoryTagSetId TagSet);

	void AddTagSnapshot(FMemoryTrackerId TrackerId, double Time, const int64* Tags, const FMemoryTagSample* Values, uint32 TagCount);

	void OnAnalysisCompleted();

	//////////////////////////////////////////////////

private:
	struct FTagSampleData
	{
		// Sample values.
		TPagedArray<FMemoryTagSample> Values;

		FMemoryTagInfo* TagInfo = nullptr;

		FTagSampleData(ILinearAllocator& Allocator)
			: Values(Allocator, 65536)
		{}
		FTagSampleData(const FTagSampleData& Other)
			: Values(Other.Values)
		{}
	};

	struct FTrackerData
	{
		// The memory tracker.
		FMemoryTrackerInfo* Info = nullptr;

		// All tags and samples for this tracker.
		TMap<FMemoryTagId, FTagSampleData*> Samples;

		FTrackerData(FMemoryTrackerInfo* InInfo) : Info(InInfo) {}

	private:
		FTrackerData(const FTrackerData&) = delete;
	};

	struct FTagSetData
	{
		// The memory tag set.
		FMemoryTagSetInfo* Info = nullptr;

		// All tags for this tag set.
		TMap<FMemoryTagId, FMemoryTagInfo*> Tags; // tag id --> TagInfo

		FTagSetData(FMemoryTagSetInfo* InInfo) : Info(InInfo) {}

	private:
		FTagSetData(const FTagSetData&) = delete;
	};

private:
	void InternalAddTrackerSpec(FMemoryTrackerId TrackerId, const FString& Name);
	void InternalAddTagSetSpec(FMemoryTagSetId TagSetId, const FString& Name);
	void InternalAddTagSpec(FMemoryTagId Tag, const FString& Name, FMemoryTagId ParentTag, FMemoryTagSetId TagSet);
	FTrackerData* GetTracker(FMemoryTrackerId TrackerId) const;
	FTrackerData* GetOrAddTracker(FMemoryTrackerId TrackerId);
	FTagSetData* GetTagSet(FMemoryTagSetId TagSetId) const;
	FTagSetData* GetOrAddTagSet(FMemoryTagSetId TagSetId);

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;

	TArray<FTrackerData*> AvailableTrackers;
	uint32 NumTrackers = 0;

	TArray<FTagSetData*> AvailableTagSets;
	uint32 NumTagSets = 0;

	TArray<FMemoryTagInfo*> AvailableTags;
	TMap<FMemoryTagId, FMemoryTagInfo*> TagMap; // tag id --> TagInfo

	// Timestamps for samples
	TPagedArray<double> SnapshotTimes;

	uint32 TagsSerial = 0;

	std::atomic<bool> bIsInitialized = false;
	std::atomic<bool> bIsCompleted = false;
};

} // namespace TraceServices
