// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetadataProvider.h"

#include "Common/Utils.h"

#define METADATA_PROVIDER_DEBUG_LOG(Thread, Fmt, ...) //{ if (Thread == 64108) { UE_LOG(LogTraceServices, Warning, Fmt, __VA_ARGS__); } }
#define METADATA_PROVIDER_ERROR(ErrorCount, Fmt, ...) { if (++ErrorCount <= MaxLogMessagesPerErrorType) { UE_LOG(LogTraceServices, Error, Fmt, __VA_ARGS__); } }

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GMetadataProviderLockState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMetadataProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataProvider::FMetadataProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, RegisteredTypes(Session.GetLinearAllocator(), 16)
	, MetadataStore(Session.GetLinearAllocator(), 1024)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataProvider::~FMetadataProvider()
{
	for (const auto& KV : Threads)
	{
		for (const FMetadataStackEntry& StackEntry : KV.Value->CurrentStack)
		{
			if (StackEntry.StoreIndex == InvalidMetadataStoreIndex)
			{
				InternalFreeStoreEntry(StackEntry.StoreEntry);
			}
		}
		delete KV.Value;
	}

	if (MetadataStore.Num() > 0)
	{
		for (const FMetadataStoreEntry& StoreEntry : MetadataStore)
		{
			InternalFreeStoreEntry(StoreEntry);
		}
	}

	check(AllocationCount == 0);
	check(TotalAllocatedMemory == 0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint16 FMetadataProvider::RegisterMetadataType(const TCHAR* InName, const FMetadataSchema& InSchema)
{
	EditAccessCheck();

	check(RegisteredTypes.Num() <= MaxMetadataTypeId);
	const uint16 Type = static_cast<uint16>(RegisteredTypes.Num());
	RegisteredTypes.EmplaceBack(InSchema);
	RegisteredTypesMap.Add(InName, Type);

	return Type;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint16 FMetadataProvider::GetRegisteredMetadataType(FName InName) const
{
	ReadAccessCheck();

	const uint16* TypePtr = RegisteredTypesMap.Find(InName);
	return TypePtr ? *TypePtr : InvalidMetadataType;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName FMetadataProvider::GetRegisteredMetadataName(uint16 InType) const
{
	ReadAccessCheck();

	const FName* Name = RegisteredTypesMap.FindKey(InType);
	return Name ? *Name : FName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMetadataSchema* FMetadataProvider::GetRegisteredMetadataSchema(uint16 InType) const
{
	ReadAccessCheck();

	const int32 Index = (int32)InType;
	return Index < RegisteredTypes.Num() ? &RegisteredTypes[Index] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataProvider::FMetadataThread::FMetadataThread(uint32 InThreadId, ILinearAllocator& InAllocator)
	: ThreadId(InThreadId)
	, CurrentStack()
	, Metadata(InAllocator, 1024)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataProvider::FMetadataThread& FMetadataProvider::GetOrAddThread(uint32 InThreadId)
{
	FMetadataThread** MetadataThreadPtr = Threads.Find(InThreadId);
	if (MetadataThreadPtr)
	{
		return **MetadataThreadPtr;
	}
	FMetadataThread* MetadataThread = new FMetadataThread(InThreadId, Session.GetLinearAllocator());
	Threads.Add(InThreadId, MetadataThread);
	return *MetadataThread;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMetadataProvider::FMetadataThread* FMetadataProvider::GetThread(uint32 InThreadId)
{
	FMetadataThread** MetadataThreadPtr = Threads.Find(InThreadId);
	return (MetadataThreadPtr) ? *MetadataThreadPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMetadataProvider::FMetadataThread* FMetadataProvider::GetThread(uint32 InThreadId) const
{
	FMetadataThread* const* MetadataThreadPtr = Threads.Find(InThreadId);
	return (MetadataThreadPtr) ? *MetadataThreadPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::InternalAllocStoreEntry(FMetadataStoreEntry& InStoreEntry, uint16 InType, const void* InData, uint32 InSize)
{
	if (InSize > MaxInlinedMetadataSize)
	{
		InStoreEntry.Ptr = FMemory::Malloc(InSize);
		++AllocationEventCount;
		++AllocationCount;
		TotalAllocatedMemory += InSize;
		FMemory::Memcpy(InStoreEntry.Ptr, InData, InSize);
	}
	else
	{
		FMemory::Memcpy(InStoreEntry.Value, InData, InSize);
	}
	InStoreEntry.Size = static_cast<uint16>(InSize);
	InStoreEntry.Type = InType;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::InternalFreeStoreEntry(const FMetadataStoreEntry& InStoreEntry)
{
	if (InStoreEntry.Size > MaxInlinedMetadataSize)
	{
		FMemory::Free(InStoreEntry.Ptr);
		check(AllocationCount > 0);
		--AllocationCount;
		check(TotalAllocatedMemory >= InStoreEntry.Size);
		TotalAllocatedMemory -= InStoreEntry.Size;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::InternalPushStackEntry(FMetadataThread& InMetadataThread, uint16 InType, const void* InData, uint32 InSize)
{
	FMetadataStackEntry StackEntry;
	InternalAllocStoreEntry(StackEntry.StoreEntry, InType, InData, InSize);
	StackEntry.StoreIndex = InvalidMetadataStoreIndex;
	StackEntry.PinnedId = InvalidMetadataId;
	InMetadataThread.CurrentStack.Push(StackEntry);

	METADATA_PROVIDER_DEBUG_LOG(InMetadataThread.ThreadId, TEXT("  --> push stack : size %d"), InMetadataThread.CurrentStack.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::InternalPopStackEntry(FMetadataThread& InMetadataThread)
{
	const FMetadataStackEntry& Top = InMetadataThread.CurrentStack.Top();
	if (Top.StoreIndex == InvalidMetadataStoreIndex)
	{
		InternalFreeStoreEntry(Top.StoreEntry);
	}
	InMetadataThread.CurrentStack.Pop();

	METADATA_PROVIDER_DEBUG_LOG(InMetadataThread.ThreadId, TEXT("  --> pop stack : size %d"), InMetadataThread.CurrentStack.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::InternalClearStack(FMetadataThread& InMetadataThread)
{
	METADATA_PROVIDER_DEBUG_LOG(InMetadataThread.ThreadId, TEXT("  --> clear stack of %d"), InMetadataThread.CurrentStack.Num());

	while (InMetadataThread.CurrentStack.Num() > 0)
	{
		InternalPopStackEntry(InMetadataThread);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::InternalPushSavedStack(FMetadataThread& InMetadataThread, FMetadataThread& InSavedMetadataThread, uint32 InSavedMetadataId)
{
	METADATA_PROVIDER_DEBUG_LOG(InMetadataThread.ThreadId, TEXT("  --> push metadata %u (from thread %u)"), InSavedMetadataId, InSavedMetadataThread.ThreadId);

	auto Iterator = InSavedMetadataThread.Metadata.GetIteratorFromItem(InSavedMetadataId);
	const FMetadataEntry* Entry = Iterator.GetCurrentItem();
	while (true)
	{
		check(Entry->StoreIndex < MetadataStore.Num());
		const FMetadataStoreEntry& StoreEntry = *MetadataStore.GetIteratorFromItem(Entry->StoreIndex);
		const void* Data = (StoreEntry.Size > MaxInlinedMetadataSize) ? StoreEntry.Ptr : StoreEntry.Value;

		if (InMetadataThread.CurrentStack.Num() == MaxMetadataStackSize)
		{
			METADATA_PROVIDER_ERROR(PushSavedStackErrors, TEXT("[Meta] Cannot push saved metadata %u (from thread %u) to thread %u. Stack size is too large."), InSavedMetadataId, InSavedMetadataThread.ThreadId, InMetadataThread.ThreadId);
			break;
		}

		InternalPushStackEntry(InMetadataThread, StoreEntry.Type, Data, StoreEntry.Size);

		if (Entry->StackSize == 1) // last one
		{
			break;
		}
		Entry = Iterator.PrevItem();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::PushScopedMetadata(uint32 InThreadId, uint16 InType, const void* InData, uint32 InSize)
{
	EditAccessCheck();
	++EventCount;

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("PushScopedMetadata(type=%u, size=%u)"), uint32(InType), InSize);

	check(InType < RegisteredTypes.Num());
	check(InData != nullptr && InSize > 0);

	if (InSize > MaxMetadataSize)
	{
		METADATA_PROVIDER_ERROR(MetaScopeErrors, TEXT("[Meta] Cannot push metadata (thread %u, type %u, size %u). Data size is too large."), InThreadId, InType, InSize);
		return;
	}

	FMetadataThread& MetadataThread = GetOrAddThread(InThreadId);

	if (MetadataThread.CurrentStack.Num() == MaxMetadataStackSize)
	{
		METADATA_PROVIDER_ERROR(MetaScopeErrors, TEXT("[Meta] Cannot push metadata (thread %u, type %u). Stack size is too large."), InThreadId, InType);
		return;
	}

	InternalPushStackEntry(MetadataThread, InType, InData, InSize);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::PopScopedMetadata(uint32 InThreadId, uint16 InType)
{
	EditAccessCheck();
	++EventCount;

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("PopScopedMetadata(type=%u)"), uint32(InType));

	FMetadataThread& MetadataThread = GetOrAddThread(InThreadId);

	if (MetadataThread.CurrentStack.Num() == 0)
	{
		METADATA_PROVIDER_ERROR(MetaScopeErrors, TEXT("[Meta] Cannot pop metadata (thread %u, type %u). Stack is empty."), InThreadId, uint32(InType));
		return;
	}

	const FMetadataStackEntry& Top = MetadataThread.CurrentStack.Top();

	if (Top.StoreEntry.Type != InType)
	{
		METADATA_PROVIDER_ERROR(MetaScopeErrors, TEXT("[Meta] Cannot pop metadata (thread %u, type %u). Type mismatch."), InThreadId, uint32(InType));
		return;
	}

	InternalPopStackEntry(MetadataThread);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::BeginClearStackScope(uint32 InThreadId)
{
	EditAccessCheck();
	++EventCount;

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("BeginClearStackScope()"));

	FMetadataThread& MetadataThread = GetOrAddThread(InThreadId);

	// Save the current stack.
	const uint32 SavedMetadataId = PinAndGetId(InThreadId);
	MetadataThread.SavedMetadataStack.Push(SavedMetadataId);

	if (SavedMetadataId != InvalidMetadataId)
	{
		METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("  --> id = %u"), SavedMetadataId);
	}

	// Clear the current stack.
	InternalClearStack(MetadataThread);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::EndClearStackScope(uint32 InThreadId)
{
	EditAccessCheck();
	++EventCount;

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("EndClearStackScope()"));

	FMetadataThread& MetadataThread = GetOrAddThread(InThreadId);
	if (MetadataThread.SavedMetadataStack.IsEmpty())
	{
		METADATA_PROVIDER_ERROR(ClearScopeErrors, TEXT("[Meta] Cannot end clear stack (thread %u). Clear scope stack is already empty."), InThreadId);
		return;
	}

	if (MetadataThread.CurrentStack.Num() > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[Meta] Invalid end clear stack event (thread %u). Current metadata stack is not empty.", InThreadId);
	}

	const uint32 SavedMetadataId = MetadataThread.SavedMetadataStack.Pop();

	if (SavedMetadataId == InvalidMetadataId)
	{
		METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("  --> empty"));

		// Empty stack. Nothing to push.
		return;
	}

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("  --> id = %u"), SavedMetadataId);

	if (SavedMetadataId >= MetadataThread.Metadata.Num())
	{
		METADATA_PROVIDER_ERROR(ClearScopeErrors, TEXT("[Meta] Cannot end clear stack (thread %u). Invalid saved metadata."), InThreadId);
		return;
	}

	// Push the saved stack.
	InternalPushSavedStack(MetadataThread, MetadataThread, SavedMetadataId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::SaveStack(uint32 InThreadId, uint32 InSavedStackId)
{
	EditAccessCheck();
	++EventCount;

	const uint32 MetadataId = PinAndGetId(InThreadId);
	SavedStackMap.Add(InSavedStackId, { InThreadId, MetadataId });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::BeginRestoreSavedStackScope(uint32 InThreadId, uint32 InSavedStackId)
{
	EditAccessCheck();
	++EventCount;

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("BeginRestoreSavedStackScope()"));

	FMetadataThread& MetadataThread = GetOrAddThread(InThreadId);

	// Save the current stack.
	const uint32 SavedMetadataId = PinAndGetId(InThreadId);
	MetadataThread.SavedMetadataStack.Push(SavedMetadataId);

	if (SavedMetadataId != InvalidMetadataId)
	{
		METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("  --> id = %u"), SavedMetadataId);
	}

	// Clear the current stack.
	InternalClearStack(MetadataThread);

	const FMetadataSavedStackInfo* SavedStackInfo = SavedStackMap.Find(InSavedStackId);
	if (!SavedStackInfo)
	{
		METADATA_PROVIDER_ERROR(RestoreScopeErrors, TEXT("[Meta] Cannot begin restore saved stack (thread %u, id %u). Invalid saved stack."), InThreadId, InSavedStackId);
		return;
	}

	if (SavedStackInfo->MetadataId == InvalidMetadataId)
	{
		// Empty stack. Nothing to push.
		return;
	}

	FMetadataThread* SavedMetadataThread = GetThread(SavedStackInfo->ThreadId);

	if (!SavedMetadataThread || SavedStackInfo->MetadataId >= SavedMetadataThread->Metadata.Num())
	{
		METADATA_PROVIDER_ERROR(RestoreScopeErrors, TEXT("[Meta] Cannot begin restore saved stack (thread %u, id %u). Invalid saved metadata."), InThreadId, InSavedStackId);
		return;
	}

	// Push the previously saved stack.
	InternalPushSavedStack(MetadataThread, *SavedMetadataThread, SavedStackInfo->MetadataId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::EndRestoreSavedStackScope(uint32 InThreadId)
{
	EditAccessCheck();
	++EventCount;

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("EndRestoreSavedStackScope()"));

	FMetadataThread& MetadataThread = GetOrAddThread(InThreadId);
	if (MetadataThread.SavedMetadataStack.IsEmpty())
	{
		METADATA_PROVIDER_ERROR(RestoreScopeErrors, TEXT("[Meta] Cannot end restore saved stack (thread %u). Restore scope stack is already empty."), InThreadId);
		return;
	}

	// Clears the restored saved stack.
	InternalClearStack(MetadataThread);

	const uint32 SavedMetadataId = MetadataThread.SavedMetadataStack.Pop();

	if (SavedMetadataId == InvalidMetadataId)
	{
		METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("  --> empty"));

		// Empty stack. Nothing to push.
		return;
	}

	METADATA_PROVIDER_DEBUG_LOG(InThreadId, TEXT("  --> id = %u"), SavedMetadataId);

	if (SavedMetadataId >= MetadataThread.Metadata.Num())
	{
		METADATA_PROVIDER_ERROR(ClearScopeErrors, TEXT("[Meta] Cannot end restore saved stack (thread %u). Invalid saved metadata."), InThreadId);
		return;
	}

	// Push the saved stack.
	InternalPushSavedStack(MetadataThread, MetadataThread, SavedMetadataId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FMetadataProvider::PinAndGetId(uint32 InThreadId)
{
	EditAccessCheck();

	FMetadataThread* MetadataThread = GetThread(InThreadId);

	if (!MetadataThread || MetadataThread->CurrentStack.Num() == 0)
	{
		return InvalidMetadataId;
	}

	TArray<FMetadataStackEntry>& CurrentStack = MetadataThread->CurrentStack;
	TPagedArray<FMetadataEntry>& Metadata = MetadataThread->Metadata;

	const uint32 LastPinnedId = CurrentStack.Top().PinnedId;
	if (LastPinnedId != InvalidMetadataId)
	{
		return LastPinnedId;
	}

	const int32 StackSize = CurrentStack.Num();

	// Store all entries of the current stack.
	for (int32 StackIndex = StackSize - 1; StackIndex >= 0; --StackIndex)
	{
		FMetadataStackEntry& StackEntry = CurrentStack[StackIndex];
		if (StackEntry.PinnedId != InvalidMetadataId)
		{
			break;
		}
		if (StackEntry.StoreIndex == InvalidMetadataStoreIndex)
		{
			StackEntry.StoreIndex = static_cast<uint32>(MetadataStore.Num());
			MetadataStore.EmplaceBack(StackEntry.StoreEntry);
		}
	}

	// Check if we can reuse the last pinned stack.
	int32 LastPinnedStackIndex = -1;
	if (Metadata.Num() > 0)
	{
		const uint32 LastPinnedMetadataId = static_cast<uint32>(Metadata.Num()) - 1;
		for (int32 StackIndex = StackSize - 2; StackIndex >= 0; --StackIndex)
		{
			const FMetadataStackEntry& StackEntry = CurrentStack[StackIndex];
			if (StackEntry.PinnedId != InvalidMetadataId)
			{
				if (StackEntry.PinnedId == LastPinnedMetadataId)
				{
					// We can reuse the last pinned metadata stack.
					// In this case we pin only the additional entries.
					LastPinnedStackIndex = StackIndex;
				}
				break;
			}
		}
	}

	// Pin metadata entries in the current stack (reusing last pinned stack if possible).
	// It needs to be done in increasing stack index order.
	// Each entry gives the size of the partial stack (previous entries).
	for (int32 StackIndex = LastPinnedStackIndex + 1; StackIndex < StackSize; ++StackIndex)
	{
		FMetadataStackEntry& StackEntry = CurrentStack[StackIndex];
		StackEntry.PinnedId = static_cast<uint32>(Metadata.Num());
		FMetadataEntry& Entry = Metadata.PushBack();
		Entry.StoreIndex = StackEntry.StoreIndex;
		Entry.Type = StackEntry.StoreEntry.Type;
		Entry.StackSize = static_cast<uint16>(StackIndex + 1);
	}

	return CurrentStack.Top().PinnedId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::OnAnalysisCompleted()
{
	EditAccessCheck();

	if (MetaScopeErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[Meta] Meta scope errors: %u", MetaScopeErrors);
	}
	if (ClearScopeErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[Meta] Clear scope errors: %u", ClearScopeErrors);
	}
	if (RestoreScopeErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[Meta] Restore scope errors: %u", RestoreScopeErrors);
	}

	UE_LOGF(LogTraceServices, Log, "[Meta] Analysis completed (%llu events, %llu alloc events, %llu allocs, %llu bytes).",
		EventCount, AllocationEventCount, AllocationCount, TotalAllocatedMemory);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FMetadataProvider::GetMetadataStackSize(uint32 InThreadId, uint32 InMetadataId) const
{
	ReadAccessCheck();

	const FMetadataThread* MetadataThread = GetThread(InThreadId);

	if (!MetadataThread || InMetadataId >= MetadataThread->Metadata.Num())
	{
		UE_LOGF(LogTraceServices, Warning, "[Meta] Invalid metadata id (thread %u, id %u).", InThreadId, InMetadataId);
		return 0;
	}

	auto Iterator = MetadataThread->Metadata.GetIteratorFromItem(InMetadataId);
	const FMetadataEntry* Entry = Iterator.GetCurrentItem();
	return Entry->StackSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMetadataProvider::GetMetadata(uint32 InThreadId, uint32 InMetadataId, uint32 InStackDepth, uint16& OutType, const void*& OutData, uint32& OutSize) const
{
	ReadAccessCheck();

	const FMetadataThread* MetadataThread = GetThread(InThreadId);

	if (!MetadataThread || InMetadataId >= MetadataThread->Metadata.Num())
	{
		UE_LOGF(LogTraceServices, Warning, "[Meta] Invalid metadata id (thread %u, id %u).", InThreadId, InMetadataId);
		OutType = InvalidMetadataType;
		OutData = nullptr;
		OutSize = 0;
		return false;
	}

	auto Iterator = MetadataThread->Metadata.GetIteratorFromItem(InMetadataId);
	const FMetadataEntry* Entry = Iterator.GetCurrentItem();
	if (InStackDepth >= Entry->StackSize)
	{
		UE_LOGF(LogTraceServices, Warning, "[Meta] Invalid metadata stack detph (thread %u, id %u, depth %u).", InThreadId, InMetadataId, InStackDepth);
		OutType = InvalidMetadataType;
		OutData = nullptr;
		OutSize = 0;
		return false;
	}
	while (Entry->StackSize != InStackDepth + 1)
	{
		Entry = Iterator.PrevItem();
		check(Entry != nullptr);
	}

	check(Entry->StoreIndex < MetadataStore.Num());
	const FMetadataStoreEntry& StoreEntry = *MetadataStore.GetIteratorFromItem(Entry->StoreIndex);
	OutType = StoreEntry.Type;
	OutData = (StoreEntry.Size > MaxInlinedMetadataSize) ? StoreEntry.Ptr : StoreEntry.Value;
	OutSize = StoreEntry.Size;
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataProvider::EnumerateMetadata(uint32 InThreadId, uint32 InMetadataId, TFunctionRef<bool(uint32 StackDepth, uint16 Type, const void* Data, uint32 Size)> Callback) const
{
	ReadAccessCheck();

	const FMetadataThread* MetadataThread = GetThread(InThreadId);

	if (!MetadataThread || InMetadataId >= MetadataThread->Metadata.Num())
	{
		UE_LOGF(LogTraceServices, Warning, "[Meta] Invalid metadata id (thread %u, id %u).", InThreadId, InMetadataId);
		return;
	}

	auto Iterator = MetadataThread->Metadata.GetIteratorFromItem(InMetadataId);
	const FMetadataEntry* Entry = Iterator.GetCurrentItem();
	while (true)
	{
		check(Entry->StoreIndex < MetadataStore.Num());
		const FMetadataStoreEntry& StoreEntry = *MetadataStore.GetIteratorFromItem(Entry->StoreIndex);
		const void* Data = (StoreEntry.Size > MaxInlinedMetadataSize) ? StoreEntry.Ptr : StoreEntry.Value;
		if (!Callback(Entry->StackSize - 1, StoreEntry.Type, Data, StoreEntry.Size))
		{
			break;
		}
		if (Entry->StackSize == 1) // last one
		{
			break;
		}
		Entry = Iterator.PrevItem();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName GetMetadataProviderName()
{
	static const FName Name("MetadataProvider");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IMetadataProvider* ReadMetadataProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IMetadataProvider>(GetMetadataProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IEditableMetadataProvider* EditMetadataProvider(IAnalysisSession& Session)
{
	return Session.EditProvider<IEditableMetadataProvider>(GetMetadataProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef METADATA_PROVIDER_ERROR
#undef METADATA_PROVIDER_DEBUG_LOG
