// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/MetadataProvider.h"

namespace TraceServices
{

class IAnalysisSession;
class ILinearAllocator;

extern thread_local FProviderLock::FThreadLocalState GMetadataProviderLockState;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMetadataProviderLock
{
public:
	void ReadAccessCheck() const;
	void WriteAccessCheck() const;

	void BeginRead();
	void EndRead();

	void BeginWrite();
	void EndWrite();

private:
	FRWLock RWLock;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMetadataProvider
	: public IMetadataProvider
	, public IEditableMetadataProvider
{
public:
	static constexpr uint32 MaxMetadataSize = 0xFFFF;
	static constexpr uint32 MaxLogMessagesPerErrorType = 100;

private:
	static constexpr uint32 MaxInlinedMetadataSize = 12;
	static constexpr uint32 MaxMetadataTypeId = 0xFFFF;
	static constexpr int32 MaxMetadataStackSize = 0xFFFF;
	static constexpr uint32 InvalidMetadataStoreIndex = 0xFFFFFFFF;

	struct FMetadataStoreEntry
	{
		union
		{
			void* Ptr;
			uint8 Value[8];
		};
		uint8 ValueEx[4];
		uint16 Size;
		uint16 Type;
	};
	static_assert(sizeof(FMetadataStoreEntry) == 16, "sizeof(FMetadataStoreEntry)");

	struct FMetadataStackEntry
	{
		FMetadataStoreEntry StoreEntry;
		uint32 StoreIndex;
		uint32 PinnedId;
	};
	static_assert(sizeof(FMetadataStackEntry) == 24, "sizeof(FMetadataStackEntry)");

	struct FMetadataEntry
	{
		uint32 StoreIndex;
		uint16 Type;
		uint16 StackSize;
	};
	static_assert(sizeof(FMetadataEntry) == 8, "sizeof(FMetadataEntry)");

	struct FMetadataThread
	{
		FMetadataThread(uint32 InThreadId, ILinearAllocator&);

		uint32 ThreadId;
		TArray<FMetadataStackEntry> CurrentStack;
		TPagedArray<FMetadataEntry> Metadata; // a metadata id is an index in this array
		TArray<uint32> SavedMetadataStack; // stack of MetadataId values for each saved Clear or Restore scope
	};

	struct FMetadataSavedStackInfo
	{
		uint32 ThreadId;
		uint32 MetadataId;
	};

public:
	explicit FMetadataProvider(IAnalysisSession& InSession);
	virtual ~FMetadataProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override       { Lock.BeginRead(GMetadataProviderLockState); }
	virtual void EndRead() const override         { Lock.EndRead(GMetadataProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GMetadataProviderLockState); }

	virtual uint16 GetRegisteredMetadataType(FName InName) const override;
	virtual FName GetRegisteredMetadataName(uint16 InType) const override;
	virtual const FMetadataSchema* GetRegisteredMetadataSchema(uint16) const override;

	virtual uint32 GetMetadataStackSize(uint32 InThreadId, uint32 InMetadataId) const override;
	virtual bool GetMetadata(uint32 InThreadId, uint32 InMetadataId, uint32 InStackDepth, uint16& OutType, const void*& OutData, uint32& OutSize) const override;
	virtual void EnumerateMetadata(uint32 InThreadId, uint32 InMetadataId, TFunctionRef<bool(uint32 StackDepth, uint16 Type, const void* Data, uint32 Size)> Callback) const override;

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override       { Lock.BeginWrite(GMetadataProviderLockState); }
	virtual void EndEdit() const override         { Lock.EndWrite(GMetadataProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GMetadataProviderLockState); }

	virtual uint16 RegisterMetadataType(const TCHAR* InName, const FMetadataSchema& InSchema) override;

	virtual void PushScopedMetadata(uint32 InThreadId, uint16 InType, const void* InData, uint32 InSize) override;
	virtual void PopScopedMetadata(uint32 InThreadId, uint16 InType) override;

	virtual void BeginClearStackScope(uint32 InThreadId) override;
	virtual void EndClearStackScope(uint32 InThreadId) override;

	virtual void SaveStack(uint32 InThreadId, uint32 InSavedStackId) override;
	virtual void BeginRestoreSavedStackScope(uint32 InThreadId, uint32 InSavedStackId) override;
	virtual void EndRestoreSavedStackScope(uint32 InThreadId) override;

	// Pins the metadata stack and returns an id for it.
	virtual uint32 PinAndGetId(uint32 InThreadId) override;

	virtual void OnAnalysisCompleted() override;

	//////////////////////////////////////////////////

private:
	FMetadataThread& GetOrAddThread(uint32 InThreadId);
	FMetadataThread* GetThread(uint32 InThreadId);
	const FMetadataThread* GetThread(uint32 InThreadId) const;
	void InternalAllocStoreEntry(FMetadataStoreEntry& InStoreEntry, uint16 InType, const void* InData, uint32 InSize);
	void InternalFreeStoreEntry(const FMetadataStoreEntry& InStoreEntry);
	void InternalPushStackEntry(FMetadataThread& InMetadataThread, uint16 InType, const void* InData, uint32 InSize);
	void InternalPopStackEntry(FMetadataThread& InMetadataThread);
	void InternalClearStack(FMetadataThread& InMetadataThread);
	void InternalPushSavedStack(FMetadataThread& InMetadataThread, FMetadataThread& InSavedMetadataThread, uint32 InSavedMetadataId);

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;

	TPagedArray<FMetadataSchema> RegisteredTypes;
	TMap<FName, uint16> RegisteredTypesMap;

	TPagedArray<FMetadataStoreEntry> MetadataStore; // stores individual metadata values

	TMap<uint32, FMetadataThread*> Threads;

	TMap<uint32, FMetadataSavedStackInfo> SavedStackMap; // SavedStackId --> (ThreadId, MetadataId)

	uint32 MetaScopeErrors = 0; // debug
	uint32 ClearScopeErrors = 0; // debug
	uint32 RestoreScopeErrors = 0; // debug
	uint32 PushSavedStackErrors = 0; // debug

	uint64 EventCount = 0; // debug
	uint64 AllocationEventCount = 0; // debug
	uint64 AllocationCount = 0; // debug
	uint64 TotalAllocatedMemory = 0; // debug
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
