// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModuleProvider.h"
#include <atomic>

#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Async/ParallelFor.h"
#include "Common/CachedPagedArray.h"
#include "Common/CachedStringStore.h"
#include "Common/Utils.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeRWLock.h"
#include "TraceServices/Model/AnalysisCache.h"
#include "TraceServices/Model/AnalysisSession.h"

// Choose which symbol resolver to use.
// If both are enabled and the RAD Syms library fails to initialize, it will fall back to DbgHelp (on Windows).
#define USE_SYMSLIB 1
#define USE_DBGHELP 1
// Psym resolver supports symbols in the breakpad cross-platform text format
#define USE_PSYMRESOLVER 1

// Symbol files implementations
#if USE_SYMSLIB
#include "SymslibResolver.h"
#endif
#if USE_DBGHELP
#include "DbgHelpResolver.h"
#endif
#if USE_PSYMRESOLVER
#include "PsymResolver.h"
#endif

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FResolvedSymbolFilter : public IResolvedSymbolFilter
{
public:
	FResolvedSymbolFilter();
	virtual ~FResolvedSymbolFilter();

	virtual void Update(FResolvedSymbol& InSymbol) const override;

private:
	TArray<FStringView> IgnoreSymbolsByModuleName;
	TArray<FStringView> IgnoreSymbolsByFunctionName;
	TArray<FStringView> IgnoreSymbolsByFilePath;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
class TModuleProvider : public IModuleAnalysisProvider
{
public:
	explicit TModuleProvider(IAnalysisSession& Session);
	virtual ~TModuleProvider();

	// Query interface
	const FResolvedSymbol* GetSymbol(uint64 Address) override;
	uint32 GetNumModules() const override;
	void EnumerateModules(uint32 Start, TFunctionRef<void(const FModule& Module)> Callback) const override;
	FGraphEventRef LoadSymbolsForModuleUsingPath(uint64 Base, const TCHAR* Path) override;
	void EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const override;
	void GetStats(FStats* OutStats) const override;
	bool HasFinishedResolving() const override;

	// Analysis interface
	void OnModuleLoad(const FStringView& Module, uint64 Base, uint32 Size, const uint8* Checksum, uint32 ChecksumSize) override;
	void OnModuleUnload(uint64 Base) override;
	void OnAnalysisComplete() override;

	void SaveSymbolsToCache(IAnalysisCache& Cache);
	void LoadSymbolsFromCache(IAnalysisCache& Cache);

private:
	uint32 GetNumCachedSymbolsFromModule(uint64 Base, uint32 Size);
	const TCHAR* GetSafeFullName(FStringView InName);

	struct FSavedSymbol
	{
		uint64 Address;
		uint32 ModuleOffset;
		uint32 NameOffset;
		uint32 FileOffset;
		uint32 Line;
	};

	mutable FRWLock ModulesLock;
	TPagedArray<FModule> Modules;

	FRWLock SymbolsLock;
	// Persistently stored symbol strings
	FCachedStringStore Strings;
	// Efficient representation of symbols
	TPagedArray<FResolvedSymbol> SymbolCache;
	// Lookup table for symbols
	TMap<uint64, const FResolvedSymbol*> SymbolCacheLookup;
	// Number of cached symbols that was loaded (used for stats)
	uint32 NumCachedSymbols;
	// Number of discovered symbols
	std::atomic<uint32> SymbolsDiscovered;

	IAnalysisSession& Session;
	FString Platform;
	TUniquePtr<SymbolResolverType> Resolver;
	FGraphEventRef LoadSymbolsTask;
	bool LoadSymbolsAbort = false;

	FResolvedSymbolFilter SymbolFilter;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
TModuleProvider<SymbolResolverType>::TModuleProvider(IAnalysisSession& Session)
	: Modules(Session.GetLinearAllocator(), 128)
	, Strings(TEXT("ModuleProvider.Strings"), Session.GetCache())
	, SymbolCache(Session.GetLinearAllocator(), 1024*1024)
	, NumCachedSymbols(0)
	, Session(Session)
{
	Resolver = TUniquePtr<SymbolResolverType>(new SymbolResolverType(Session, SymbolFilter));
	LoadSymbolsFromCache(Session.GetCache());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
TModuleProvider<SymbolResolverType>::~TModuleProvider()
{
	if (LoadSymbolsTask)
	{
		LoadSymbolsAbort = true;
		LoadSymbolsTask->Wait();
	}
	// Delete the resolver in order to flush all pending resolves
	Resolver.Reset();
	SaveSymbolsToCache(Session.GetCache());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
const FResolvedSymbol* TModuleProvider<SymbolResolverType>::GetSymbol(uint64 Address)
{
	{
		// Attempt to read from the cached symbols.
		FReadScopeLock _(SymbolsLock);
		if (const FResolvedSymbol** Entry = SymbolCacheLookup.Find(Address))
		{
			return *Entry;
		}
	}

	FResolvedSymbol* ResolvedSymbol = nullptr;
	{
		FWriteScopeLock _(SymbolsLock);

		// Attempt again to read from the cached symbols.
		const FResolvedSymbol* CachedResolvedSymbol = SymbolCacheLookup.FindRef(Address);
		if (CachedResolvedSymbol)
		{
			return CachedResolvedSymbol;
		}

		// Add a pending entry to our cache.
		ResolvedSymbol = &SymbolCache.EmplaceBack(ESymbolQueryResult::Pending, nullptr, nullptr, nullptr, (uint16)0, EResolvedSymbolFilterStatus::Unknown);
		SymbolCacheLookup.Add(Address, ResolvedSymbol);

		++SymbolsDiscovered;
	}
	check(ResolvedSymbol);

	// If not in cache yet, queue it up in the resolver.
	Resolver->QueueSymbolResolve(Address, ResolvedSymbol);

	return ResolvedSymbol;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
uint32 TModuleProvider<SymbolResolverType>::GetNumModules() const
{
	FReadScopeLock _(ModulesLock);
	return static_cast<uint32>(Modules.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::EnumerateModules(uint32 Start, TFunctionRef<void(const FModule& Module)> Callback) const
{
	FReadScopeLock _(ModulesLock);
	for (uint32 ModuleIndex = Start; ModuleIndex < Modules.Num(); ++ModuleIndex)
	{
		Callback(Modules[ModuleIndex]);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
FGraphEventRef TModuleProvider<SymbolResolverType>::LoadSymbolsForModuleUsingPath(uint64 Base, const TCHAR* Path)
{
	FReadScopeLock _(ModulesLock);
	FModule* Module = nullptr;
	for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
	{
		if (Modules[ModuleIndex].Base == Base)
		{
			Module = &Modules[ModuleIndex];
			break;
		}
	}
	if (Module)
	{
		const FString FullPath = FPaths::ConvertRelativePathToFull(Path);
		if (Resolver && !FullPath.IsEmpty() && Module->Status.load() != EModuleStatus::Loaded)
		{
			// Setup a task to queue and watch the queued module. If it succeeds in resolving with the new path
			// re-resolve any cached symbols
			LoadSymbolsTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this, Module, FullPath]()
			{
				auto ReloadModuleFn = [this] (FModule* InModule, const TCHAR* InPath)
				{
					const uint32 DiscoveredSymbols = InModule->Stats.Discovered.load();
					const uint64 ModuleBegin = InModule->Base;
					const uint64 ModuleEnd = InModule->Base + InModule->Size;

					auto ReresolveOnSuccess = [this, DiscoveredSymbols, ModuleBegin, ModuleEnd] (TArray<TTuple<uint64,FResolvedSymbol*>>& OutSymbols)
					{
						OutSymbols.Reserve(DiscoveredSymbols);

						FReadScopeLock _(SymbolsLock);
						for (auto Pair : SymbolCacheLookup)
						{
							const uint64 Address = Pair.template Get<0>();
							FResolvedSymbol* Symbol = const_cast<FResolvedSymbol*>(Pair.template Get<1>());
							if (FMath::IsWithin(Address, ModuleBegin, ModuleEnd))
							{
								OutSymbols.Add(TTuple<uint64,FResolvedSymbol*>(Address, Symbol));
							}
						}
					};

					Resolver->QueueModuleReload(InModule, InPath, ReresolveOnSuccess);

					// Wait for the resolver to do it's work
					do
					{
						FPlatformProcess::Sleep(0.1f);
					}
					while (InModule->Status.load() == EModuleStatus::Pending);

					return InModule->Status.load();
				};

				UE_LOGF(LogTraceServices, Display, "[Modules] Queuing symbol loading using path %ls.", *FullPath);

				// Load the requested module
				const EModuleStatus Result = ReloadModuleFn(Module, *FullPath);

				if (Result == EModuleStatus::Loaded)
				{
					// Queue up any other failed module using the directory.
					IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
					const FString Directory = PlatformFile->DirectoryExists(*FullPath) ? FullPath : FPaths::GetPath(FullPath);

					TArray<FModule*> OtherModules;
					{
						FReadScopeLock _(ModulesLock);
						for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
						{
							FModule* OtherModule = &Modules[ModuleIndex];
							if (OtherModule != Module)
							{
								const EModuleStatus ModuleStatus = OtherModule->Status.load();
								if (ModuleStatus >= EModuleStatus::FailedStatusStart)
								{
									OtherModules.Add(OtherModule);
								}
							}
						}
					}
					for (FModule* OtherModule : OtherModules)
					{
						if (LoadSymbolsAbort)
						{
							return;
						}
						ReloadModuleFn(OtherModule, *Directory);
					}
				}

				UE_LOGF(LogTraceServices, Display, "[Modules] Completed loading symbols for path %ls.", *FullPath);
			});

			return LoadSymbolsTask;
		}
	}
	return FGraphEventRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const
{
	Resolver->EnumerateSymbolSearchPaths(Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::GetStats(FStats* OutStats) const
{
	// Get stats from resolver.
	Resolver->GetStats(OutStats);

	// The resolver does not include Cached into Discovered stat (but only Resolved and Failed).
	// Also, the resolver's Discovered and Cached stats (computed as sum of all modules) are not accurate
	// due to overlaps and unloaded/reloaded modules.
	// So we override these with the provider's stats.
	OutStats->SymbolsDiscovered = SymbolsDiscovered.load() + NumCachedSymbols;
	OutStats->SymbolsCached = NumCachedSymbols;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
bool TModuleProvider<SymbolResolverType>::HasFinishedResolving() const
{
	return Resolver->HasFinishedResolving();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
const TCHAR* TModuleProvider<SymbolResolverType>::GetSafeFullName(FStringView InFullName)
{
	// Module names cannot start with a network path.
	if (InFullName.StartsWith(TEXT("\\\\")))
	{
		// Read the list of allowed network paths once.
		static TArray<FString> AllowedNetworkPaths = [](){
			FString SettingsIni;
			TArray<FString> OutPaths;
			if (FConfigContext::ReadIntoGConfig().Load(TEXT("UnrealInsightsSettings"), SettingsIni) && GConfig != nullptr)
			{
				GConfig->GetArray(TEXT("Insights"), TEXT("AllowedNetworkPaths"), OutPaths, SettingsIni);
			}
			return MoveTemp(OutPaths);
		}();

		for (const auto& AllowedNetworkPath : AllowedNetworkPaths)
		{
			if (FPathViews::IsParentPathOf(AllowedNetworkPath, InFullName))
			{
				return Session.StoreString(InFullName);
			}
		}

		UE_LOGF(LogTraceServices, Warning, "[Modules] Module '%.*ls' is ignored. If trusted add to 'AllowedNetworkPaths' setting.", InFullName.Len(), InFullName.GetData());
		return nullptr;
	}
	return Session.StoreString(InFullName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::OnModuleLoad(const FStringView& ModuleName, uint64 Base, uint32 Size, const uint8* ImageId, uint32 ImageIdSize)
{
	if (ModuleName.Len() == 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[Modules] No name module is ignored (Base=0x%llX, Size=%u) !!!", Base, Size);
		return;
	}

	const FStringView NameTemp = FPathViews::GetCleanFilename(ModuleName);
	const TCHAR* Name = Session.StoreString(NameTemp);
	const TCHAR* FullName = GetSafeFullName(ModuleName);
	if (!FullName)
	{
		return;
	}

	FWriteScopeLock _(ModulesLock);

	for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
	{
		FModule& LoadedModule = Modules[ModuleIndex];

		// Check if the module was already added.
		if (LoadedModule.Name == Name && // only comparing pointers from session's string store
			LoadedModule.Base == Base &&
			LoadedModule.Size == Size)
		{
			if (LoadedModule.bIsUnloaded)
			{
				UE_LOGF(LogTraceServices, Log, "[Modules] Re-loaded module %ls (Base=0x%llX, Size=%u).", LoadedModule.Name, LoadedModule.Base, LoadedModule.Size);
				LoadedModule.bIsUnloaded = false;
			}
			else
			{
				// Duplicated module !!!
				UE_LOGF(LogTraceServices, Warning, "[Modules] Duplicated module %ls (Base=0x%llX, Size=%u) !!!", LoadedModule.Name, LoadedModule.Base, LoadedModule.Size);
			}
			return;
		}

		// Check if the new module overlaps an existing (active) module.
		if (!LoadedModule.bIsUnloaded &&
			Base + Size > LoadedModule.Base &&
			Base < LoadedModule.Base + LoadedModule.Size)
		{
			// Overlapping modules !!!
			UE_LOGF(LogTraceServices, Warning, "[Modules] Module %ls (Base=0x%llX, Size=%u) overlaps with module %ls (Base=0x%llX, Size=%u) !!!",
				Name, Base, Size, LoadedModule.Name, LoadedModule.Base, LoadedModule.Size);
		}
	}

	FModule& NewModule = Modules.EmplaceBack(Name, FullName, Base, Size, EModuleStatus::Discovered);

	UE_LOGF(LogTraceServices, Log, "[Modules] Loaded module %ls (Base=0x%llX, Size=%u).", NewModule.Name, NewModule.Base, NewModule.Size);

	// The number of cached symbols for this module is equal to the number
	// of matching addresses in the cache.
	NewModule.Stats.Cached = GetNumCachedSymbolsFromModule(Base, Size);

	Resolver->QueueModuleLoad(ImageId, ImageIdSize, &NewModule);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::OnModuleUnload(uint64 Base)
{
	FWriteScopeLock _(ModulesLock);

	// Check if the module was already added.
	for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
	{
		FModule& LoadedModule = Modules[ModuleIndex];

		if (LoadedModule.Base == Base && !LoadedModule.bIsUnloaded)
		{
			LoadedModule.bIsUnloaded = true;
			UE_LOGF(LogTraceServices, Log, "[Modules] Unloaded module %ls (Base=0x%llX, Size=%u).", LoadedModule.Name, LoadedModule.Base, LoadedModule.Size);
			return;
		}
	}

	// Unknown module !!!
	UE_LOGF(LogTraceServices, Warning, "[Modules] No module to unload with Base=0x%llX !!!", Base);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::OnAnalysisComplete()
{
	Resolver->OnAnalysisComplete();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::SaveSymbolsToCache(IAnalysisCache& Cache)
{
	// Create a temporary reverse lookup for symbol -> address
	TMap<const FResolvedSymbol*, uint64> SymbolReverseLookup;
	SymbolReverseLookup.Reserve(SymbolCacheLookup.Num());
	Algo::Transform(SymbolCacheLookup, SymbolReverseLookup, [](const TTuple<uint64, const FResolvedSymbol*>& Pair) {
		return TTuple<const FResolvedSymbol*, uint64>(Pair.Value, Pair.Key);
	});

	// Save new symbols
	TCachedPagedArray<FSavedSymbol, 1024> SavedSymbols(TEXT("ModuleProvider.Symbols"), Cache);
	const uint32 NumPreviouslySavedSymbols = static_cast<uint32>(SavedSymbols.Num());
	const uint32 NumSymbols = static_cast<uint32>(SymbolCache.Num());
	uint32 NumSavedSymbols = 0;
	for (uint32 SymbolIndex = NumPreviouslySavedSymbols; SymbolIndex < NumSymbols; ++SymbolIndex)
	{
		const FResolvedSymbol& Symbol = SymbolCache[SymbolIndex];
		if (Symbol.GetResult() != ESymbolQueryResult::OK)
		{
			continue;
		}
		const uint64* Address = SymbolReverseLookup.Find(&Symbol);
		const uint32 ModuleOffset = static_cast<uint32>(Strings.Store_GetOffset(Symbol.Module));
		const uint32 NameOffset = static_cast<uint32>(Strings.Store_GetOffset(Symbol.Name));
		const uint32 FileOffset = static_cast<uint32>(Strings.Store_GetOffset(Symbol.File));
		SavedSymbols.EmplaceBack(FSavedSymbol{*Address, ModuleOffset, NameOffset, FileOffset, Symbol.Line});
		++NumSavedSymbols;
	}
	UE_LOGF(LogTraceServices, Display, "[Modules] Added %d symbols to the %d previously saved symbols.", NumSavedSymbols, NumPreviouslySavedSymbols);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
void TModuleProvider<SymbolResolverType>::LoadSymbolsFromCache(IAnalysisCache& Cache)
{
	// Load saved symbols
	TCachedPagedArray<FSavedSymbol, 1024> SavedSymbols(TEXT("ModuleProvider.Symbols"), Cache);
	for (uint64 SymbolIndex = 0; SymbolIndex < SavedSymbols.Num(); ++SymbolIndex)
	{
		const FSavedSymbol& Symbol = SavedSymbols[SymbolIndex];
		const TCHAR* Module = Strings.GetStringAtOffset(Symbol.ModuleOffset);
		const TCHAR* Name = Strings.GetStringAtOffset(Symbol.NameOffset);
		const TCHAR* File = Strings.GetStringAtOffset(Symbol.FileOffset);
		if (Module == nullptr || Name == nullptr || File == nullptr)
		{
			UE_LOGF(LogTraceServices, Warning, "[Modules] Found cached symbol (address 0x%llX) which referenced unknown string !!!", Symbol.Address);
			continue;
		}
		FResolvedSymbol& Resolved = SymbolCache.EmplaceBack(ESymbolQueryResult::OK, Module, Name, File, static_cast<uint16>(Symbol.Line), EResolvedSymbolFilterStatus::Unknown);
		SymbolCacheLookup.Add(Symbol.Address, &Resolved);
	}
	NumCachedSymbols = SymbolCacheLookup.Num();

	// Update filter for all cached symbols.
	ParallelFor(static_cast<int32>(SymbolCache.Num()), [this](int32 Index)
		{
			SymbolFilter.Update(SymbolCache[Index]);
		},
		EParallelForFlags::BackgroundPriority);

	UE_LOGF(LogTraceServices, Display, "[Modules] Loaded %d symbols from cache.", SymbolCacheLookup.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename SymbolResolverType>
uint32 TModuleProvider<SymbolResolverType>::GetNumCachedSymbolsFromModule(uint64 Base, uint32 Size)
{
	uint32 Count(0);
	const uint64 Start = Base;
	const uint64 End = Base + Size;
	FWriteScopeLock _(SymbolsLock);
	for (auto& AddressSymbolPair : SymbolCacheLookup)
	{
		const uint64 Address = AddressSymbolPair.template Get<0>();
		if (FMath::IsWithin(Address, Start, End))
		{
			++Count;
		}
	}
	return Count;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FResolvedSymbolFilter::FResolvedSymbolFilter()
{
	// Filter by module name (prefix)
	//IgnoreSymbolsByModuleName.Add(TEXTVIEW("kernel32.dll"));
	//IgnoreSymbolsByModuleName.Add(TEXTVIEW("ntdll.dll"));

	// Filter by function name (prefix)
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FMemory::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FMallocWrapper::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FMallocPoisonProxy::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FMallocLeakDetectionProxy::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FVirtualWinApiHooks::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("Malloc"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("Realloc"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("Free"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("MemoryTrace_"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("operator new"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("operator delete"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("std::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FWindowsPlatformMemory::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FCachedOSPageAllocator::"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FMallocBinned"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("FD3D12Adapter::TraceMemoryAllocation"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("AnsiMalloc"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("AnsiRealloc"));
	IgnoreSymbolsByFunctionName.Add(TEXTVIEW("AnsiFree"));

	// Filter by file path (substring)
	IgnoreSymbolsByFilePath.Add(TEXTVIEW("/Containers/"));
	IgnoreSymbolsByFilePath.Add(TEXTVIEW("/ConcurrentLinearAllocator"));
	IgnoreSymbolsByFilePath.Add(TEXTVIEW("/D3D12PoolAllocator"));
	IgnoreSymbolsByFilePath.Add(TEXTVIEW("/AutoRTFM/"));
	IgnoreSymbolsByFilePath.Add(TEXTVIEW("/HAL/FMemory.inl"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FResolvedSymbolFilter::~FResolvedSymbolFilter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FResolvedSymbolFilter::Update(FResolvedSymbol& InSymbol) const
{
	bool bIsFiltered = false;

	if (InSymbol.GetResult() == ESymbolQueryResult::NotLoaded)
	{
		bIsFiltered = true;
	}

	if (!bIsFiltered && InSymbol.Module)
	{
		// Ignore symbols by module name prefix.
		for (const FStringView& Prefix : IgnoreSymbolsByModuleName)
		{
			if (FCString::Strnicmp(InSymbol.Module, Prefix.GetData(), Prefix.Len()) == 0)
			{
				bIsFiltered = true;
				break;
			}
		}
	}

	if (!bIsFiltered && InSymbol.Name)
	{
		// Ignore symbols by function name prefix.
		for (const FStringView& Prefix : IgnoreSymbolsByFunctionName)
		{
			if (FCString::Strnicmp(InSymbol.Name, Prefix.GetData(), Prefix.Len()) == 0)
			{
				bIsFiltered = true;
				break;
			}
		}
	}

	if (!bIsFiltered && InSymbol.File)
	{
		FString File(InSymbol.File);
		File.ReplaceCharInline(TEXT('\\'), TEXT('/'), ESearchCase::CaseSensitive);

		// Ignore symbols by file path, specified as substrings.
		for (const FStringView& SubString : IgnoreSymbolsByFilePath)
		{
			if (File.Contains(SubString, ESearchCase::CaseSensitive))
			{
				bIsFiltered = true;
				break;
			}
		}
	}

	EResolvedSymbolFilterStatus FilterStatus = bIsFiltered ? TraceServices::EResolvedSymbolFilterStatus::Filtered : TraceServices::EResolvedSymbolFilterStatus::NotFiltered;
	InSymbol.FilterStatus.store(FilterStatus, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IModuleAnalysisProvider> CreateModuleProvider(IAnalysisSession& InSession, const FAnsiStringView& InSymbolFormat)
{
	TSharedPtr<IModuleAnalysisProvider> Provider;

#if USE_SYMSLIB && UE_SYMSLIB_AVAILABLE
	if (!Provider && (InSymbolFormat.Equals("pdb") || InSymbolFormat.Equals("dwarf")))
	{
		Provider = MakeShared<TModuleProvider<FSymslibResolver>>(InSession);
	}
#endif //USE_SYMSLIB
#if PLATFORM_WINDOWS && USE_DBGHELP
	if (!Provider && InSymbolFormat.Equals("pdb"))
	{
		Provider = MakeShared<TModuleProvider<FDbgHelpResolver>>(InSession);
	}
#endif // PLATFORM_WINDOWS && USE_DBGHELP
#if USE_PSYMRESOLVER
	if (!Provider && InSymbolFormat.Equals("psym"))
	{
		Provider = MakeShared<TModuleProvider<FPsymResolver>>(InSession);
	}
#endif
	return Provider;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName GetModuleProviderName()
{
	static const FName Name("ModuleProvider");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IModuleProvider* ReadModuleProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IModuleProvider>(GetModuleProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef USE_SYMSLIB
#undef USE_DBGHELP
