// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TraceServices/Model/Modules.h"
#include "Common/PagedArray.h"
#include "Misc/ScopeRWLock.h"

namespace TraceServices 
{

class IAnalysisSession;

class IModuleAnalysisProvider : public IModuleProvider
{
public:
	virtual void OnModuleLoad(const FStringView& Module, uint64 Base, uint32 Size, const uint8* ImageId, uint32 ImageIdSize) = 0;
	virtual void OnModuleUnload(uint64 Base) = 0;
	virtual void OnAnalysisComplete() = 0;
};

/** Create a module provider with the given symbol format. */
TSharedPtr<IModuleAnalysisProvider> CreateModuleProvider(IAnalysisSession& Session, const FAnsiStringView& SymbolFormat);

} // namespace TraceServices
