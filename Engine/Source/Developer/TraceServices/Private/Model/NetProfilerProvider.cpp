// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetProfilerProvider.h"
#include "AnalysisServicePrivate.h"
#include "Containers/ArrayView.h"
#include "Common/StringStore.h"

namespace TraceServices
{

const TCHAR* LexToString(const ENetProfilerChannelCloseReason Value)
{
	switch (Value)
	{
	case ENetProfilerChannelCloseReason::Destroyed:
		return TEXT("Destroyed");
	case ENetProfilerChannelCloseReason::Dormancy:
		return TEXT("Dormancy");
	case ENetProfilerChannelCloseReason::LevelUnloaded:
		return TEXT("LevelUnloaded");
	case ENetProfilerChannelCloseReason::Relevancy:
		return TEXT("Relevancy");
	case ENetProfilerChannelCloseReason::TearOff:
		return TEXT("TearOff");
	}

	return TEXT("Unknown");
}

const TCHAR* LexToString(const ENetProfilerConnectionState Value)
{
	switch (Value)
	{
	case ENetProfilerConnectionState::USOCK_Closed:
		return TEXT("Closed");
	case ENetProfilerConnectionState::USOCK_Open:
		return TEXT("Open");
	case ENetProfilerConnectionState::USOCK_Pending:
		return TEXT("Pending");
	case ENetProfilerConnectionState::USOCK_Invalid:
	default:
		return TEXT("Invalid");
	}
}

FNetProfilerProvider::FNetProfilerProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, Connections(InSession.GetLinearAllocator(), 4096)
	, NetTraceVersion(0U)
	, ConnectionChangeCount(0u)
	, GameInstanceChangeCount(0u)
{
	// Use name index 0 to indicate that we do not know the name
	AddNetProfilerName(TEXT("N/A"));

	AggregatedStatsTableLayout.
		AddColumn(&FNetProfilerAggregatedStats::EventTypeIndex, TEXT("EventTypeIndex")).
		AddColumn(&FNetProfilerAggregatedStats::InstanceCount, TEXT("Count")).
		AddColumn(&FNetProfilerAggregatedStats::TotalInclusive, TEXT("Incl")).
		AddColumn(&FNetProfilerAggregatedStats::MaxInclusive, TEXT("I.Max")).
		AddColumn(&FNetProfilerAggregatedStats::AverageInclusive, TEXT("I.Avg")).
		AddColumn(&FNetProfilerAggregatedStats::TotalExclusive, TEXT("Excl")).
		AddColumn(&FNetProfilerAggregatedStats::MaxExclusive, TEXT("E.Max"));
}

FNetProfilerProvider::~FNetProfilerProvider()
{
}

void FNetProfilerProvider::SetNetTraceVersion(uint32 Version)
{
	Session.WriteAccessCheck();

	NetTraceVersion = Version;
}

uint32 FNetProfilerProvider::GetNetTraceVersion() const
{
	return NetTraceVersion;
}

FNetProfilerNameIndexType FNetProfilerProvider::AddNetProfilerName(const TCHAR* Name)
{
	Session.WriteAccessCheck();

	FNetProfilerName& NewName = Names.AddDefaulted_GetRef();
	NewName.NameIndex = Names.Num() - 1;
	NewName.Name = Session.StoreString(Name);

	return NewName.NameIndex;
}

const FNetProfilerName* FNetProfilerProvider::GetNetProfilerName(FNetProfilerNameIndexType NameIndex) const
{
	return NameIndex < (uint32)Names.Num() ? &Names[NameIndex] : nullptr;
}

uint32 FNetProfilerProvider::AddNetProfilerEventType(FNetProfilerNameIndexType NameIndex, uint32 Level)
{
	Session.WriteAccessCheck();

	FNetProfilerEventType& NewEventType = EventTypes.AddDefaulted_GetRef();
	NewEventType.EventTypeIndex = EventTypes.Num() - 1;
	NewEventType.NameIndex = NameIndex;
	NewEventType.Name = GetNetProfilerName(NameIndex)->Name;
	NewEventType.Level = IntCastChecked<uint16>(Level);

	return NewEventType.EventTypeIndex;
}

const FNetProfilerEventType* FNetProfilerProvider::GetNetProfilerEventType(uint32 EventTypeIndex) const
{
	return EventTypeIndex < (uint32)EventTypes.Num() ? &EventTypes[EventTypeIndex] : nullptr;
}

uint32 FNetProfilerProvider::AddNetProfilerStatsCounterType(FNetProfilerNameIndexType NameIndex, ENetProfilerStatsCounterType Type)
{
	Session.WriteAccessCheck();

	FNetProfilerStatsCounterType& NewStatsCounterType = StatsCounterTypes.AddDefaulted_GetRef();
	NewStatsCounterType.StatsCounterTypeIndex = StatsCounterTypes.Num() - 1;
	NewStatsCounterType.NameIndex = NameIndex;
	NewStatsCounterType.Type = Type;

	return NewStatsCounterType.StatsCounterTypeIndex;
}

const FNetProfilerStatsCounterType* FNetProfilerProvider::GetNetProfilerStatsCounterType(uint32 StatsCounterTypeIndex) const
{
	return StatsCounterTypeIndex < (uint32)StatsCounterTypes.Num() ? &StatsCounterTypes[StatsCounterTypeIndex] : nullptr;
}

FNetProfilerGameInstanceInternal& FNetProfilerProvider::CreateGameInstance()
{
	Session.WriteAccessCheck();

	FNetProfilerGameInstanceInternal& GameInstance = GameInstances.AddDefaulted_GetRef();
	GameInstance.Instance.GameInstanceIndex = GameInstances.Num() - 1;

	GameInstance.Objects = (TPagedArray<FNetProfilerObjectInstance>*)Session.GetLinearAllocator().Allocate(sizeof(TPagedArray<FNetProfilerObjectInstance>));
	new (GameInstance.Objects) TPagedArray<FNetProfilerObjectInstance>(Session.GetLinearAllocator(), 4096);
	GameInstance.ObjectsChangeCount = 0u;

	GameInstance.Frames = (TPagedArray<FNetProfilerFrame>*)Session.GetLinearAllocator().Allocate(sizeof(TPagedArray<FNetProfilerFrame>));
	new (GameInstance.Frames) TPagedArray<FNetProfilerFrame>(Session.GetLinearAllocator(), 4096);
	GameInstance.FramesChangeCount = 0u;

	GameInstance.FrameStats = (TPagedArray<FNetProfilerStats>*)Session.GetLinearAllocator().Allocate(sizeof(TPagedArray<FNetProfilerStats>));
	new (GameInstance.FrameStats) TPagedArray<FNetProfilerStats>(Session.GetLinearAllocator(), 4096);

	// We reserve object index 0 as an invalid object
	CreateObject(GameInstance.Instance.GameInstanceIndex);

	MarkGameInstancesDirty();

	return GameInstance;
}

FNetProfilerGameInstanceInternal* FNetProfilerProvider::EditGameInstance(uint32 GameInstanceIndex)
{
	Session.WriteAccessCheck();

	if (ensure(GameInstanceIndex < (uint32)GameInstances.Num()))
	{
		return &GameInstances[GameInstanceIndex];
	}
	else
	{
		return nullptr;
	}
}

void FNetProfilerProvider::MarkGameInstancesDirty()
{
	Session.WriteAccessCheck();
	++GameInstanceChangeCount;
}

FNetProfilerConnectionInternal& FNetProfilerProvider::CreateConnection(uint32 GameInstanceIndex)
{
	Session.WriteAccessCheck();

	FNetProfilerGameInstanceInternal* GameInstance = EditGameInstance(GameInstanceIndex);
	check(GameInstance);

	// Create new connection
	uint32 ConnectionIndex = static_cast<uint32>(Connections.Num());
	FNetProfilerConnectionInternal& Connection = Connections.PushBack();

	Connection.Connection.ConnectionIndex = ConnectionIndex;
	Connection.Connection.GameInstanceIndex = GameInstanceIndex;
	Connection.Connection.bHasIncomingData = false;
	Connection.Connection.bHasOutgoingData = false;

	GameInstance->Connections.Push(ConnectionIndex);

	// Allocate storage for packets and events
	Connection.Data[ENetProfilerConnectionMode::Outgoing] = (FNetProfilerConnectionData*)Session.GetLinearAllocator().Allocate(sizeof(FNetProfilerConnectionData));
	new (Connection.Data[ENetProfilerConnectionMode::Outgoing]) FNetProfilerConnectionData(Session.GetLinearAllocator());

	Connection.Data[ENetProfilerConnectionMode::Incoming] = (FNetProfilerConnectionData*)Session.GetLinearAllocator().Allocate(sizeof(FNetProfilerConnectionData));
	new (Connection.Data[ENetProfilerConnectionMode::Incoming]) FNetProfilerConnectionData(Session.GetLinearAllocator());

	++ConnectionChangeCount;

	return Connection;
}

FNetProfilerObjectInstance& FNetProfilerProvider::CreateObject(uint32 GameInstanceIndex)
{
	Session.WriteAccessCheck();

	FNetProfilerGameInstanceInternal* GameInstance = EditGameInstance(GameInstanceIndex);
	check(GameInstance);

	FNetProfilerObjectInstance& Object = GameInstance->Objects->PushBack();
	Object.ObjectIndex = static_cast<uint32>(GameInstance->Objects->Num()) - 1;
	++GameInstance->ObjectsChangeCount;

	return Object;
}

FNetProfilerObjectInstance* FNetProfilerProvider::EditObject(uint32 GameInstanceIndex, uint32 ObjectIndex)
{
	Session.WriteAccessCheck();

	FNetProfilerGameInstanceInternal* GameInstance = EditGameInstance(GameInstanceIndex);
	check(GameInstance);

	if (ensure(ObjectIndex < (uint32)GameInstance->Objects->Num()))
	{
		++GameInstance->ObjectsChangeCount;
		return &(*GameInstance->Objects)[ObjectIndex];
	}
	else
	{
		return nullptr;
	}
}

FNetProfilerConnectionInternal* FNetProfilerProvider::EditConnection(uint32 ConnectionIndex)
{
	Session.WriteAccessCheck();

	if (ensure(ConnectionIndex < (uint32)Connections.Num()))
	{
		++ConnectionChangeCount;
		return &Connections[ConnectionIndex];
	}
	else
	{
		return nullptr;
	}
}

void FNetProfilerProvider::EditPacketDeliveryStatus(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 SequenceNumber, ENetProfilerDeliveryStatus DeliveryStatus)
{
	check(ConnectionIndex < (uint32)Connections.Num());

	FNetProfilerConnectionInternal& Connection = Connections[ConnectionIndex];
	FNetProfilerConnectionData& Data = *Connections[ConnectionIndex].Data[Mode];

	// try to locate packet
	uint32 PacketCount = static_cast<uint32>(Data.Packets.Num());
	for (uint32 It = 0; It < PacketCount; ++It)
	{
		const uint32 PacketIndex = PacketCount - It - 1u;
		if (Data.Packets[PacketIndex].SequenceNumber == SequenceNumber)
		{
			Data.Packets[PacketIndex].DeliveryStatus = DeliveryStatus;
			++Data.PacketChangeCount;
			return;
		}
	}
}

FNetProfilerConnectionData& FNetProfilerProvider::EditConnectionData(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode)
{
	check(ConnectionIndex < (uint32)Connections.Num());

	FNetProfilerConnectionInternal& Connection = Connections[ConnectionIndex];

	if (Mode == ENetProfilerConnectionMode::Incoming && !Connection.Connection.bHasIncomingData)
	{
		Connection.Connection.bHasIncomingData = true;
		++ConnectionChangeCount;
	}
	if (Mode == ENetProfilerConnectionMode::Outgoing && !Connection.Connection.bHasOutgoingData)
	{
		Connection.Connection.bHasOutgoingData = true;
		++ConnectionChangeCount;
	}

	return *Connections[ConnectionIndex].Data[Mode];
}

void FNetProfilerProvider::ReadNames(TFunctionRef<void(const FNetProfilerName*, uint64)> Callback) const
{
	Session.ReadAccessCheck();

	Callback(Names.GetData(), Names.Num());
}

void FNetProfilerProvider::ReadName(uint32 NameIndex, TFunctionRef<void(const FNetProfilerName&)> Callback) const
{
	Session.ReadAccessCheck();
	check(NameIndex < (uint32)Names.Num());

	Callback(*GetNetProfilerName(NameIndex));
}

void FNetProfilerProvider::ReadEventTypes(TFunctionRef<void(const FNetProfilerEventType*, uint64)> Callback) const
{
	Session.ReadAccessCheck();

	Callback(EventTypes.GetData(), EventTypes.Num());
}

void FNetProfilerProvider::ReadEventType(uint32 EventTypeIndex, TFunctionRef<void(const FNetProfilerEventType&)> Callback) const
{
	Session.ReadAccessCheck();
	check(EventTypeIndex < (uint32)EventTypes.Num());

	Callback(*GetNetProfilerEventType(EventTypeIndex));
}

void FNetProfilerProvider::ReadNetStatsCounterTypes(TFunctionRef<void(const FNetProfilerStatsCounterType*, uint64)> Callback) const
{
	Session.ReadAccessCheck();

	Callback(StatsCounterTypes.GetData(), StatsCounterTypes.Num());
}

void FNetProfilerProvider::ReadNetStatsCounterType(uint32 TypeIndex, TFunctionRef<void(const FNetProfilerStatsCounterType&)> Callback) const
{
	Session.ReadAccessCheck();
	check(TypeIndex < (uint32)StatsCounterTypes.Num());

	Callback(*GetNetProfilerStatsCounterType(TypeIndex));
}

void FNetProfilerProvider::ReadGameInstances(TFunctionRef<void(const FNetProfilerGameInstance&)> Callback) const
{
	Session.ReadAccessCheck();

	for (const FNetProfilerGameInstanceInternal& Instance : GameInstances)
	{
		Callback(Instance.Instance);
	}
}

uint32 FNetProfilerProvider::GetConnectionCount(uint32 GameInstanceIndex) const
{
	Session.ReadAccessCheck();

	check(GameInstanceIndex < (uint32)GameInstances.Num());

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[GameInstanceIndex];

	return GameInstance.Connections.Num();
}

void FNetProfilerProvider::ReadConnections(uint32 GameInstanceIndex, TFunctionRef<void(const FNetProfilerConnection&)> Callback) const
{
	Session.ReadAccessCheck();

	check(GameInstanceIndex < (uint32)GameInstances.Num());

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[GameInstanceIndex];

	for (uint32 ConnectionIndex : MakeArrayView(GameInstance.Connections.GetData(), GameInstance.Connections.Num()))
	{
		Callback(Connections[ConnectionIndex].Connection);
	}
}

void FNetProfilerProvider::ReadConnection(uint32 ConnectionIndex, TFunctionRef<void(const FNetProfilerConnection&)> Callback) const
{
	Session.ReadAccessCheck();

	check(ConnectionIndex < Connections.Num());

	Callback(Connections[ConnectionIndex].Connection);
}

uint32 FNetProfilerProvider::GetObjectCount(uint32 GameInstanceIndex) const
{
	Session.ReadAccessCheck();

	check(GameInstanceIndex < (uint32)GameInstances.Num());

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[GameInstanceIndex];

	return static_cast<uint32>(GameInstance.Objects->Num());
}

void FNetProfilerProvider::ReadObjects(uint32 GameInstanceIndex, TFunctionRef<void(const FNetProfilerObjectInstance&)> Callback) const
{
	Session.ReadAccessCheck();

	check(GameInstanceIndex < (uint32)GameInstances.Num());

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[GameInstanceIndex];
	const auto& Objects = *GameInstance.Objects;

	const uint32 ObjectsEndIt = static_cast<uint32>(GameInstance.Objects->Num());
	for (uint32 ObjectsIt = 0; ObjectsIt < ObjectsEndIt; ++ObjectsIt)
	{
		Callback(Objects[ObjectsIt]);
	}
}

void FNetProfilerProvider::ReadObject(uint32 GameInstanceIndex, uint32 ObjectIndex, TFunctionRef<void(const FNetProfilerObjectInstance&)> Callback) const
{
	Session.ReadAccessCheck();

	check(GameInstanceIndex < (uint32)GameInstances.Num());

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[GameInstanceIndex];

	check(ObjectIndex < GameInstance.Objects->Num());

	Callback((*GameInstance.Objects)[ObjectIndex]);
}

uint32 FNetProfilerProvider::GetObjectsChangeCount(uint32 GameInstanceIndex) const
{
	Session.ReadAccessCheck();

	check(GameInstanceIndex < (uint32)GameInstances.Num());

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[GameInstanceIndex];

	return GameInstance.ObjectsChangeCount;
}

int32 FNetProfilerProvider::FindPacketIndexFromPacketSequence(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 SequenceNumber) const
{
	Session.ReadAccessCheck();

	check(ConnectionIndex < Connections.Num());

	const auto& Packets = Connections[ConnectionIndex].Data[Mode]->Packets;
	const int32 PacketCount = (int32)Packets.Num();

	if (PacketCount == 0)
	{
		return -1;
	}

	if (SequenceNumber < Packets[0].SequenceNumber)
	{
		return -1;
	}

	if (SequenceNumber > Packets[PacketCount - 1].SequenceNumber)
	{
		return -1;
	}

	// Brute force it, we can cache some data to speed this up if necessary
	for (int32 PacketIt = 0, PacketEndIt = PacketCount - 1; PacketIt <= PacketEndIt; ++PacketIt)
	{
		if (Packets[PacketIt].SequenceNumber == SequenceNumber)
		{
			return PacketIt;
		}
	}

	return -1;
}

uint32 FNetProfilerProvider::GetPacketCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const
{
	Session.ReadAccessCheck();

	check(ConnectionIndex < Connections.Num());

	const auto& Packets = Connections[ConnectionIndex].Data[Mode]->Packets;

	return static_cast<uint32>(Packets.Num());
}

void FNetProfilerProvider::EnumeratePackets(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd, TFunctionRef<void(const FNetProfilerPacket&)> Callback) const
{
	Session.ReadAccessCheck();

	check(ConnectionIndex < Connections.Num());

	const auto& Packets = Connections[ConnectionIndex].Data[Mode]->Packets;

	const uint32 PacketCount = static_cast<uint32>(Packets.Num());

	// [PacketIndexIntervalStart, PacketIndexIntervalEnd] is an inclusive interval.
	if (PacketCount == 0 || PacketIndexIntervalStart > PacketIndexIntervalEnd)
	{
		return;
	}

	for (uint32 PacketIt = PacketIndexIntervalStart, PacketEndIt = FMath::Min(PacketIndexIntervalEnd, PacketCount - 1u); PacketIt <= PacketEndIt; ++PacketIt)
	{
		Callback(Packets[PacketIt]);
	}
}

// Enumerate packet content events by range
void FNetProfilerProvider::EnumeratePacketContentEventsByIndex(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 StartEventIndex, uint32 EndEventIndex, TFunctionRef<void(const FNetProfilerContentEvent&)> Callback) const
{
	Session.ReadAccessCheck();

	check(ConnectionIndex < Connections.Num());

	const auto& ContentEvents = Connections[ConnectionIndex].Data[Mode]->ContentEvents;

	const uint32 EventCount = static_cast<uint32>(ContentEvents.Num());

	// [StartEventIndex, EndEventIndex] is an inclusive interval.
	if (EventCount == 0 || StartEventIndex > EndEventIndex)
	{
		return;
	}

	for (uint32 EventIt = StartEventIndex, EventEndIt = FMath::Min(EndEventIndex, EventCount - 1u); EventIt <= EventEndIt; ++EventIt)
	{
		Callback(ContentEvents[EventIt]);
	}
}

void FNetProfilerProvider::EnumeratePacketContentEventsByPosition(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndex, uint32 StartPos, uint32 EndPos, TFunctionRef<void(const FNetProfilerContentEvent&)> Callback) const
{
	Session.ReadAccessCheck();

	check(ConnectionIndex < Connections.Num());

	 const FNetProfilerConnectionData& ConnectionData = * Connections[ConnectionIndex].Data[Mode];

	const FNetProfilerPacket& Packet = ConnectionData.Packets[PacketIndex];
	if (Packet.EventCount == 0u)
	{
		return;
	}

	const uint32 StartEventIndex = Packet.StartEventIndex;
	const uint32 EndEventIndex = StartEventIndex + Packet.EventCount - 1u;

	const auto& ContentEvents = ConnectionData.ContentEvents;

	// The input [StartPos, EndPos) is an exclusive bit range.
	// Also, the [StartPos, EndPos) for ContentEvents[EventIt] is an exclusive bit range.

	uint32 EventIt = StartEventIndex;
	// Skip all Events outside of the scope
	while (EventIt <= EndEventIndex && ContentEvents[EventIt].EndPos <= StartPos)
	{
		++EventIt;
	}

	// Execute callback for all found events
	while (EventIt <= EndEventIndex && ContentEvents[EventIt].StartPos < EndPos)
	{
		Callback(ContentEvents[EventIt]);
		EndPos = FMath::Max(EndPos, (uint32)ContentEvents[EventIt].EndPos);
		++EventIt;
	}
}

uint32 FNetProfilerProvider::GetPacketChangeCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const
{
	Session.ReadAccessCheck();
	check(ConnectionIndex < Connections.Num());

	return Connections[ConnectionIndex].Data[Mode]->PacketChangeCount;
}

uint32 FNetProfilerProvider::GetPacketContentEventChangeCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const
{
	Session.ReadAccessCheck();
	check(ConnectionIndex < Connections.Num());

	return Connections[ConnectionIndex].Data[Mode]->ContentEventChangeCount;
}

ITable<FNetProfilerAggregatedStats>* FNetProfilerProvider::CreateAggregation(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd, uint32 StartPosition, uint32 EndPosition) const
{
	Session.ReadAccessCheck();

	if (!ensure(ConnectionIndex < Connections.Num()))
	{
		return nullptr;
	}

	// [PacketIndexIntervalStart, PacketIndexIntervalEnd] is an inclusive interval.
	if (!ensure(PacketIndexIntervalStart <= PacketIndexIntervalEnd))
	{
		return nullptr;
	}

	const auto& Packets = Connections[ConnectionIndex].Data[Mode]->Packets;
	const uint32 PacketCount = static_cast<uint32>(Packets.Num());

	if (!ensure(PacketCount > 0))
	{
		return nullptr;
	}

	struct FStackEntry
	{
		uint32 EventTypeIndex = 0U;
		uint32 StartPos = 0U;
		uint32 EndPos = 0U;
		uint32 ExclusiveAccumulator = 0U;
	};

	struct FStatsHelper
	{
		TMap<uint32, FNetProfilerAggregatedStats> AggregatedStats;
		uint64 StackSize;
		TArray<FStackEntry> Stack;
	};

	FStatsHelper Helper;
	Helper.AggregatedStats.Reserve(EventTypes.Num());
	Helper.StackSize = 0; // stack is empty
	Helper.Stack.SetNum(256);

	auto GetStatsFunction = [&Helper, this](const FNetProfilerContentEvent& ContentEvent)
	{
		FNetProfilerAggregatedStats* StatsEntry = Helper.AggregatedStats.Find(ContentEvent.EventTypeIndex);
		if (!StatsEntry)
		{
			StatsEntry = &Helper.AggregatedStats.Add(ContentEvent.EventTypeIndex);
			StatsEntry->EventTypeIndex = ContentEvent.EventTypeIndex;
		}

		// Fill in basics.
		const uint32 InclusiveSize = static_cast<uint32>(ContentEvent.EndPos - ContentEvent.StartPos);
		++StatsEntry->InstanceCount;
		StatsEntry->TotalInclusive += InclusiveSize;
		StatsEntry->MaxInclusive = FMath::Max(InclusiveSize, StatsEntry->MaxInclusive);

		// Pops events from the stack. Keeps only the parent hierarchy of the current event.
		while (Helper.StackSize > ContentEvent.Level)
		{
			--Helper.StackSize;
			FStackEntry& StackEntry = Helper.Stack[static_cast<uint32>(Helper.StackSize)];

			// Finalize exclusive for each poped event (all its children were already processed).
			FNetProfilerAggregatedStats& Stats = Helper.AggregatedStats.FindChecked(StackEntry.EventTypeIndex);
			const uint32 ExclusiveSize = (StackEntry.EndPos - StackEntry.StartPos) - StackEntry.ExclusiveAccumulator;
			Stats.TotalExclusive += ExclusiveSize;
			Stats.MaxExclusive = FMath::Max(Stats.MaxExclusive, ExclusiveSize);
		}

		// Pushes the new event on the stack.
		ensure(ContentEvent.Level == Helper.StackSize);
		Helper.StackSize = ContentEvent.Level + 1; // push
		FStackEntry& CurrentEventLevelStackEntry = Helper.Stack[ContentEvent.Level];

		// We track what we have visited to be able to update exclusive bits for our parent.
		// We accumulate during the first pass and finalize the values during the second.
		CurrentEventLevelStackEntry.EventTypeIndex = ContentEvent.EventTypeIndex;
		CurrentEventLevelStackEntry.StartPos = ContentEvent.StartPos;
		CurrentEventLevelStackEntry.EndPos = ContentEvent.EndPos;
		CurrentEventLevelStackEntry.ExclusiveAccumulator = 0U;

		if (ContentEvent.Level > 0U)
		{
			// Update parent event with the contribution from current event.
			FStackEntry& ParentStackEntry = Helper.Stack[static_cast<uint32>(ContentEvent.Level) - 1];
			ParentStackEntry.ExclusiveAccumulator += InclusiveSize;
		}
	};

	// Iterate over content events
	if (PacketIndexIntervalStart == PacketIndexIntervalEnd)
	{
		EnumeratePacketContentEventsByPosition(ConnectionIndex, Mode, PacketIndexIntervalStart, StartPosition, EndPosition, GetStatsFunction);

		// Pops the remaining events from the stack.
		while (Helper.StackSize > 0)
		{
			--Helper.StackSize;
			FStackEntry& StackEntry = Helper.Stack[static_cast<uint32>(Helper.StackSize)];

			// Finalize exclusive for each poped event (all its children were already processed).
			FNetProfilerAggregatedStats& Stats = Helper.AggregatedStats.FindChecked(StackEntry.EventTypeIndex);
			const uint32 ExclusiveSize = (StackEntry.EndPos - StackEntry.StartPos) - StackEntry.ExclusiveAccumulator;
			Stats.TotalExclusive += ExclusiveSize;
			Stats.MaxExclusive = FMath::Max(Stats.MaxExclusive, ExclusiveSize);
		}
	}
	else
	{
		// Iterate over packets
		for (uint32 PacketIt = PacketIndexIntervalStart, PacketEndIt = FMath::Min(PacketIndexIntervalEnd, PacketCount - 1u); PacketIt <= PacketEndIt; ++PacketIt)
		{
			const auto& ContentEvents = Connections[ConnectionIndex].Data[Mode]->ContentEvents;
			const uint32 EventCount = static_cast<uint32>(ContentEvents.Num());

			const FNetProfilerPacket& Packet = Packets[PacketIt];
			if (Packet.EventCount > 0U)
			{
				ensure(Helper.StackSize == 0); // stack should be empty (before each packet)

				const uint32 StartEventIndex = Packet.StartEventIndex;
				for (uint32 It = 0U; It < Packet.EventCount; ++It)
				{
					GetStatsFunction(ContentEvents[StartEventIndex + It]);
				}

				// Pops the remaining events from the stack, for each packet.
				while (Helper.StackSize > 0)
				{
					--Helper.StackSize;
					FStackEntry& StackEntry = Helper.Stack[static_cast<uint32>(Helper.StackSize)];

					// Finalize exclusive for each poped event (all its children were already processed).
					FNetProfilerAggregatedStats& Stats = Helper.AggregatedStats.FindChecked(StackEntry.EventTypeIndex);
					const uint32 ExclusiveSize = (StackEntry.EndPos - StackEntry.StartPos) - StackEntry.ExclusiveAccumulator;
					Stats.TotalExclusive += ExclusiveSize;
					Stats.MaxExclusive = FMath::Max(Stats.MaxExclusive, ExclusiveSize);
				}
			}
		}
	}

	// Calculate averages and populate table
	TTable<FNetProfilerAggregatedStats>* Table = new TTable<FNetProfilerAggregatedStats>(AggregatedStatsTableLayout);
	for (const auto& KV : Helper.AggregatedStats)
	{
		FNetProfilerAggregatedStats& Row = Table->AddRow();
		Row = KV.Value;

		// Finalize StatsEntry
		Row.AverageInclusive = (uint64)((double)KV.Value.TotalInclusive / KV.Value.InstanceCount);
	}
	return Table;
}

ITable<FNetProfilerAggregatedStatsCounterStats>* FNetProfilerProvider::CreateStatsCountersAggregation(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd) const
{
	Session.ReadAccessCheck();

	if (!ensure(ConnectionIndex < Connections.Num()))
	{
		return nullptr;
	}

	// [PacketIndexIntervalStart, PacketIndexIntervalEnd] is an inclusive interval.
	if (!ensure(PacketIndexIntervalStart <= PacketIndexIntervalEnd))
	{
		return nullptr;
	}

	const FNetProfilerConnectionData* ConnectionData = Connections[ConnectionIndex].Data[Mode];
	const auto& Packets = ConnectionData->Packets;

	const uint32 PacketCount = static_cast<uint32>(Packets.Num());

	if (!ensure(PacketCount > 0))
	{
		return nullptr;
	}

	TMap<uint32, FNetProfilerAggregatedStatsCounterStats> AggregatedStatsMap;
	AggregatedStatsMap.Reserve(StatsCounterTypes.Num());

	auto AccumulateStatsFunction = [&AggregatedStatsMap, this](const FNetProfilerStats& StatsCounter)
	{
		FNetProfilerAggregatedStatsCounterStats* StatsEntry = AggregatedStatsMap.Find(StatsCounter.StatsCounterTypeIndex);
		if (!StatsEntry)
		{
			StatsEntry = &AggregatedStatsMap.Add(StatsCounter.StatsCounterTypeIndex);
			StatsEntry->StatsCounterTypeIndex = StatsCounter.StatsCounterTypeIndex;
		}
		const uint32 StatsValue = StatsCounter.StatsValue;
		StatsEntry->Sum += StatsValue;
		StatsEntry->Max = FMath::Max(StatsValue, StatsEntry->Max);
		++StatsEntry->Count;
	};

	const FNetProfilerGameInstanceInternal& GameInstance = GameInstances[Connections[ConnectionIndex].Connection.GameInstanceIndex];
	const auto& PacketStatsCounters = ConnectionData->PacketStats;
	const auto& FrameStatsCounters = *GameInstance.FrameStats;
	const auto& Frames = *GameInstance.Frames;

	// Iterate over packets
	for (uint32 PacketIt = PacketIndexIntervalStart, PacketEndIt = FMath::Min(PacketIndexIntervalEnd, PacketCount - 1u); PacketIt <= PacketEndIt; ++PacketIt)
	{
		const FNetProfilerPacket& Packet = Packets[PacketIt];
		const uint32 StatsCounterCount = static_cast<uint32>(PacketStatsCounters.Num());

		// Iterate over all StatsCounters stored for the Packet
		for (uint32 StatsCounterIt = 0; StatsCounterIt < Packet.StatsCount; ++StatsCounterIt)
		{
			AccumulateStatsFunction(PacketStatsCounters[Packet.StartStatsIndex + StatsCounterIt]);
		}

		// Include frame stats as well
		const uint32 NetProfilerFrameIndex = Packet.NetProfilerFrameIndex;
		if (NetProfilerFrameIndex < Frames.Num())
		{
			const FNetProfilerFrame& Frame = Frames[Packet.NetProfilerFrameIndex];
			for (uint32 StatsCounterIt = 0; StatsCounterIt < Frame.StatsCount; ++StatsCounterIt)
			{
				AccumulateStatsFunction(FrameStatsCounters[Frame.StartStatsIndex + StatsCounterIt]);
			}
		}
	}

	// Calculate averages and populate table
	TTable<FNetProfilerAggregatedStatsCounterStats>* Table = new TTable<FNetProfilerAggregatedStatsCounterStats>(AggregatedStatsCounterStatsTableLayout);
	for (const auto& KV : AggregatedStatsMap)
	{
		FNetProfilerAggregatedStatsCounterStats& Row = Table->AddRow();
		Row = KV.Value;

		// Finalize StatsEntry
		Row.Average = (uint64)((double)KV.Value.Sum / KV.Value.Count);
	}
	return Table;
}

FName GetNetProfilerProviderName()
{
	static const FName Name("NetProfilerProvider");
	return Name;
}

const INetProfilerProvider* ReadNetProfilerProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<INetProfilerProvider>(GetNetProfilerProviderName());
}

} // namespace TraceServices
