// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/NetProfiler.h"
#include "TraceServices/Containers/SlabAllocator.h"
#include "Model/Tables.h"

namespace TraceServices
{

class FAnalysisSessionLock;
class FStringStore;

struct FNetProfilerGameInstanceInternal
{
	FNetProfilerGameInstance Instance;

	TPagedArray<FNetProfilerObjectInstance>* Objects;
	TArray<uint32, TInlineAllocator<128>> Connections;

	TPagedArray<FNetProfilerFrame>* Frames;
	TPagedArray<FNetProfilerStats>* FrameStats;

	uint32 ObjectsChangeCount;
	uint32 FramesChangeCount;	
};

struct FNetProfilerConnectionData
{
	FNetProfilerConnectionData(ILinearAllocator& Allocator)
		: Packets(Allocator, 1024)
		, PacketStats(Allocator, 4096)
		, ContentEvents(Allocator, 8192)
		, PacketChangeCount(0u)
		, PacketStatsChangeCount(0u)
		, ContentEventChangeCount(0u)
	{}

	TPagedArray<FNetProfilerPacket> Packets;
	TPagedArray<FNetProfilerStats> PacketStats;
	TPagedArray<FNetProfilerContentEvent> ContentEvents;

	uint32 PacketChangeCount;
	uint32 PacketStatsChangeCount;
	uint32 ContentEventChangeCount;
};

struct FNetProfilerConnectionInternal
{
	FNetProfilerConnection Connection;

	FNetProfilerConnectionData* Data[ENetProfilerConnectionMode::Count];
};

class FNetProfilerProvider
	: public INetProfilerProvider
{
public:
	explicit FNetProfilerProvider(IAnalysisSession& InSession);
	virtual ~FNetProfilerProvider();

	void SetNetTraceVersion(uint32 NetTraceVersion);

	FNetProfilerNameIndexType AddNetProfilerName(const TCHAR* Name);

	uint32 AddNetProfilerEventType(uint32 NameIndex, uint32 Level);

	uint32 AddNetProfilerStatsCounterType(uint32 NameIndex, ENetProfilerStatsCounterType Type);

	FNetProfilerGameInstanceInternal& CreateGameInstance();
	FNetProfilerGameInstanceInternal* EditGameInstance(uint32 GameInstanceIndex);
	void MarkGameInstancesDirty();

	FNetProfilerConnectionInternal& CreateConnection(uint32 GameInstanceIndex);
	FNetProfilerConnectionInternal* EditConnection(uint32 ConnectionIndex);
	FNetProfilerConnectionData& EditConnectionData(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode);

	void EditPacketDeliveryStatus(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 SequenceNumber, ENetProfilerDeliveryStatus DeliveryStatus);

	FNetProfilerObjectInstance& CreateObject(uint32 GameInstanceIndex);
	FNetProfilerObjectInstance* EditObject(uint32 GameInstanceIndex, uint32 ObjectIndex);

	// INetProfilerProvider Interface

	virtual uint32 GetNetTraceVersion() const override;

	// Access Names
	virtual uint32 GetNameCount() const override { return Names.Num(); }
	virtual void ReadNames(TFunctionRef<void(const FNetProfilerName*, uint64)> Callback) const override;
	virtual void ReadName(uint32 NameIndex, TFunctionRef<void(const FNetProfilerName&)> Callback) const override;

	// Access EventTypers
	virtual uint32 GetEventTypesCount() const override { return EventTypes.Num(); }
	virtual void ReadEventTypes(TFunctionRef<void(const FNetProfilerEventType*, uint64)> Callback) const override;
	virtual void ReadEventType(uint32 EventTypeIndex, TFunctionRef<void(const FNetProfilerEventType&)> Callback) const override;

	// Access GameInstances
	virtual uint32 GetGameInstanceCount() const override { return GameInstances.Num(); }
	virtual void ReadGameInstances(TFunctionRef<void(const FNetProfilerGameInstance&)> Callback) const override;
	virtual uint32 GetGameInstanceChangeCount() const override { return GameInstanceChangeCount; }

	// Access Connections
	virtual uint32 GetConnectionCount(uint32 GameInstanceIndex) const override;
	virtual void ReadConnections(uint32 GameInstanceIndex, TFunctionRef<void(const FNetProfilerConnection&)> Callback) const override;
	virtual void ReadConnection(uint32 ConnectionIndex, TFunctionRef<void(const FNetProfilerConnection&)> Callback) const override;
	virtual uint32 GetConnectionChangeCount() const override { return ConnectionChangeCount; }

	// Access Object Instances
	virtual uint32 GetObjectCount(uint32 GameInstanceIndex) const override;
	virtual void ReadObjects(uint32 GameInstanceIndex, TFunctionRef<void(const FNetProfilerObjectInstance&)> Callback) const override;
	virtual void ReadObject(uint32 GameInstanceIndex, uint32 ObjectIndex, TFunctionRef<void(const FNetProfilerObjectInstance&)> Callback) const override;
	virtual uint32 GetObjectsChangeCount(uint32 GameInstanceIndex) const override;

	// Find Packet Index from SequenceNumber
	virtual int32 FindPacketIndexFromPacketSequence(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 SequenceNumber) const override;

	// Enumerate packets in the provided packet interval
	virtual uint32 GetPacketCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const override;
	virtual void EnumeratePackets(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd, TFunctionRef<void(const FNetProfilerPacket&)> Callback) const override;
	virtual uint32 GetPacketChangeCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const override;

	// Enumerate packet content events by range
	virtual void EnumeratePacketContentEventsByIndex(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 StartEventIndex, uint32 EndEventIndex, TFunctionRef<void(const FNetProfilerContentEvent&)> Callback) const override;
	virtual void EnumeratePacketContentEventsByPosition(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndex, uint32 StartPos, uint32 EndPos, TFunctionRef<void(const FNetProfilerContentEvent&)> Callback) const override;
	virtual uint32 GetPacketContentEventChangeCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const override;

	// Access StatsCounterTypes
	virtual uint32 GetNetStatsCounterTypesCount() const override { return StatsCounterTypes.Num(); }
	virtual void ReadNetStatsCounterTypes(TFunctionRef<void(const FNetProfilerStatsCounterType*, uint64)> Callback) const override;
	virtual void ReadNetStatsCounterType(uint32 TypeIndex, TFunctionRef<void(const FNetProfilerStatsCounterType&)> Callback) const override;

	// Stats queries
	virtual ITable<FNetProfilerAggregatedStats>* CreateAggregation(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd, uint32 StartPosition, uint32 EndPosition) const override;
	virtual ITable<FNetProfilerAggregatedStatsCounterStats>* CreateStatsCountersAggregation(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd) const override;

	const TCHAR* InternalGetNetProfilerName(FNetProfilerNameIndexType NameIndex) const { return Names[NameIndex].Name; }
private:
	
	const FNetProfilerName* GetNetProfilerName(FNetProfilerNameIndexType ProfilerNameId) const;
	const FNetProfilerEventType* GetNetProfilerEventType(uint32 ProfilerEventTypeId) const;
	const FNetProfilerStatsCounterType* GetNetProfilerStatsCounterType(uint32 ProfilerStatsCounterTypeId) const;

	IAnalysisSession& Session;

	TArray<FNetProfilerName> Names;

	TArray<FNetProfilerEventType> EventTypes;

	TArray<FNetProfilerStatsCounterType> StatsCounterTypes;

	// All GameInstances seen throughout the session
	TArray<FNetProfilerGameInstanceInternal, TInlineAllocator<4>> GameInstances;

	// All connections we have seen throughout the session
	TPagedArray<FNetProfilerConnectionInternal> Connections;

	TTableLayout<FNetProfilerAggregatedStats> AggregatedStatsTableLayout;
	TTableLayout<FNetProfilerAggregatedStatsCounterStats> AggregatedStatsCounterStatsTableLayout;

	uint32 NetTraceVersion;
	uint32 ConnectionChangeCount;
	uint32 GameInstanceChangeCount;
};

} // namespace TraceServices
