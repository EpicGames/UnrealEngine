// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/ObjectProvider.h"
#include "ObjectProviderPrivate.h"

#include "Containers/Utf8String.h"
#include "HAL/PlatformFileManager.h"
#include "UObject/ObjectMacros.h"
#include "GenericPlatform/GenericPlatformFile.h"

// TraceServices
#include "Common/Utils.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Containers/Allocators.h"

#include <limits>

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GObjectProviderLockState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectSnapshot
////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectSnapshot::FObjectSnapshot()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectSnapshot::~FObjectSnapshot()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectInfo& FObjectSnapshot::GetOrAddObject(uint32 ObjectId)
{
	check(ObjectId < MaxNumObjects);
	if (ObjectId >= uint32(Objects.Num()))
	{
		Objects.AddDefaulted(int32(ObjectId) + 1 - Objects.Num());
	}
	return Objects[ObjectId];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectSnapshot::EnumerateObjects(TFunctionRef<bool(const FObjectInfo&)> Callback) const
{
	const uint32 TotalObjectCount = GetObjectArrayNum();
	for (uint32 ObjectId = 0; ObjectId < TotalObjectCount; ++ObjectId)
	{
		const FObjectInfo& Object = Objects[ObjectId];
		if (Object.Id == ObjectId)
		{
			if (!Callback(Object))
			{
				break;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FObjectInfo* FObjectSnapshot::FindObject(FStringView Name) const
{
	const uint32 TotalObjectCount = GetObjectArrayNum();
	for (uint32 ObjectId = 0; ObjectId < TotalObjectCount; ++ObjectId)
	{
		const FObjectInfo& Object = Objects[ObjectId];
		if (Object.Id == ObjectId &&
			Name.Equals(Object.Name, ESearchCase::CaseSensitive))
		{
			return &Object;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectSnapshot::Validate()
{
	UE_LOGF(LogTraceServices, Log, "[Obj] Validating objects...");

	struct FObjectValidationStats
	{
		const TCHAR* BaseClassName;
		uint32 BaseClassId;
		EObjectInfoFlags SingleFlag;
		EObjectInfoFlags ExpectedFlags;
		uint32 NumClasses;
		uint32 NumInstances;
	};

	constexpr EObjectInfoFlags ExpectedClassFlags = EObjectInfoFlags::IsClass | EObjectInfoFlags::IsStruct | EObjectInfoFlags::IsField;

	FObjectValidationStats ValidationStats[] =
	{
#define DECLARE_VALIDATION_STATS(ClassName, ExpectedFlags) \
		{ TEXT(#ClassName), InvalidObjectId, EObjectInfoFlags::Is##ClassName, ExpectedFlags, 0, 0 },
		DECLARE_VALIDATION_STATS(Class, ExpectedClassFlags)
		DECLARE_VALIDATION_STATS(Function, EObjectInfoFlags::IsFunction | EObjectInfoFlags::IsStruct | EObjectInfoFlags::IsField)
		DECLARE_VALIDATION_STATS(Struct, EObjectInfoFlags::IsStruct | EObjectInfoFlags::IsField)
		DECLARE_VALIDATION_STATS(Field, EObjectInfoFlags::IsField)
		DECLARE_VALIDATION_STATS(Package, EObjectInfoFlags::IsPackage)
#undef DECLARE_VALIDATION_STATS
	};
	constexpr int32 NumValidationStats = UE_ARRAY_COUNT(ValidationStats);

	// Searches the UClass object and counts the number of valid objects.
	// Also validates object flags for known instances.
	uint32 TotalObjectCount = 0;
	for (FObjectInfo& Obj : Objects)
	{
		if (Obj.Id == InvalidObjectId)
		{
			continue;
		}
		if (Obj.ClassId >= uint32(Objects.Num()))
		{
			UE_LOGF(LogTraceServices, Error, "[Obj] Object %u (%ls) with flags %u has invalid class id %u!",
				Obj.Id, Obj.Name, EnumToUnderlyingType(Obj.FlagsEx), Obj.ClassId);
			return;
		}
		FObjectInfo& ClassObj = Objects[Obj.ClassId];
		if (ClassObj.Id != Obj.ClassId)
		{
			UE_LOGF(LogTraceServices, Error, "[Obj] Object %u (%ls) with flags %u has invalid class object at id %u!",
				Obj.Id, Obj.Name, EnumToUnderlyingType(Obj.FlagsEx), Obj.ClassId);
			return;
		}
		++TotalObjectCount;
		for (int32 Index = 0; Index < NumValidationStats; ++Index)
		{
			FObjectValidationStats& Stats = ValidationStats[Index];
			if (EnumHasAnyFlags(Obj.FlagsEx, Stats.SingleFlag))
			{
				++Stats.NumInstances;
				ValidateExpectedFlags(Obj, Stats.ExpectedFlags);
			}
			if (Obj.SuperId != Obj.InheritanceSuperId &&
				Obj.InheritanceSuperId != InvalidObjectId)
			{
				UE_LOGF(LogTraceServices, Log, "[Obj] Object %u (%ls) with class id %u (%ls) and flags %u has super id %d != inheritance super id %d!",
					Obj.Id, Obj.Name, ClassObj.Id, ClassObj.Name, EnumToUnderlyingType(Obj.FlagsEx), int32(Obj.SuperId), int32(Obj.InheritanceSuperId));
			}
			if (Obj.SuperId != InvalidObjectId && Obj.SuperId >= uint32(Objects.Num()))
			{
				UE_LOGF(LogTraceServices, Warning, "[Obj] Object %u (%ls) with class id %u (%ls) and flags %u has invalid super id %u!",
					Obj.Id, Obj.Name, ClassObj.Id, ClassObj.Name, EnumToUnderlyingType(Obj.FlagsEx), Obj.SuperId);
				Obj.SuperId = InvalidObjectId;
			}
			if ((Stats.BaseClassId == InvalidObjectId) &&
				(FCString::Strcmp(Obj.Name, Stats.BaseClassName) == 0))
			{
				Stats.BaseClassId = Obj.Id;
				if (!EnumHasAllFlags(Obj.FlagsEx, ExpectedClassFlags))
				{
					UE_LOGF(LogTraceServices, Error, "[Obj] Object %u (%ls) with class id %u (%ls) is expected to be a class, but has invalid flags (current: %u, expected: %u)!",
						Obj.Id, Obj.Name, ClassObj.Id, ClassObj.Name, EnumToUnderlyingType(Obj.FlagsEx), EnumToUnderlyingType(ExpectedClassFlags));
				}
			}
		}
	}
	UE_LOGF(LogTraceServices, Log, "[Obj] Found %d objects.", TotalObjectCount);

	const uint32 ClassClassId = ValidationStats[0].BaseClassId;
	if (ClassClassId == InvalidObjectId)
	{
		UE_LOGF(LogTraceServices, Error, "[Obj] No UClass object!");
		return;
	}

	// Gather classes.
	uint32 DefaultClassCount = 0;
	uint32 OtherBaseClassCount = 0;
	uint32 OtherClassCount = 0;
	TArray<uint32> Classes;
	for (FObjectInfo& Obj : Objects)
	{
		if (Obj.Id == InvalidObjectId)
		{
			continue;
		}
		FObjectInfo& ClassObj = Objects[Obj.ClassId];
		if (Obj.ClassId == ClassClassId)
		{
			Classes.Add(Obj.Id);
			ValidateExpectedFlags(Obj, ExpectedClassFlags);
		}
		else if (EnumHasAnyFlags(Obj.FlagsEx, EObjectInfoFlags::IsClass))
		{
			uint8 ClassType = 0;
			if (Obj.SuperId == InvalidObjectId)
			{
				if (FCString::Strncmp(Obj.Name, TEXT("Default__"), 9) == 0)
				{
					++DefaultClassCount;
					ClassType = 1;
				}
				else
				{
					++OtherBaseClassCount;
					ClassType = 2;
				}
			}
			else
			{
				++OtherClassCount;
				ClassType = 3;
			}

			UE_LOGF(LogTraceServices, Verbose, "[Obj] %s %u (%ls) with class id %u (%ls), super id %d and flags %u has IsClass flag!",
				ClassType == 1 ? "Default Class Object" :
				ClassType == 2 ? "Other Base Class Object" :
								 "Other Class Object",
				Obj.Id, Obj.Name,
				ClassObj.Id, ClassObj.Name,
				int32(Obj.SuperId),
				EnumToUnderlyingType(Obj.FlagsEx));

			Classes.Add(Obj.Id);
			ValidateExpectedFlags(Obj, ExpectedClassFlags);
		}
	}
	const uint32 BaseClassCount = uint32(Classes.Num()) - DefaultClassCount - OtherBaseClassCount - OtherClassCount; 
	if (uint32(Classes.Num()) != ValidationStats[0].NumInstances)
	{
		UE_LOGF(LogTraceServices, Error, "[Obj] Found %u classes (base: %u + default: %u + other base: %u + other: %u) vs. %u class instances!",
			uint32(Classes.Num()),
			BaseClassCount,
			DefaultClassCount,
			OtherBaseClassCount,
			OtherClassCount,
			ValidationStats[0].NumInstances);
	}
	else
	{
		UE_LOGF(LogTraceServices, Log, "[Obj] Found %u classes (base: %u + default: %u + other base: %u + other: %u)!",
			uint32(Classes.Num()),
			BaseClassCount,
			DefaultClassCount,
			OtherBaseClassCount,
			OtherClassCount);
	}

	//////////////////////////////////////////////////

	for (int32 Index = 0; Index < NumValidationStats; ++Index)
	{
		FObjectValidationStats& Stats = ValidationStats[Index];

		if (Stats.BaseClassId != InvalidObjectId)
		{
			Stats.NumClasses = ValidateClassesRec(Classes, ClassClassId, Stats.BaseClassId, Stats.ExpectedFlags);
		}
		else
		{
			UE_LOGF(LogTraceServices, Error, "[Obj] No U%ls class!", Stats.BaseClassName);
		}

		UE_LOGF(LogTraceServices, Log, "[Obj] Class id %u (%ls) has %u derived classes and %u object instances.",
			Stats.BaseClassId, Stats.BaseClassName, Stats.NumClasses, Stats.NumInstances);
	}

	//////////////////////////////////////////////////

	UE_LOGF(LogTraceServices, Log, "[Obj] Validation completed.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectSnapshot::ValidateExpectedFlags(FObjectInfo& Obj, EObjectInfoFlags ExpectedFlags)
{
	if (!EnumHasAllFlags(Obj.FlagsEx, ExpectedFlags))
	{
		UE_LOGF(LogTraceServices, Warning, "[Obj] Object %u \"%ls\" (class id %u, super id %d) does not have all expected flags (current: %u, expected: %u)!",
			Obj.Id, Obj.Name,
			Obj.ClassId, int32(Obj.SuperId),
			EnumToUnderlyingType(Obj.FlagsEx), EnumToUnderlyingType(ExpectedFlags));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FObjectSnapshot::ValidateClassesRec(const TArray<uint32>& Classes, uint32 ClassClassId, uint32 BaseClassId, EObjectInfoFlags ExpectedClassFlags)
{
	uint32 Count = 0;
	for (uint32 CurrentClassId : Classes)
	{
		FObjectInfo& Obj = Objects[CurrentClassId];
		if (Obj.SuperId == BaseClassId)
		{
			FObjectInfo& ClassObj = Objects[Obj.ClassId];
			UE_LOGF(LogTraceServices, Verbose, "[Obj] Object %u (%ls) with class id %u (%ls) has super id %d, inheritance super id %d and flags %u.",
				Obj.Id, Obj.Name, ClassObj.Id, ClassObj.Name, int32(Obj.SuperId), int32(Obj.InheritanceSuperId), EnumToUnderlyingType(Obj.FlagsEx));

			if (Obj.ClassId != ClassClassId)
			{
				UE_LOGF(LogTraceServices, Warning, "[Obj] Object %u (%ls) is not class! It is %u (%ls)!",
					Obj.Id, Obj.Name, Obj.ClassId, Objects[Obj.ClassId].Name);
			}

			++Count;
			Count += ValidateClassesRec(Classes, ClassClassId, Obj.Id, ExpectedClassFlags);
		}
	}
	return Count;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectSnapshot::SaveAs(const TCHAR* CsvFileName) const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	IFileHandle* File = PlatformFile.OpenWrite(CsvFileName, false, false);
	if (File)
	{
		FUtf8StringView Header = UTF8TEXTVIEW("Id,ClassId,OuterId,Flags,StructureSize,SystemMemoryBytes,VideoMemoryBytes,TotalSystemMemoryBytes,TotalVideoMemoryBytes,Name,VersePath\n");
		File->Write((const uint8*)Header.GetData(), Header.NumBytes());
		for (const FObjectInfo& Obj : Objects)
		{
			if (Obj.Id == InvalidObjectId)
			{
				continue;
			}
			FUtf8String Str = FUtf8String::Printf(UTF8TEXT("%d,%d,%d,0x%08X,%d,%llu,%llu,%llu,%llu,%s,%s\n"),
				int32(Obj.Id),
				int32(Obj.ClassId),
				int32(Obj.OuterId),
				Obj.Flags, //TCHAR_TO_UTF8(*LexToString((EObjectFlags)Flags)),
				Obj.StructureSize,
				Obj.SystemMemoryBytes,
				Obj.VideoMemoryBytes,
				Obj.TotalSystemMemoryBytes,
				Obj.TotalVideoMemoryBytes,
				TCHAR_TO_UTF8(Obj.Name),
				TCHAR_TO_UTF8(Obj.VersePath));
			const auto& Data = Str.GetCharArray();
			File->Write((const uint8*)Data.GetData(), Data.NumBytes());
		}
		delete File;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectProvider::FObjectProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectProvider::~FObjectProvider()
{
	InternalReset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectProvider::InternalReset()
{
	if (CurrentSnapshot)
	{
		delete CurrentSnapshot;
		CurrentSnapshot = nullptr;
	}

	for (const FObjectSnapshot* Snapshot : Snapshots)
	{
		delete Snapshot;
	}
	Snapshots.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectProvider::EnumerateObjects(uint32 SnapshotId, TFunctionRef<bool(const FObjectInfo&)> Callback) const
{
	ReadAccessCheck();

	if (SnapshotId >= uint32(Snapshots.Num()))
	{
		return;
	}
	checkSlow(Snapshots[SnapshotId]);
	const FObjectSnapshot& Snapshot = *Snapshots[SnapshotId];
	Snapshot.EnumerateObjects(Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FObjectInfo* FObjectProvider::GetObject(uint32 SnapshotId, uint32 ObjectId) const
{
	ReadAccessCheck();

	if (SnapshotId >= uint32(Snapshots.Num()))
	{
		return nullptr;
	}
	checkSlow(Snapshots[SnapshotId]);
	const FObjectSnapshot& Snapshot = *Snapshots[SnapshotId];
	return Snapshot.GetObject(ObjectId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FObjectInfo* FObjectProvider::FindObject(uint32 SnapshotId, FStringView Name) const
{
	ReadAccessCheck();

	if (SnapshotId >= uint32(Snapshots.Num()))
	{
		return nullptr;
	}
	checkSlow(Snapshots[SnapshotId]);
	const FObjectSnapshot& Snapshot = *Snapshots[SnapshotId];
	return Snapshot.FindObject(Name);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FObjectProvider::GetCurrentSnapshotObjectCount() const
{
	ReadAccessCheck();

	if (CurrentSnapshot)
	{
		return CurrentSnapshot->GetObjectArrayNum();
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FObjectProvider::GetCurrentSnapshotTotalObjectCount() const
{
	ReadAccessCheck();

	if (CurrentSnapshot)
	{
		return CurrentSnapshot->GetTracedObjectArrayNum();
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FObjectProvider::GetNumSnapshots() const
{
	ReadAccessCheck();

	return uint32(Snapshots.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectProvider::EnumerateSnapshots(TFunctionRef<bool(const IObjectSnapshot&)> Callback) const
{
	ReadAccessCheck();

	for (const FObjectSnapshot* Snapshot : Snapshots)
	{
		checkSlow(Snapshot);
		if (!Callback(*Snapshot))
		{
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IObjectSnapshot* FObjectProvider::GetSnapshot(uint32 SnapshotId) const
{
	ReadAccessCheck();

	return (SnapshotId < uint32(Snapshots.Num())) ? Snapshots[SnapshotId] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FObjectProvider::IsCurrentSnapshotValid() const
{
	ReadAccessCheck();

	return CurrentSnapshot != nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IObjectSnapshot* FObjectProvider::GetLowerBoundSnapshot(double Time) const
{
	ReadAccessCheck();

	const int32 Index = Algo::UpperBound(Snapshots, Time, [](double Value, const FObjectSnapshot* Snapshot) { return Value < Snapshot->StartTime; });
	if (Index > 0 && Index <= Snapshots.Num())
	{
		return Snapshots[Index - 1];
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IObjectEditableSnapshot* FObjectProvider::GetCurrentSnapshot()
{
	EditAccessCheck();

	return CurrentSnapshot;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IObjectEditableSnapshot* FObjectProvider::BeginSnapshot(double Time)
{
	EditAccessCheck();

	if (CurrentSnapshot)
	{
		return nullptr;
	}

	CurrentSnapshot = new FObjectSnapshot();
	CurrentSnapshot->Id = uint32(Snapshots.Num());
	CurrentSnapshot->StartTime = Time;
	CurrentSnapshot->EndTime = std::numeric_limits<double>::infinity();

	return CurrentSnapshot;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IObjectEditableSnapshot* FObjectProvider::EndSnapshot(double Time)
{
	EditAccessCheck();

	if (!CurrentSnapshot)
	{
		return nullptr;
	}

#if 0
	// Ignores empty snapshots.
	if (CurrentSnapshot->IsEmpty())
	{
		delete CurrentSnapshot;
		CurrentSnapshot = nullptr;
		return nullptr;
	}
#endif

	CurrentSnapshot->EndTime = FMath::Max(Time, CurrentSnapshot->StartTime);
	check(CurrentSnapshot->Id == uint32(Snapshots.Num()));
#if !UE_BUILD_SHIPPING
	CurrentSnapshot->Validate();
#endif
	Snapshots.Add(CurrentSnapshot);

	FObjectSnapshot* Snapshot = CurrentSnapshot;
	CurrentSnapshot = nullptr;
	return Snapshot;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectInfo* FObjectProvider::AddObject(uint32 Id, uint32 ClassId, uint32 OuterId, uint32 Flags, const TCHAR* PersistentName)
{
	EditAccessCheck();

	if (!ensure(CurrentSnapshot))
	{
		return nullptr;
	}

	if (!FObjectSnapshot::IsValidObjectId(Id))
	{
		return nullptr;
	}

	FObjectInfo& Object = CurrentSnapshot->GetOrAddObject(Id);
	Object.Id = Id;
	Object.ClassId = ClassId;
	Object.OuterId = OuterId;
	Object.Name = PersistentName;
	Object.Flags = Flags;

	CurrentSnapshot->ObjectCount++;
	return &Object;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectInfo* FObjectProvider::GetEditableObject(uint32 Id)
{
	EditAccessCheck();

	if (!ensure(CurrentSnapshot))
	{
		return nullptr;
	}

	if (!FObjectSnapshot::IsValidObjectId(Id))
	{
		return nullptr;
	}

	return CurrentSnapshot->GetEditableObject(Id);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectReferenceInfo* FObjectProvider::AddObjectReference(uint32 ReferencerId, uint32 ObjectId)
{
	EditAccessCheck();

	if (!ensure(CurrentSnapshot))
	{
		return nullptr;
	}

	if (!FObjectSnapshot::IsValidObjectId(ReferencerId) ||
		!FObjectSnapshot::IsValidObjectId(ObjectId))
	{
		return nullptr;
	}

	FObjectReferenceInfo& ReferenceInfo = CurrentSnapshot->References.AddDefaulted_GetRef();
	ReferenceInfo.ReferencerId = ReferencerId;
	ReferenceInfo.ObjectId = ObjectId;

	return &ReferenceInfo;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUtf8String FunctionFlagsToUtf8String(uint32 Flags)
{
	if (Flags == 0)
	{
		return FUtf8String();
	}

	const ANSICHAR* FlagNames[] =
	{
		"Final",
		"RequiredAPI",
		"BlueprintAuthorityOnly",
		"BlueprintCosmetic",
		"_UnusedBit4",
		"_UnusedBit5",
		"Net",
		"NetReliable",
		"NetRequest",
		"Exec",
		"Native",
		"Event",
		"NetResponse",
		"Static",
		"NetMulticast",
		"UbergraphFunction",
		"MulticastDelegate",
		"Public",
		"Private",
		"Protected",
		"Delegate",
		"NetServer",
		"HasOutParms",
		"HasDefaults",
		"NetClient",
		"DLLImport",
		"BlueprintCallable",
		"BlueprintEvent",
		"BlueprintPure",
		"EditorOnly",
		"Const",
		"NetValidate"
	};
	static_assert(UE_ARRAY_COUNT(FlagNames) == 32, "");

	bool bFirst = true;
	TUtf8StringBuilder<256> Str;
	for (int32 BitIndex = 0; BitIndex < 32; ++BitIndex)
	{
		if ((Flags & (1u << BitIndex)) != 0)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				Str.Append(" | ");
			}
			Str.Append(FlagNames[BitIndex]);
		}
	}
	return FUtf8String(Str.ToView());
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

