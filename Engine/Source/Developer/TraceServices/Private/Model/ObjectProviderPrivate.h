// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/ObjectProvider.h"

namespace TraceServices
{

extern thread_local FProviderLock::FThreadLocalState GObjectProviderLockState;

class FObjectSnapshot : public IObjectEditableSnapshot
{
	friend class FObjectProvider;

public:
	static constexpr uint32 MaxNumObjects = 1 << 25; // ~33 million objects
	static constexpr uint32 InvalidObjectId = uint32(-1);

public:
	FObjectSnapshot();
	virtual ~FObjectSnapshot();

	virtual uint32 GetId() const { return Id; }

	static inline bool IsValidObjectId(uint32 ObjectId)
	{
		return ObjectId < MaxNumObjects;
	}
	inline bool IsValidObject(uint32 ObjectId)
	{
		return ObjectId < uint32(Objects.Num()) ? Objects[ObjectId].Id == ObjectId : false;
	}

	virtual double GetStartTime() const override { return StartTime; }
	virtual double GetEndTime() const override { return EndTime; }

	virtual uint32 GetObjectArrayNum() const override { return uint32(Objects.Num()); }
	virtual uint32 GetObjectCount() const override { return ObjectCount; }
	virtual uint32 GetNumReferences() const override { return uint32(References.Num()); }

	virtual bool HasTotalMemorySizes() const override { return bHasTotalMemorySizes; }
	virtual void EnableTotalMemorySizes() override { bHasTotalMemorySizes = true; }

	virtual uint32 GetTracedObjectArrayNum() const override { return TracedObjectArrayNum; }
	virtual void SetTracedObjectArrayNum(uint32 Count) override { TracedObjectArrayNum = Count; }

	virtual uint32 GetTracedObjectArrayNumMinusAvailable() const override { return TracedObjectArrayNumMinusAvailable; }
	virtual void SetTracedObjectArrayNumMinusAvailable(uint32 Count) override { TracedObjectArrayNumMinusAvailable = Count; }

	virtual uint32 GetTracedObjectArrayNumPermanent() const override { return TracedObjectArrayNumPermanent; }
	virtual void SetTracedObjectArrayNumPermanent(uint32 Count) override { TracedObjectArrayNumPermanent = Count; }

	virtual const FObjectInfo* GetObject(uint32 ObjectId) const override
	{
		return ObjectId < uint32(Objects.Num()) && Objects[ObjectId].Id == ObjectId ? &Objects[ObjectId] : nullptr;
	}
	virtual FObjectInfo* GetEditableObject(uint32 ObjectId) override
	{
		return ObjectId < uint32(Objects.Num()) && Objects[ObjectId].Id == ObjectId ? &Objects[ObjectId] : nullptr;
	}

	virtual TConstArrayView<FObjectInfo> GetObjectsArray() const override { return Objects; }
	virtual TConstArrayView<FObjectReferenceInfo> GetReferencesArray() const override { return References; }

	inline bool IsEmpty() const
	{
		return Objects.IsEmpty() && References.IsEmpty();
	}

	FObjectInfo& GetOrAddObject(uint32 ObjectId);
	void EnumerateObjects(TFunctionRef<bool(const FObjectInfo&)> Callback) const;
	const FObjectInfo* FindObject(FStringView Name) const;

	void Validate();

	virtual void SaveAs(const TCHAR* CsvFileName) const;

private:
	void ValidateExpectedFlags(FObjectInfo& Obj, EObjectInfoFlags ExpectedFlags);
	uint32 ValidateClassesRec(const TArray<uint32>& Classes, uint32 ClassClassId, uint32 BaseClassId, EObjectInfoFlags ExpectedClassFlags);

private:
	uint32 Id; // snapshot's unique id
	double StartTime = 0.0;
	double EndTime = 0.0;
	bool bHasTotalMemorySizes = false;
	uint32 TracedObjectArrayNum = 0;
	uint32 TracedObjectArrayNumMinusAvailable = 0;
	uint32 TracedObjectArrayNumPermanent = 0;
	uint32 ObjectCount = 0;
	TArray<FObjectInfo> Objects;
	TArray<FObjectReferenceInfo> References;
};

class FObjectProvider : public IObjectProvider, public IEditableObjectProvider
{
public:
	explicit FObjectProvider(IAnalysisSession& InSession);
	virtual ~FObjectProvider();

	//////////////////////////////////////////////////
	// Read operations

	// IProvider
	virtual void BeginRead() const override { Lock.BeginRead(GObjectProviderLockState); }
	virtual void EndRead() const override { Lock.EndRead(GObjectProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GObjectProviderLockState); }

	// IObjectProvider
	virtual void EnumerateObjects(uint32 SnapshotId, TFunctionRef<bool(const FObjectInfo&)> Callback) const override;
	virtual const FObjectInfo* GetObject(uint32 SnapshotId, uint32 ObjectId) const override;
	virtual const FObjectInfo* FindObject(uint32 SnapshotId, FStringView Name) const override;
	virtual uint32 GetCurrentSnapshotObjectCount() const override;
	virtual uint32 GetCurrentSnapshotTotalObjectCount() const override;
	virtual uint32 GetNumSnapshots() const override;
	virtual void EnumerateSnapshots(TFunctionRef<bool(const IObjectSnapshot&)> Callback) const override;
	virtual const IObjectSnapshot* GetSnapshot(uint32 SnapshotId) const override;
	virtual bool IsCurrentSnapshotValid() const override;
	virtual const IObjectSnapshot* GetLowerBoundSnapshot(double Time) const override;

	//////////////////////////////////////////////////
	// Edit operations

	// IEditableProvider
	virtual void BeginEdit() const override { Lock.BeginWrite(GObjectProviderLockState); }
	virtual void EndEdit() const override { Lock.EndWrite(GObjectProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GObjectProviderLockState); }

	// IEditableObjectProvider
	virtual IObjectEditableSnapshot* GetCurrentSnapshot() override;
	virtual IObjectEditableSnapshot* BeginSnapshot(double Time) override;
	virtual IObjectEditableSnapshot* EndSnapshot(double Time) override;
	virtual FObjectInfo* AddObject(uint32 Id, uint32 ClassId, uint32 OuterId, uint32 Flags, const TCHAR* PersistentName) override;
	virtual FObjectInfo* GetEditableObject(uint32 Id) override;
	virtual FObjectReferenceInfo* AddObjectReference(uint32 ReferencerId, uint32 ObjectId) override;
	virtual const IObjectProvider* GetReadProvider() const override { return this; }

	//////////////////////////////////////////////////

private:
	void InternalReset();

private:
	mutable FProviderLock Lock;
	IAnalysisSession& Session;

	TArray<FObjectSnapshot*> Snapshots;
	FObjectSnapshot* CurrentSnapshot = nullptr;
};

} // namespace TraceServices
