// Copyright Epic Games, Inc. All Rights Reserved.

#include "PsymResolver.h"

// TraceServices
#include "Common/SymbolHelper.h"

#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Containers/Queue.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/Parse.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"

// TraceServices
#include "Common/SymbolStringAllocator.h"
#include "TraceServices/Model/AnalysisSession.h"

#include <atomic>

DEFINE_LOG_CATEGORY(LogPsymResolver);

/////////////////////////////////////////////////////////////////////

namespace TraceServices
{

static const TCHAR* GUnknownModuleTextPsym = TEXT("Unknown");

/////////////////////////////////////////////////////////////////////
FPsymResolver::FPsymResolver(IAnalysisSession& InSession, IResolvedSymbolFilter& InSymbolFilter)
	: bRunWorkerThread(false)
	, bDrainThenStop(false)
	, Session(InSession)
	, SymbolFilter(InSymbolFilter)
{
	// Setup search paths. Paths are searched in the following order:
	// 1. Any new path entered by the user this session
	// 2. Path of the executable (if available)
	// 3. Paths from UE_INSIGHTS_SYMBOL_PATH environment variable
	// 4. Paths from _NT_SYMBOL_PATH environment variable
	// 5. Paths from the user configuration file

	// Get the pre-configured symbol search paths (environment variables + user configuration file).
	FSymbolHelper::GetSymbolSearchPaths(LogPsymResolver, ConfigSymbolSearchPaths);

	Start();
}

/////////////////////////////////////////////////////////////////////
FPsymResolver::~FPsymResolver()
{
	bRunWorkerThread = false;
	if (Thread)
	{
		Thread->WaitForCompletion();
	}
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::QueueModuleLoad(const uint8* ImageId, uint32 ImageIdSize, FModule* Module)
{
	check(Module != nullptr);

	FScopeLock _(&ModulesCs);

	const FStringView ModuleName = FPathViews::GetCleanFilename(Module->FullName);

	// Add module and sort list according to base address
	const int32 Index = LoadedModules.Add(FModuleEntry{
		Module->Base, Module->Size, Session.StoreString(ModuleName), Session.StoreString(Module->FullName),
		Module, TArray(ImageId, ImageIdSize)
	});
	Algo::Sort(LoadedModules, [](const FModuleEntry& Lhs, const FModuleEntry& Rhs) { return Lhs.Base < Rhs.Base; });

	// Reset stats
	Module->Stats.Discovered.store(0u);
	Module->Stats.Resolved.store(0u);
	Module->Stats.Failed.store(0u);
	Module->Stats.Available.store(0u);

#if 0
	// TODO: search for psym file in the symbols search paths
	const TCHAR* PsymPath = nullptr;

	Module->Status.store(EModuleStatus::Pending);

	// Queue up module to have symbols loaded
	LoadSymbolsQueue.Enqueue(FQueuedModule{ Module, PsymPath, LoadedModules[Index].ImageId });
#endif

	++ModulesDiscovered;
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::QueueModuleReload(FModule* Module, const TCHAR* InPath, TFunction<void(SymbolArray&)> ResolveOnSuccess)
{
	check(Module != nullptr);

	FScopeLock _(&ModulesCs);

	const FModuleEntry* Entry = GetEntryForModule(Module);
	if (!Entry)
	{
		return;
	}
	check(Entry->Module == Module);

	// No use in trying reload already loaded modules
	if (Module->Status == EModuleStatus::Loaded)
	{
		return;
	}

	// Reset stats
	Module->Stats.Discovered.store(0u);
	Module->Stats.Resolved.store(0u);
	Module->Stats.Failed.store(0u);
	Module->Stats.Available.store(0u);

	Module->Status.store(EModuleStatus::Pending);

	// Queue up module to have symbols re-loaded
	LoadSymbolsQueue.Enqueue(FQueuedModule{ Module, Session.StoreString(InPath), TArrayView<const uint8>(Entry->ImageId) });

	SymbolArray SymbolsToResolve;
	ResolveOnSuccess(SymbolsToResolve);
	for(TTuple<uint64, FResolvedSymbol*> Pair : SymbolsToResolve)
	{
		QueueSymbolResolve(Pair.Get<0>(), Pair.Get<1>());
	}

	if (!bRunWorkerThread && Thread)
	{
		// Restart the worker thread if it has stopped.
		Start();
	}
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::QueueSymbolResolve(uint64 Address, FResolvedSymbol* Symbol)
{
	ResolveQueue.Enqueue(FQueuedAddress{Address, Symbol});
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::GetStats(IModuleProvider::FStats* OutStats) const
{
	FScopeLock _(&ModulesCs);
	FMemory::Memzero(*OutStats);
	for(const FModuleEntry& Entry : LoadedModules)
	{
		const FModule::SymbolStats& ModuleStats = Entry.Module->Stats;
		OutStats->SymbolsDiscovered += ModuleStats.Discovered.load();
		OutStats->SymbolsCached += ModuleStats.Cached.load();
		OutStats->SymbolsResolved += ModuleStats.Resolved.load();
		OutStats->SymbolsFailed += ModuleStats.Failed.load();
	}
	OutStats->SymbolsFailed += NoModuleSymbols;
	OutStats->ModulesDiscovered = ModulesDiscovered.load();
	OutStats->ModulesFailed = ModulesFailed.load();
	OutStats->ModulesLoaded = ModulesLoaded.load();
}

/////////////////////////////////////////////////////////////////////
bool FPsymResolver::HasFinishedResolving() const
{
	return LoadSymbolsQueue.IsEmpty() && ResolveQueue.IsEmpty();
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const
{
	{
		FReadScopeLock _(CustomSymbolSearchPathsLock);
		Algo::ForEach(CustomSymbolSearchPaths, Callback);
	}

	Algo::ForEach(ConfigSymbolSearchPaths, Callback);
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::OnAnalysisComplete()
{
	// At this point no more module loads or symbol requests will be queued,
	// we drain the current queue, then release resources and file locks.
	bDrainThenStop = true;
}

/////////////////////////////////////////////////////////////////////
uint32 FPsymResolver::Run()
{
	while (bRunWorkerThread)
	{
		// Prioritize queued module loads
		while (!LoadSymbolsQueue.IsEmpty() && bRunWorkerThread)
		{
			FQueuedModule Item;
			if (LoadSymbolsQueue.Dequeue(Item))
			{
				LoadModuleSymbols(Item.Module, Item.Path, Item.ImageId);
			}
		}

		// Resolve one symbol at a time to give way for modules
		while (!ResolveQueue.IsEmpty() && LoadSymbolsQueue.IsEmpty() && bRunWorkerThread)
		{
			FQueuedAddress Item;
			if (ResolveQueue.Dequeue(Item))
			{
				ResolveSymbol(Item.Address, *Item.Target);
			}
		}

		if (bDrainThenStop && ResolveQueue.IsEmpty() && LoadSymbolsQueue.IsEmpty())
		{
			bRunWorkerThread = false;
		}

		// ...and breathe...
		FPlatformProcess::Sleep(0.2f);
	}

	return 0;
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::Start()
{
	// Start the worker thread
	bRunWorkerThread = true;
	Thread = FRunnableThread::Create(this, TEXT("PSymHelpWorker"), 0, TPri_Normal);
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::Stop()
{
	bRunWorkerThread = false;
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::UpdateResolvedSymbol(FResolvedSymbol& Symbol, ESymbolQueryResult Result, const TCHAR* Module, const TCHAR* Name, const TCHAR* File, uint16 Line)
{
	Symbol.Module = Module;
	Symbol.Name = Name;
	Symbol.File = File;
	Symbol.Line = Line;
	Symbol.Result.store(Result, std::memory_order_release);
}

/////////////////////////////////////////////////////////////////////
const FPsymResolver::FModuleEntry* FPsymResolver::GetEntryForModule(const FModule* Module) const
{
	const int32 EntryIdx = Algo::BinarySearchBy(LoadedModules, Module->Base, [](const FModuleEntry& Entry) { return Entry.Base; });
	if (EntryIdx != INDEX_NONE)
	{
		return &LoadedModules[EntryIdx];
	}
	return nullptr;
}

/////////////////////////////////////////////////////////////////////
const FPsymResolver::FModuleEntry* FPsymResolver::GetModuleForAddress(uint64 Address) const
{
	const int32 EntryIdx = Algo::LowerBoundBy(LoadedModules, Address, [](const FModuleEntry& Entry) { return Entry.Base; }) - 1;
	if (EntryIdx >= 0 && EntryIdx < LoadedModules.Num())
	{
		return &LoadedModules[EntryIdx];
	}
	return nullptr;
}

/////////////////////////////////////////////////////////////////////

static inline bool IsWhiteSpace(TCHAR Value)
{
	return Value == TCHAR(' ') || Value == TCHAR('\n') || Value == TCHAR('\r');
}

/////////////////////////////////////////////////////////////////////
static FString NextToken(const FStringView& InStringView, int32& InIndex)
{
	FString Result;

	// prevent reallocs
	Result.Empty(32);

	// find first non-whitespace char
	while (InIndex < InStringView.Len() && IsWhiteSpace(InStringView[InIndex]))
	{
		InIndex++;
	}

	// copy non-whitespace chars
	while (InIndex < InStringView.Len() && (!IsWhiteSpace(InStringView[InIndex])))
	{
		Result += InStringView[InIndex];
		InIndex++;
	}

	return Result;
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::LoadModuleSymbols(const FModule* Module, const TCHAR* Path, const TArrayView<const uint8> ImageId)
{
	check(Module);

	if (!Path)
	{
		const_cast<FModule*>(Module)->Status.store(EModuleStatus::NotFound);
		return;
	}

	// Find the module entry
	FScopeLock _(&ModulesCs);
	const FModuleEntry* EntryPtr = GetEntryForModule(Module);
	check(EntryPtr && EntryPtr->Module == Module);
	const FModuleEntry& Entry = *EntryPtr;

	Entry.Module->Stats.Available.store(0u);

	uint64 BaseAddress = Module->Base;

	FSymbolStringAllocator StringAllocator(Session.GetLinearAllocator(), (64 << 10) / sizeof(TCHAR)); // using 64 KiB page size

	// https://github.com/google/breakpad/blob/master/docs/symbol_files.md
	//
	// Prefix	: Info								   : Number of spaces
	// ------------------------------------------------------------------
	// MODULE	: operatingsystem architecture id name : 4
	// FILE		: number name						   : 2
	// FUNC m	: address size parameter_size name	   : 5
	// FUNC		: address size parameter_size name	   : 4
	// address	: size line filenum					   : 3
	// PUBLIC m : address parameter_size name		   : 4
	// PUBLIC	: address parameter_size name		   : 3
	// STACK	:									   : 0 // Ignore
	// INFO		:									   : 0 // Ignore

	constexpr uint32 BuildIdSize = 16;
	uint8 BuildId[BuildIdSize] = { 0 };
	auto LineVisitor = [this, BaseAddress, &BuildId, BuildIdSize, Entry, &StringAllocator](FStringView Line)
		{
			int32 Index = 0;
			FString Command = NextToken(Line, Index);
			if (Command == TEXT("MODULE"))
			{
				FString OS = NextToken(Line, Index);
				FString Architecture = NextToken(Line, Index);
				FString BuildIdStr = NextToken(Line, Index);
				int i = 0;
				for (const TCHAR* Ptr = *BuildIdStr; *Ptr && *(Ptr+1) && i < BuildIdSize; Ptr+=2, i++)
				{
					BuildId[i] = (uint8)((FParse::HexDigit(*Ptr) << 4) + (FParse::HexDigit(*(Ptr+1))));
				}
				FString Name = NextToken(Line, Index);
			}
			else if (Command == TEXT("FILE"))
			{
				FString FileIndex = NextToken(Line, Index);
				FString FileName = NextToken(Line, Index);

				PsymSourceFiles.Add(FCString::Atoi(*FileIndex), StringAllocator.Store(*FileName));
			}
			else if (Command == TEXT("FUNC"))
			{
				constexpr int32 MaxSymbolLength = 255;

				FString Address = NextToken(Line, Index);
				if (Address == "m")
				{
					Address = NextToken(Line, Index);
				}

				FString Size = NextToken(Line, Index);
				FString ParamSize = NextToken(Line, Index);

				// find first non-whitespace char
				while (Index < Line.Len() && IsWhiteSpace(Line[Index]))
				{
					Index++;
				}
				FString Name = Index == Line.Len() ? FString(TEXT("")) : FString(Line.RightChop(Index).Left(MaxSymbolLength));

				++Entry.Module->Stats.Available;
				PsymSymbols.Add(FPsymSymbol(FParse::HexNumber64(*Address) + BaseAddress, FParse::HexNumber(*Size), StringAllocator.Store(*Name)));
			}
			else if (Command == TEXT("PUBLIC"))
			{
				FString Address = NextToken(Line, Index);
				if (Address == "m")
				{
					Address = NextToken(Line, Index);
				}
				FString ParamSize = NextToken(Line, Index);
				FString Name = NextToken(Line, Index);

				++Entry.Module->Stats.Available;
				PsymSymbols.Add(FPsymSymbol(FParse::HexNumber64(*Address) + BaseAddress, FParse::HexNumber(*ParamSize), StringAllocator.Store(*Name)));
			}
			else if (Command == TEXT("STACK"))
			{
				// ignore
			}
			else if (Command == TEXT("INFO"))
			{
				// ignore
			}
			else
			{
				FString Size = NextToken(Line, Index);
				FString LineNumber = NextToken(Line, Index);
				FString FileNumber = NextToken(Line, Index);

				PsymSourceLines.Add(FPsymLine(FParse::HexNumber64(*Command) + BaseAddress, FParse::HexNumber(*Size), FCString::Atoi(*LineNumber), FCString::Atoi(*FileNumber)));
			}
		};

	FFileHelper::LoadFileToStringWithLineVisitor(Path, LineVisitor);

	Algo::Sort(PsymSymbols, [](const FPsymSymbol& Left, const FPsymSymbol& Right)
		{
			return Left.Address < Right.Address;
		});
	Algo::Sort(PsymSourceLines, [](const FPsymLine& Left, const FPsymLine& Right)
		{
			return Left.Address < Right.Address;
		});

	TStringBuilder<256> StatusMessage;
	EModuleStatus Status;

	// Only check the first 16 bytes of the BuildId as the psym generator appears to add a trailing 0.
	if (FMemory::Memcmp(Entry.ImageId.GetData(), BuildId, FMath::Min<int>(Entry.ImageId.Num(), BuildIdSize)) != 0)
	{
		StatusMessage.Appendf(TEXT("Build ID of psym does not match that of trace! Is this the correct psym (%s) for module %s?"), Path, Module->Name);
		Status = EModuleStatus::VersionMismatch;
		++ModulesFailed;
	}
	else
	{
		StatusMessage.Append(Path);
		Status = EModuleStatus::Loaded;
		++ModulesLoaded;
	}

	// Make the status visible to the world
	Entry.Module->StatusMessage = Session.StoreString(StatusMessage.ToView());
	Entry.Module->Status.store(Status);

	uint64 BytesAllocated = StringAllocator.GetAllocatedSize();
	uint64 BytesUsed = StringAllocator.GetUsedSize();
	uint64 BytesWasted = BytesAllocated - BytesUsed;
	UE_LOGF(LogPsymResolver, VeryVerbose, "String allocator: %.02f KiB used (+ %.02f KiB wasted) in %d blocks",
		(double)BytesUsed / 1024.0,
		(double)BytesWasted / 1024.0,
		StringAllocator.GetNumAllocatedBlocks());
}

/////////////////////////////////////////////////////////////////////
void FPsymResolver::ResolveSymbol(uint64 Address, FResolvedSymbol& Target)
{
	const ESymbolQueryResult PreviousResult = Target.Result.load();
	if (PreviousResult == ESymbolQueryResult::OK)
	{
		++AlreadyResolvedSymbols;
		return;
	}

	const FModuleEntry* Entry = GetModuleForAddress(Address);
	if (!Entry)
	{
		UE_LOGF(LogPsymResolver, Warning, "No module mapped to address 0x%llX.", Address);
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotLoaded,
			GUnknownModuleTextPsym,
			GUnknownModuleTextPsym,
			GUnknownModuleTextPsym,
			0);
		SymbolFilter.Update(Target);
		++NoModuleSymbols;
		return;
	}

	FModule::SymbolStats& ModuleStats = Entry->Module->Stats;
	++ModuleStats.Discovered;

	if (Entry->Module->Status == EModuleStatus::Loaded)
	{
		int32 SymbolIndex = Algo::UpperBoundBy(PsymSymbols, Address, &FPsymSymbol::Address) - 1;
		if (PsymSymbols.IsValidIndex(SymbolIndex))
		{
			int32 LineIndex = Algo::UpperBoundBy(PsymSourceLines, Address, &FPsymLine::Address) - 1;

			int32 LineNumber = 0;
			const TCHAR* FileName = TEXT("?");
			if (PsymSourceLines.IsValidIndex(LineIndex))
			{
				LineNumber = PsymSourceLines[LineIndex].LineNumber;
				int32 FileIndex = PsymSourceLines[LineIndex].FileIndex;

				const TCHAR** FileNamePtr = PsymSourceFiles.Find(FileIndex);
				if (FileNamePtr)
				{
					FileName = *FileNamePtr;
				}
			}

			UpdateResolvedSymbol(Target,
				ESymbolQueryResult::OK,
				Entry->Name,
				PsymSymbols[SymbolIndex].Name,
				FileName,
				static_cast<uint16>(LineNumber));
			SymbolFilter.Update(Target);
			++ModuleStats.Resolved;
		}
		else
		{
			UpdateResolvedSymbol(Target,
				ESymbolQueryResult::NotFound,
				Entry->Name,
				GUnknownModuleTextPsym,
				GUnknownModuleTextPsym,
				0);
			SymbolFilter.Update(Target);
			++ModuleStats.Failed;
		}
	}
	else
	{
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotLoaded,
			Entry->Name,
			GUnknownModuleTextPsym,
			GUnknownModuleTextPsym,
			0);
		SymbolFilter.Update(Target);
		++ModuleStats.Failed;
	}
}

/////////////////////////////////////////////////////////////////////

} // namespace TraceServices
