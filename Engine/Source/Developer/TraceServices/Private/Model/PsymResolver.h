// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "HAL/Runnable.h"
#include "Containers/Queue.h"
#include "Async/MappedFileHandle.h"
#include "Async/TaskGraphInterfaces.h"
#include "Common/PagedArray.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Modules.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPsymResolver, Log, All);

namespace TraceServices
{
////////////////////////////////////////////////////////////////////////////////

class FPsymResolver : public FRunnable
{
public:
	typedef TArray<TTuple<uint64, FResolvedSymbol*>> SymbolArray;

	FPsymResolver(IAnalysisSession& InSession, IResolvedSymbolFilter& InSymbolFilter);
	~FPsymResolver();

	void Start();
	void QueueModuleLoad(const uint8* ImageId, uint32 ImageIdSize, FModule* Module);
	void QueueModuleReload(FModule* Module, const TCHAR* Path, TFunction<void(SymbolArray&)> ResolveOnSuccess);
	void QueueSymbolResolve(uint64 Address, FResolvedSymbol* Symbol);
	void OnAnalysisComplete();
	void GetStats(IModuleProvider::FStats* OutStats) const;
	bool HasFinishedResolving() const;
	void EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const;

private:
	struct FModuleEntry
	{
		uint64 Base;
		uint32 Size;
		const TCHAR* Name;
		const TCHAR* Path;
		FModule* Module;
		TArray<uint8> ImageId;
	};

	struct FQueuedAddress
	{
		uint64 Address;
		FResolvedSymbol* Target;
	};

	struct FQueuedModule
	{
		const FModule* Module;
		const TCHAR* Path;
		TArrayView<const uint8> ImageId;
	};

	enum : uint32 {
		MaxNameLen = 512,
	};

	virtual uint32 Run() override;
	virtual void Stop() override;
	static void UpdateResolvedSymbol(FResolvedSymbol& Symbol, ESymbolQueryResult Result, const TCHAR* Module, const TCHAR* Name, const TCHAR* File, uint16 Line);

	void ResolveSymbol(uint64 Address, FResolvedSymbol& Target);
	void LoadModuleSymbols(const FModule* Module, const TCHAR* Path, const TArrayView<const uint8> ImageId);
	const FModuleEntry* GetEntryForModule(const FModule* Module) const;
	const FModuleEntry* GetModuleForAddress(uint64 Address) const;

	mutable FCriticalSection ModulesCs;
	TArray<FModuleEntry> LoadedModules;
	TQueue<FQueuedModule, EQueueMode::Mpsc> LoadSymbolsQueue;
	TQueue<FQueuedAddress, EQueueMode::Mpsc> ResolveQueue;

	std::atomic<uint32> ModulesDiscovered;
	std::atomic<uint32> ModulesFailed;
	std::atomic<uint32> ModulesLoaded;
	std::atomic<uint32> AlreadyResolvedSymbols;
	std::atomic<uint32> NoModuleSymbols;

	mutable FRWLock CustomSymbolSearchPathsLock;
	TArray<FString> CustomSymbolSearchPaths; // search paths added by user from UI in the current session
	TArray<FString> ConfigSymbolSearchPaths; // search paths specified in environment variables and config files

	bool bRunWorkerThread;
	bool bDrainThenStop;
	UPTRINT Handle;
	IAnalysisSession& Session;
	IResolvedSymbolFilter& SymbolFilter;
	FRunnableThread* Thread;
	
	struct FPsymSymbol
	{
		uint64 Address;
		uint32 Size;
		const TCHAR* Name;

		FPsymSymbol(uint64 InAddress, uint32 InSize, const TCHAR* InName)
			: Address(InAddress)
			, Size(InSize)
			, Name(InName)
		{}
	};
	struct FPsymLine
	{
		uint64 Address;
		uint32 Size;
		int32 LineNumber;
		int32 FileIndex;

		FPsymLine(uint64 InAddress, uint32 InSize, int32 InLineNumber, int32 InFileIndex)
			: Address(InAddress)
			, Size(InSize)
			, LineNumber(InLineNumber)
			, FileIndex(InFileIndex)
		{}
	};

	TMap<int32, const TCHAR*> PsymSourceFiles;
	TArray<FPsymSymbol> PsymSymbols;
	TArray<FPsymLine> PsymSourceLines;
};

/////////////////////////////////////////////////////////////////////

} // namespace TraceServices