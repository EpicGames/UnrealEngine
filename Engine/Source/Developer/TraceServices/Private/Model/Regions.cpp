// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Regions.h"
#include "Model/RegionsPrivate.h"

#include "Algo/ForEach.h"
#include "Internationalization/Internationalization.h"

// TraceServices
#include "AnalysisServicePrivate.h"
#include "Common/FormatArgs.h"
#include "Common/Utils.h"

#define LOCTEXT_NAMESPACE "RegionProvider"

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GRegionsProviderLockState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// FRegionTimeline
////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FRegionTimeline::GetCategoryName() const
{
	Provider.ReadAccessCheck();
	return Category ? Category->Name : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FRegionCategory* FRegionTimeline::GetCategory() const
{
	Provider.ReadAccessCheck();
	return Category;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FRegionTimeline::GetLaneCount() const
{
	Provider.ReadAccessCheck();
	return Lanes.Num();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FRegionLane* FRegionTimeline::GetLane(int32 Index) const
{
	Provider.ReadAccessCheck();

	if (Index >= 0 && Index < Lanes.Num())
	{
		return &(Lanes[Index]);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionTimeline::EnumerateRegions(double IntervalStart, double IntervalEnd,
	TFunctionRef<bool(const FTimeRegion&)> Callback) const
{
	Provider.ReadAccessCheck();

	if (IntervalStart > IntervalEnd)
	{
		return false;
	}

	TArray<FLaneEnumeration> SearchLanes;
	for (const FRegionLane& Lane: Lanes)
	{
		const FInt32Interval OverlapRange = GetElementRangeOverlappingGivenRange<FTimeRegion>(Lane.Regions, IntervalStart, IntervalEnd,
			[](const FTimeRegion& r) { return r.BeginTime; },
			[](const FTimeRegion& r) { return r.EndTime; });

		if (OverlapRange.Min != -1 && OverlapRange.Size() >= 0)
		{
			FLaneEnumeration EnumLane = { Lane, OverlapRange.Min, OverlapRange.Max };
			SearchLanes.Add(EnumLane);
		}
	}

	SearchLanes.Heapify([](const FLaneEnumeration& A, const FLaneEnumeration& B)
	{
		return A < B;
	});

	while (SearchLanes.Num() > 0)
	{
		FLaneEnumeration& TopLane = SearchLanes.HeapTop();

		if (!Callback(TopLane.Lane.Regions[TopLane.IntervalStart]))
		{
			return false;
		}

		if (TopLane.IntervalStart >= TopLane.IntervalEnd)
		{
			SearchLanes.HeapPopDiscard();
		}
		else
		{
			TopLane.IntervalStart++;
			SearchLanes.Heapify([](const FLaneEnumeration& A, const FLaneEnumeration& B)
			{
				return A < B;
			});
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionTimeline::EnumerateRegionsBackwards(double IntervalEnd, double IntervalStart,
	TFunctionRef<bool(const FTimeRegion&)> Callback) const
{
	Provider.ReadAccessCheck();

	if (IntervalStart > IntervalEnd)
	{
		return false;
	}

	TArray<FLaneEnumeration> SearchLanes;
	for (const FRegionLane& Lane: Lanes)
	{
		const FInt32Interval OverlapRange = GetElementRangeOverlappingGivenRange<FTimeRegion>(Lane.Regions, IntervalStart, IntervalEnd,
			[](const FTimeRegion& r) { return r.BeginTime; },
			[](const FTimeRegion& r) { return r.EndTime; });

		if (OverlapRange.Min != -1 && OverlapRange.Size() >= 0)
		{
			FLaneEnumeration EnumLane = { Lane, OverlapRange.Min, OverlapRange.Max };
			SearchLanes.Add(EnumLane);
		}
	}

	SearchLanes.Heapify([](const FLaneEnumeration& A, const FLaneEnumeration& B)
	{
		return A > B;
	});

	while (SearchLanes.Num() > 0)
	{
		FLaneEnumeration& TopLane = SearchLanes.HeapTop();

		if (!Callback(TopLane.Lane.Regions[TopLane.IntervalEnd]))
		{
			return false;
		}

		if (TopLane.IntervalEnd <= TopLane.IntervalStart)
		{
			SearchLanes.HeapPopDiscard();
		}
		else
		{
			TopLane.IntervalEnd--;
			SearchLanes.Heapify([](const FLaneEnumeration& A, const FLaneEnumeration& B)
			{
				return A > B;
			});
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionTimeline::EnumerateLanes(TFunctionRef<void(const FRegionLane&, const int32)> Callback) const
{
	Provider.ReadAccessCheck();

	for (int32 LaneIndex = 0; LaneIndex < Lanes.Num(); ++LaneIndex)
	{
		Callback(Lanes[LaneIndex], LaneIndex);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimeRegion* FRegionTimeline::InsertNewRegion(const FRegionTimerImpl& Timer, double BeginTime, uint64 Id)
{
	Provider.EditAccessCheck();

	FTimeRegion Region;
	Region.Timer = &Timer;
	Region.BeginTime = BeginTime;
	Region.Id = Id;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Region.Text = Timer.Name;
	Region.Category = Timer.Category ? Timer.Category->Name : nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Region.Depth = CalculateRegionDepth(Region);

	if (Region.Depth == Lanes.Num())
	{
		Lanes.Emplace(Session.GetLinearAllocator());
	}
	FRegionLane& NewLane = Lanes[Region.Depth];
	NewLane.Regions.EmplaceBack(Region);

	FTimeRegion* NewOpenRegion = &(Lanes[Region.Depth].Regions.Last());

	return NewOpenRegion;
}

int32 FRegionTimeline::CalculateRegionDepth(const FTimeRegion& Region) const
{
	constexpr int32 DepthLimit = 100;
	int32 NewDepth = 0;

	// Find first free lane/depth
	while (NewDepth < DepthLimit)
	{
		if (!Lanes.IsValidIndex(NewDepth))
		{
			break;
		}

		const FTimeRegion& LastRegion = Lanes[NewDepth].Regions.Last();
		if (LastRegion.EndTime <= Region.BeginTime)
		{
			break;
		}
		NewDepth++;
	}

	ensureMsgf(NewDepth < DepthLimit, TEXT("Regions are nested too deep."));

	return NewDepth;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FRegionProvider
////////////////////////////////////////////////////////////////////////////////////////////////////


FRegionProvider::FRegionProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, Categories(InSession.GetLinearAllocator(), 1024)
	, Timers(InSession.GetLinearAllocator(), 1024)
	, AllRegionsTimeline(*this, InSession, nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionProvider::EnumerateCategories(TFunctionRef<bool(const FRegionCategory&)> Callback) const
{
	ReadAccessCheck();

	auto CategoryIterator = Categories.GetIterator();
	const FRegionCategory* Category = CategoryIterator.GetCurrentItem();
	while (Category)
	{
		if (!Callback(*Category))
		{
			return false;
		}
		Category = CategoryIterator.NextItem();
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionProvider::EnumerateTimers(TFunctionRef<bool(const FRegionTimer&)> Callback) const
{
	ReadAccessCheck();

	auto TimerIterator = Timers.GetIterator();
	const FRegionTimer* Timer = TimerIterator.GetCurrentItem();
	while (Timer)
	{
		if (!Callback(*Timer))
		{
			return false;
		}
		Timer = TimerIterator.NextItem();
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::EnumerateTimelinesByCategory(TFunctionRef<void(const IRegionTimeline&, const TCHAR*)> Callback) const
{
	ReadAccessCheck();

	auto CategoryIterator = Categories.GetIterator();
	const FRegionCategoryImpl* Category = CategoryIterator.GetCurrentItem();
	while (Category)
	{
		Callback(Category->Timeline, Category->Name);
		Category = CategoryIterator.NextItem();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IRegionTimeline& FRegionProvider::GetDefaultTimeline() const
{
	ReadAccessCheck();

	return AllRegionsTimeline;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IRegionTimeline* FRegionProvider::GetTimelineForCategory(const TCHAR* CategoryName) const
{
	ReadAccessCheck();

	if (!CategoryName)
	{
		return &AllRegionsTimeline;
	}

	FRegionCategoryImpl* const* FoundCategory = CategoriesByName.Find(CategoryName);
	if (FoundCategory)
	{
		return &(*FoundCategory)->Timeline;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FRegionProvider::GetRegionCount() const
{
	ReadAccessCheck();

	uint64 RegionCount = 0;
	AllRegionsTimeline.EnumerateLanes([&RegionCount](const FRegionLane& Lane, const int32 Index) { RegionCount += Lane.Num(); });
	return RegionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FRegionLane* FRegionProvider::GetLane(int32 index) const
{
	ReadAccessCheck();

	return AllRegionsTimeline.GetLane(index);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FRegionCategoryImpl& FRegionProvider::GetOrAddRegionCategory(const TCHAR* CategoryName)
{
	check(CategoryName);

	FRegionCategoryImpl** FoundCategory = CategoriesByName.Find(CategoryName);
	if (FoundCategory)
	{
		return **FoundCategory;
	}

	FRegionCategoryImpl& Category = Categories.EmplaceBack(*this, Session);
	Category.Name = CategoryName;
	CategoriesByName.Add(CategoryName, &Category);

	return Category;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FRegionTimerImpl& FRegionProvider::GetOrAddRegionTimer(const TCHAR* TimerName, const FRegionCategory* Category)
{
	check(TimerName);

	FRegionTimerImpl** FoundTimer = TimersByName.Find(TimerName);
	if (FoundTimer)
	{
		return **FoundTimer;
	}

	FRegionTimerImpl& Timer = Timers.EmplaceBack();
	Timer.Name = TimerName;
	Timer.Category = Category;
	TimersByName.Add(TimerName, &Timer);

	return Timer;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FRegionProvider::FOpenRegions FRegionProvider::AddRegionToTimelines(double BeginTime, const TCHAR* Name, uint64 Id, const TCHAR* CategoryName)
{
	const TCHAR* StoredCategoryName = CategoryName ? Session.StoreString(CategoryName) : UncategorizedName;
	FRegionCategoryImpl& Category = GetOrAddRegionCategory(StoredCategoryName);

	const TCHAR* StoredName = Session.StoreString(Name);
	FRegionTimerImpl& Timer = GetOrAddRegionTimer(StoredName, &Category);

	FOpenRegions OpenRegions;

	// Insert a region in the AllRegions timeline.
	OpenRegions.Region = AllRegionsTimeline.InsertNewRegion(Timer, BeginTime, Id);
	check(OpenRegions.Region);

	// Also insert a region in the category timeline.
	OpenRegions.CategoryRegion = Category.Timeline.InsertNewRegion(Timer, BeginTime, Id);
	check(OpenRegions.CategoryRegion);

	return OpenRegions;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::AppendRegionBegin(const TCHAR* Name, double Time, const TCHAR* CategoryName)
{
	EditAccessCheck();

	check(Name);

	FOpenRegions* FoundOpenRegions = OpenRegionsByName.Find(Name);

	if (FoundOpenRegions)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogTraceServices, Warning, "[Regions] A region begin event (BeginTime=%f, Name=\"%ls\", Category=\"%ls\") was encountered while a region with same name is already open.",
				Time, Name, CategoryName ? CategoryName : TEXT(""))
		}

		// Automatically end the previous region.
		AppendRegionEnd(Name, Time);
	}

	FOpenRegions OpenRegions = AddRegionToTimelines(Time, Name, 0, CategoryName);
	OpenRegionsByName.Add(OpenRegions.Region->Timer->Name, OpenRegions);

	UpdateCounter++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::AppendRegionBeginWithId(const TCHAR* Name, uint64 Id, double Time, const TCHAR* CategoryName)
{
	EditAccessCheck();

	check(Name && Id);

	FOpenRegions* FoundOpenRegions = OpenRegionsById.Find(Id);

	if (FoundOpenRegions)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogTraceServices, Warning, "[Regions] A region begin event (BeginTime=%f, Id=%llu, Name=\"%ls\", Category=\"%ls\") was encountered while a region with same name is already open.",
				Time, Id, Name, CategoryName ? CategoryName : TEXT(""))
		}

		// Automatically end the previous region.
		AppendRegionEndWithId(Id, Time);
	}

	FOpenRegions OpenRegions = AddRegionToTimelines(Time, Name, Id, CategoryName);
	OpenRegionsById.Add(Id, OpenRegions);

	UpdateCounter++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::AppendRegionEnd(const TCHAR* Name, double Time)
{
	EditAccessCheck();

	check(Name)
	FOpenRegions* OpenRegionsPos = OpenRegionsByName.Find(Name);

	if (!OpenRegionsPos)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogTraceServices, Warning, "[Regions] A region end event (EndTime=%f, Name=\"%ls\") was encountered without having seen a matching region begin event first.", Time, Name)
		}

		AppendRegionBegin(Name, Time);
		OpenRegionsPos = OpenRegionsByName.Find(Name);
		check(OpenRegionsPos);
	}

	check(OpenRegionsPos->Region);
	OpenRegionsPos->Region->EndTime = Time;

	check(OpenRegionsPos->CategoryRegion);
	OpenRegionsPos->CategoryRegion->EndTime = Time;

	OpenRegionsByName.Remove(Name);

	UpdateCounter++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::AppendRegionEndWithId(uint64 Id, double Time)
{
	EditAccessCheck();

	if (!Id)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogTraceServices, Warning, "[Regions] A region end event with id 0 was encountered, ignoring (EndTime=%f.", Time)
		}
		return;
	}

	FOpenRegions* OpenRegionsPos = OpenRegionsById.Find(Id);

	if (!OpenRegionsPos)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogTraceServices, Warning, "[Regions] A region end event (EndTime=%f, Id=%llu) was encountered without having seen a matching region begin event first.", Time, Id)
		}

		// Automatically create a new region.
		// Generates a display name if we're missing a begin and are closing by ID
		FString GeneratedName = FString::Printf(TEXT("Unknown Region (missing begin, Id=%llu)"), Id);
		AppendRegionBeginWithId(*GeneratedName, Id, Time);
		OpenRegionsPos = OpenRegionsById.Find(Id);
		check(OpenRegionsPos);
	}

	check(OpenRegionsPos->Region);
	OpenRegionsPos->Region->EndTime = Time;

	check(OpenRegionsPos->CategoryRegion);
	OpenRegionsPos->CategoryRegion->EndTime = Time;

	OpenRegionsById.Remove(Id);

	UpdateCounter++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::OnAnalysisSessionEnded()
{
	EditAccessCheck();

	auto PrintOpenRegionMessage = [this](const auto& KV)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			check(KV.Value.Region);
			const FTimeRegion& Region = *KV.Value.Region;
			check(Region.Timer && Region.Timer->Name);
			UE_LOGF(LogTraceServices, Warning, "[Regions] A region (BeginTime=%f, Id=%llu, Name=\"%ls\", Category=\"%ls\") was never closed.",
				Region.BeginTime,
				Region.Id,
				Region.Timer->Name,
				Region.Timer->Category && Region.Timer->Category->Name ? Region.Timer->Category->Name : TEXT(""));
		}
	};
	Algo::ForEach(OpenRegionsById, PrintOpenRegionMessage);
	Algo::ForEach(OpenRegionsByName, PrintOpenRegionMessage);

	if (NumWarnings > 0)
	{
		UE_LOGF(LogTraceServices, Warning, "[Regions] %u warnings", NumWarnings);
	}
	if (NumErrors > 0)
	{
		UE_LOGF(LogTraceServices, Error, "[Regions] %u errors", NumErrors);
	}

	const uint64 TotalRegionCount = GetRegionCount();
	UE_LOGF(LogTraceServices, Log, "[Regions] Analysis completed (%llu timers, %llu categories, %llu timing regions, %d lanes).",
		Timers.Num(), Categories.Num(), TotalRegionCount, AllRegionsTimeline.Lanes.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRegionProvider::EnumerateLanes(TFunctionRef<void(const FRegionLane&, int32)> Callback) const
{
	ReadAccessCheck();
	AllRegionsTimeline.EnumerateLanes(Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionProvider::EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion&)> Callback) const
{
	ReadAccessCheck();
	return AllRegionsTimeline.EnumerateRegions(IntervalStart, IntervalEnd, Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FRegionLane
////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionLane::EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion&)> Callback) const
{
	const FInt32Interval OverlapRange = GetElementRangeOverlappingGivenRange<FTimeRegion>(Regions, IntervalStart, IntervalEnd,
		[](const FTimeRegion& r) { return r.BeginTime; },
		[](const FTimeRegion& r) { return r.EndTime; });

	if (OverlapRange.Min == -1)
	{
		return true;
	}

	for (int32 Index = OverlapRange.Min; Index <= OverlapRange.Max; ++Index)
	{
		if (!Callback(Regions[Index]))
		{
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRegionLane::EnumerateRegionsBackwards(double IntervalEnd, double IntervalStart, TFunctionRef<bool(const FTimeRegion&)> Callback) const
{
	const FInt32Interval OverlapRange = GetElementRangeOverlappingGivenRange<FTimeRegion>(Regions, IntervalStart, IntervalEnd,
		[](const FTimeRegion& r) { return r.BeginTime; },
		[](const FTimeRegion& r) { return r.EndTime; });

	if (OverlapRange.Min == -1)
	{
		return true;
	}

	for (int32 Index = OverlapRange.Max; Index >= OverlapRange.Min; --Index)
	{
		if (!Callback(Regions[Index]))
		{
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

FName GetRegionProviderName()
{
	static const FName Name("RegionProvider");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IRegionProvider& ReadRegionProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<IRegionProvider>(GetRegionProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IEditableRegionProvider& EditRegionProvider(IAnalysisSession& Session)
{
	return *Session.EditProvider<IEditableRegionProvider>(GetRegionProviderName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#undef LOCTEXT_NAMESPACE
