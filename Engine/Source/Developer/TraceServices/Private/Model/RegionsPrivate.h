// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

// TraceServices
#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Regions.h"

namespace TraceServices
{

class FRegionProvider;
struct FRegionCategoryImpl;
struct FRegionTimerImpl;

extern thread_local FProviderLock::FThreadLocalState GRegionsProviderLockState;

/**
 * A View onto a set of Timing Regions, sorted into a stack of individual lanes without overlaps for display,
 * optionally filtered by Category
 */
class FRegionTimeline : public IRegionTimeline
{
public:
	virtual ~FRegionTimeline() = default;

	FRegionTimeline(FRegionProvider& InProvider, IAnalysisSession& InSession, const FRegionCategoryImpl* InCategory)
		: Provider(InProvider)
		, Session(InSession)
		, Category(InCategory)
	{}

	/**
	 * @return the category name of this timeline if filtered or nullptr
	 */
	const TCHAR* GetCategoryName() const override;

	/**
	 * @return the category of this timeline if filtered or nullptr
	 */
	const FRegionCategory* GetCategory() const override;

	/**
	 * @return the number of lanes
	 */
	int32 GetLaneCount() const override;

	/**
	 * Direct access to a certain lane at a given index/depth.
	 * The pointer is valid only in the current read scope.
	 * @return a pointer to the lane at the specified depth index or nullptr if Index is invalid
	 */
	const FRegionLane* GetLane(int32 Index) const override;

	/**
	 * Enumerates all regions that overlap a certain time interval. Will enumerate by depth but does not expose lanes.
	 *
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion& /*Region*/)> Callback) const override;

	/**
	 * Enumerates all regions that overlap a certain time interval in reverse. Will enumerate by depth but does not expose lanes.
	 *
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	bool EnumerateRegionsBackwards(double IntervalEnd, double IntervalStart, TFunctionRef<bool(const FTimeRegion& /*Region*/)> Callback) const override;

	/**
	 * Will call Callback(Lane, Depth) for each lane in order.
	 */
	void EnumerateLanes(TFunctionRef<void(const FRegionLane& /*Lane*/, const int32 /*Depth*/)> Callback) const override;

private:
	friend class FRegionProvider;

	/**
	 * Inserts a new Region into the stack, returns the created Region to allow the update of the region end time when it closes.
	 */
	FTimeRegion* InsertNewRegion(const FRegionTimerImpl& Timer, double BeginTime, uint64 Id);

	/**
	 * Update the depth member of a region to allow overlapping regions to be displayed on separate lanes.
	 */
	int32 CalculateRegionDepth(const FTimeRegion& Item) const;

private:
	// The parent provider. Required to engage the ReadAccessLock or WriteAccessLock when a Lane is modified.
	TraceServices::FRegionProvider& Provider;

	TraceServices::IAnalysisSession& Session;

	const FRegionCategoryImpl* Category = nullptr;

	// Closed regions
	TArray<FRegionLane> Lanes;
};

struct FRegionCategoryImpl : public FRegionCategory
{
	FRegionTimeline Timeline;

	FRegionCategoryImpl(FRegionProvider& InProvider, IAnalysisSession& InSession)
		: Timeline(InProvider, InSession, this)
	{
	}
};

struct FRegionTimerImpl : public FRegionTimer
{
};

class FRegionProvider
	: public IRegionProvider
	, public IEditableRegionProvider
{
public:
	explicit FRegionProvider(IAnalysisSession& Session);
	virtual ~FRegionProvider() override = default;

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override       { Lock.BeginRead(GRegionsProviderLockState); }
	virtual void EndRead() const override         { Lock.EndRead(GRegionsProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GRegionsProviderLockState); }

	virtual bool EnumerateCategories(TFunctionRef<bool(const FRegionCategory&)> Callback) const override;
	virtual bool EnumerateTimers(TFunctionRef<bool(const FRegionTimer&)> Callback) const override;

	virtual void EnumerateTimelinesByCategory(TFunctionRef<void(const IRegionTimeline& /*Timeline*/, const TCHAR* /*Category*/)> Callback) const override;
	virtual const IRegionTimeline& GetDefaultTimeline() const override;
	virtual const IRegionTimeline* GetTimelineForCategory(const TCHAR* Category) const override;
	virtual const TCHAR* GetUncategorizedRegionCategoryName() const override {return UncategorizedName;};

	virtual uint64 GetRegionCount() const override;
	virtual int32 GetLaneCount()  const override { ReadAccessCheck(); return AllRegionsTimeline.GetLaneCount(); }

	virtual const FRegionLane* GetLane(int32 Index) const override;

	virtual bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion&)> Callback) const override;

	virtual void EnumerateLanes(TFunctionRef<void(const FRegionLane&, const int32)> Callback) const override;

	virtual uint64 GetUpdateCounter() const override { ReadAccessCheck(); return UpdateCounter; }

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override       { Lock.BeginWrite(GRegionsProviderLockState); }
	virtual void EndEdit() const override         { Lock.EndWrite(GRegionsProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GRegionsProviderLockState); }

	virtual const FRegionCategory* RegisterRegionCategory(const TCHAR* Name) override
	{
		EditAccessCheck();
		return &GetOrAddRegionCategory(Name);
	}
	virtual const FRegionTimer* RegisterRegionTimer(const TCHAR* Name, const FRegionCategory* Category = nullptr) override
	{
		EditAccessCheck();
		return &GetOrAddRegionTimer(Name, Category);
	}

	virtual void AppendRegionBegin(const TCHAR* Name, double Time, const TCHAR* Category = nullptr) override;
	virtual void AppendRegionBeginWithId(const TCHAR* Name, uint64 Id, double Time, const TCHAR* Category = nullptr) override;
	virtual void AppendRegionEnd(const TCHAR* Name, double Time) override;
	virtual void AppendRegionEndWithId(uint64 Id, double Time) override;

	virtual void OnAnalysisSessionEnded() override;

	//////////////////////////////////////////////////

private:
	FRegionCategoryImpl& GetOrAddRegionCategory(const TCHAR* CategoryName);
	FRegionTimerImpl& GetOrAddRegionTimer(const TCHAR* TimerName, const FRegionCategory* Category = nullptr);

	struct FOpenRegions
	{
		FTimeRegion* Region;
		FTimeRegion* CategoryRegion;
	};

	// Adds a new region to the AllRegions timeline and to the category timeline (which can be the uncategorized timeline).
	// @return The two added regions.
	FOpenRegions AddRegionToTimelines(double BeginTime, const TCHAR* Name, uint64 Id, const TCHAR* Category);

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;

	TPagedArray<FRegionCategoryImpl> Categories;
	TPagedArray<FRegionTimerImpl> Timers;

	TMap<FStringView, FRegionCategoryImpl*> CategoriesByName;
	TMap<FStringView, FRegionTimerImpl*> TimersByName;

	// Open regions inside lanes
	TMap<FStringView, FOpenRegions> OpenRegionsByName;
	TMap<uint64, FOpenRegions> OpenRegionsById;

	FRegionTimeline AllRegionsTimeline;

	// String maps don't support nullptr, so use this key internally for all regions where category is nullptr.
	const TCHAR* UncategorizedName = TEXT("Uncategorized");

	// Counter incremented each time region data changes during analysis
	uint64 UpdateCounter = -1;

	static constexpr uint32 MaxWarningMessages = 100;
	static constexpr uint32 MaxErrorMessages = 100;

	uint32 NumWarnings = 0;
	uint32 NumErrors = 0;
};

} // namespace TraceServices
