// Copyright Epic Games, Inc. All Rights Reserved.

#include "SbTree.h"

#include "Misc/FileHelper.h"
#include "Model/AllocationItem.h"
#include "TraceServices/Containers/Allocators.h"

#include <limits>

namespace TraceServices
{

#define USE_OFFSETTED_CELLS 1

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSbTreeUtils
////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FSbTreeUtils::GetMaxDepth(uint32 TotalColumns)
{
	return 32 - FMath::CountLeadingZeros(TotalColumns);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FSbTreeUtils::GetCellAtDepth(uint32 Column, uint32 Depth)
{
	// On depth D, the cell indices starts at 2^D-1 and increases by 2^(D+1).
	uint32 LeafIndex = Column * 2;
	uint32 K = 1 << Depth;
	return (LeafIndex & ~K) | (K - 1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FSbTreeUtils::GetCommonDepth(uint32 ColumnA, uint32 ColumnB)
{
	uint32 Xor = ColumnA ^ ColumnB;
	return FSbTreeUtils::GetMaxDepth(Xor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FSbTreeUtils::GetCellWidth(uint32 CellIndex)
{
	return ((CellIndex ^ (CellIndex + 1)) >> 1) + 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSbTreeCell
////////////////////////////////////////////////////////////////////////////////////////////////////

FSbTreeCell::FSbTreeCell(ILinearAllocator& InAllocator)
	: Allocator(InAllocator)
	//, Allocs(InAllocator, 1024)
	, MinStartEventIndex(std::numeric_limits<uint32>::max())
	, MaxEndEventIndex(0)
	, MinStartTime(std::numeric_limits<double>::max())
	, MaxEndTime(0.0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSbTreeCell::~FSbTreeCell()
{
	for (const FAllocationItem* Alloc : Allocs)
	{
		delete Alloc;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTreeCell::AddAlloc(const FAllocationItem* Alloc)
{
	Allocs.Add(Alloc);

	if (Alloc->StartEventIndex < MinStartEventIndex)
	{
		MinStartEventIndex = Alloc->StartEventIndex;
	}
	if (Alloc->EndEventIndex > MaxEndEventIndex)
	{
		MaxEndEventIndex = Alloc->EndEventIndex;
	}

	if (Alloc->StartTime < MinStartTime)
	{
		MinStartTime = Alloc->StartTime;
	}
	if (Alloc->EndTime > MaxEndTime)
	{
		MaxEndTime = Alloc->EndTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTreeCell::Query(TArray<const FAllocationItem*>& OutAllocs, const IAllocationsProvider::FQueryParams& Params) const
{
	switch (Params.Rule)
	{
	case IAllocationsProvider::EQueryRule::aAf: // active allocs at A
	{
		const double Time = Params.TimeA;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime <= Time && Time <= Alloc->EndTime)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::afA: // before
	{
		const double Time = Params.TimeA;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->EndTime <= Time)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::Aaf: // after
	{
		const double Time = Params.TimeA;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= Time)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::aAfB: // decline
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime <= TimeA && Alloc->EndTime >= TimeA && Alloc->EndTime <= TimeB)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBf: // growth
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeB)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::aAfaBf: // decline + growth
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if ((Alloc->StartTime <= TimeA && Alloc->EndTime >= TimeA && Alloc->EndTime <= TimeB) || // decline
				(Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeB)) // growth
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AfB: // free events
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->EndTime >= TimeA && Alloc->EndTime <= TimeB)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AaB: // alloc events
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AafB: // short living allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= TimeA && Alloc->EndTime <= TimeB)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::aABf: // long living allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime <= TimeA && Alloc->EndTime >= TimeB)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBCf: // memory leaks
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeC)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBfC: // limited lifetime
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeB && Alloc->EndTime <= TimeC)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::aABfC: // decline of long living allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime <= TimeA && Alloc->EndTime >= TimeB && Alloc->EndTime <= TimeC)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AaBCfD: // specific lifetime
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		const double TimeC = Params.TimeC;
		const double TimeD = Params.TimeD;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if (Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeC && Alloc->EndTime <= TimeD)
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AoB: // paged out allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if ((Alloc->IsSwap() && Alloc->StartTime >= TimeA && Alloc->StartTime <= TimeB) ||
				(!Alloc->IsSwap() && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeA)) // include allocs that were freed between A and B
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	case IAllocationsProvider::EQueryRule::AiB: // paged in allocs
	{
		const double TimeA = Params.TimeA;
		const double TimeB = Params.TimeB;
		for (const FAllocationItem* Alloc : Allocs)
		{
			if ((Alloc->IsSwap() && Alloc->EndTime >= TimeA && Alloc->EndTime <= TimeB) ||
				(!Alloc->IsSwap() && Alloc->StartTime <= TimeB && Alloc->EndTime >= TimeA)) // include allocs that were freed between A and B
			{
				OutAllocs.Add(Alloc);
			}
		}
	}
	break;

	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSbTree
////////////////////////////////////////////////////////////////////////////////////////////////////

FSbTree::FSbTree(ILinearAllocator& InAllocator, uint32 InColumnShift)
	: Allocator(InAllocator)
	, ColumnShift(InColumnShift)
	, CurrentColumn(0)
{
	// Add the first cell.
	Cells.Add(nullptr);

#if USE_OFFSETTED_CELLS
	// Add the first offsetted cell.
	OffsettedCells.Add(nullptr);
#endif // USE_OFFSETTED_CELLS
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSbTree::~FSbTree()
{
	for (const FSbTreeCell* CellPtr : Cells)
	{
		if (CellPtr != nullptr)
		{
			delete CellPtr;
		}
	}
	Cells.Reset();

#if USE_OFFSETTED_CELLS
	for (const FSbTreeCell* CellPtr : OffsettedCells)
	{
		if (CellPtr != nullptr)
		{
			delete CellPtr;
		}
	}
	OffsettedCells.Reset();
#endif // USE_OFFSETTED_CELLS
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::SetTimeForEvent(uint32 EventIndex, double Time)
{
	// Detect the first event of each new column.
	if ((EventIndex & ((1 << ColumnShift) - 1)) == 0)
	{
		if (static_cast<uint32>(ColumnStartTimes.Num()) == (EventIndex >> ColumnShift))
		{
			ColumnStartTimes.Add(Time);
		}
		else
		{
			check(static_cast<uint32>(ColumnStartTimes.Num()) == (EventIndex >> ColumnShift) + 1);
			check(ColumnStartTimes.Last() == Time);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::AddAlloc(const FAllocationItem* Alloc)
{
	check(Alloc != nullptr);

	check(Alloc->EndTime >= LastAllocEndTime);
	LastAllocEndTime = Alloc->EndTime;

	uint32 StartColumn = Alloc->StartEventIndex >> ColumnShift;
	uint32 EndColumn = Alloc->EndEventIndex >> ColumnShift;

	if (EndColumn > CurrentColumn)
	{
		check(static_cast<uint32>(Cells.Num()) > (CurrentColumn << 1));
		check(static_cast<uint32>(ColumnStartTimes.Num()) == EndColumn + 1);

		// Adds 2 cells for each new column.
		uint32 CellsToAdd = (EndColumn - CurrentColumn) << 1;

		// We are only adding cell slots. A cell is created later, only if we add an alloc to it.
		Cells.AddDefaulted(CellsToAdd);
#if USE_OFFSETTED_CELLS
		OffsettedCells.AddDefaulted(CellsToAdd);
#endif // USE_OFFSETTED_CELLS

		CurrentColumn = EndColumn;

		//TODO: "Close" some of the "open" cells.
	}

#if USE_OFFSETTED_CELLS
	uint32 Depth;
	uint32 CellIndex;
	bool bUseOffsettedCells;

	const uint32 DeltaColumns = EndColumn - StartColumn;
	if (DeltaColumns == 0)
	{
		Depth = 0;
		CellIndex = StartColumn << 1;
		bUseOffsettedCells = false;
	}
	else
	{
		Depth = 32 - FMath::CountLeadingZeros(DeltaColumns);
		uint32 HalfCellWidth = (1 << Depth) >> 1;

		// In a cell, each alloc has start event column only in the first half of the cell and the end event column in the second half of the cell.
		// For a column, the bit indicating the "type of the half cell" at a certain depth ("start columns half cell" or "end columns half cell")
		// is Column & HalfCellWidth. This is also true for offsetted cells (just that for normal cells 0 means "start columns half cell"
		// while in offsetted cells 1 means "start columns half cell").

		// If the "type of the half cell" bit is the same for StartColumn and for EndColumn on the min depth (computed based on the column width for this alloc)
		// it means that the alloc doesn't fit in the (alligned) cells on this depth (neither in offsetted cells on this depth).
		// Those allocs are pushed to the next depth.
		if ((StartColumn & HalfCellWidth) == (EndColumn & HalfCellWidth))
		{
			++Depth;
			HalfCellWidth <<= 1;
		}

		CellIndex = FSbTreeUtils::GetCellAtDepth(StartColumn & ~HalfCellWidth, Depth);
		bUseOffsettedCells = ((StartColumn & HalfCellWidth) != 0);
	}
#else // USE_OFFSETTED_CELLS
	const uint32 Depth = FSbTreeUtils::GetCommonDepth(StartColumn, EndColumn);
	const uint32 CellIndex = FSbTreeUtils::GetCellAtDepth(StartColumn, Depth);
	const bool bUseOffsettedCells = false;
#endif // USE_OFFSETTED_CELLS

	FSbTreeCell* CellPtr;

	if (!bUseOffsettedCells)
	{
		check(CellIndex < static_cast<uint32>(Cells.Num()));
		CellPtr = Cells[CellIndex];
		if (CellPtr == nullptr)
		{
			// Create new cell.
			CellPtr = new FSbTreeCell(Allocator);
			Cells[CellIndex] = CellPtr;
		}
	}
#if USE_OFFSETTED_CELLS
	else
	{
		check((CellIndex & 1) != 0); // depth 0 doesn't use offsetted cells

		check(CellIndex < static_cast<uint32>(OffsettedCells.Num()));
		CellPtr = OffsettedCells[CellIndex];
		if (CellPtr == nullptr)
		{
			// Create new cell.
			CellPtr = new FSbTreeCell(Allocator);
			OffsettedCells[CellIndex] = CellPtr;
		}
	}
#endif // USE_OFFSETTED_CELLS

	CellPtr->AddAlloc(Alloc);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FSbTree::GetColumnsAtTime(double Time, int32* OutStartColumn, int32* OutEndColumn) const
{
	// Returns the first column with T >= Time.
	// [0 .. N]
	const int32 LowerBoundColumn = Algo::LowerBound(ColumnStartTimes, Time);

	if (OutStartColumn)
	{
		*OutStartColumn = LowerBoundColumn - 1; // [-1 .. N-1]
	}

	if (OutEndColumn)
	{
		int32 Column = LowerBoundColumn - 1; // [-1 .. N-1]
		const int32 Last = ColumnStartTimes.Num() - 1; // N-1
		while (Column < Last && ColumnStartTimes[Column + 1] == Time)
		{
			++Column;
		}
		*OutEndColumn = Column; // [-1 .. N-1]
	}

	return LowerBoundColumn;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::Query(TArray<const FSbTreeCell*>& OutCells, const IAllocationsProvider::FQueryParams& Params) const
{
	switch (Params.Rule)
	{
		case IAllocationsProvider::EQueryRule::aAf: // active allocs at A
		{
			int32 Column1, Column2;
			GetColumnsAtTime(Params.TimeA, &Column1, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::afA: // before
		{
			int32 Column2;
			GetColumnsAtTime(Params.TimeA, nullptr, &Column2);
			IterateCells(OutCells, 0, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::Aaf: // after
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeA, &Column1, nullptr);
			IterateCells(OutCells, Column1, CurrentColumn);
		}
		break;

		case IAllocationsProvider::EQueryRule::aAfB: // decline
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeA, &Column1, nullptr);
			int32 Column2;
			GetColumnsAtTime(Params.TimeB, nullptr, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::AaBf: // growth
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeB, &Column1, nullptr);
			IterateCells(OutCells, Column1, CurrentColumn);
		}
		break;

		case IAllocationsProvider::EQueryRule::aAfaBf: // decline + growth
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeA, &Column1, nullptr);
			IterateCells(OutCells, Column1, CurrentColumn);
		}
		break;

		case IAllocationsProvider::EQueryRule::AfB: // free events
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeA, &Column1, nullptr);
			int32 Column2;
			GetColumnsAtTime(Params.TimeB, nullptr, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::AaB: // alloc events
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeA, &Column1, nullptr);
			IterateCells(OutCells, Column1, CurrentColumn);
		}
		break;

		case IAllocationsProvider::EQueryRule::AafB: // short living allocs
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeA, &Column1, nullptr);
			int32 Column2;
			GetColumnsAtTime(Params.TimeB, nullptr, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::aABf: // long living allocs
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeB, &Column1, nullptr);
			IterateCells(OutCells, Column1, CurrentColumn);
		}
		break;

		case IAllocationsProvider::EQueryRule::AaBCf: // memory leaks
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeC, &Column1, nullptr);
			IterateCells(OutCells, Column1, CurrentColumn);
		}
		break;

		case IAllocationsProvider::EQueryRule::AaBfC: // limited lifetime
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeB, &Column1, nullptr);
			int32 Column2;
			GetColumnsAtTime(Params.TimeC, nullptr, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::aABfC: // decline of long living allocs
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeB, &Column1, nullptr);
			int32 Column2;
			GetColumnsAtTime(Params.TimeC, nullptr, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;

		case IAllocationsProvider::EQueryRule::AaBCfD: // specific lifetime
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeC, &Column1, nullptr);
			int32 Column2;
			GetColumnsAtTime(Params.TimeD, nullptr, &Column2);
			IterateCells(OutCells, Column1, Column2);
		}
		break;
		
		case IAllocationsProvider::EQueryRule::AoB:
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeB, nullptr, &Column1); // we care about all allocs up to Time B
			IterateCells(OutCells, 0, Column1);
		}
		break;

		case IAllocationsProvider::EQueryRule::AiB:
		{
			int32 Column1;
			GetColumnsAtTime(Params.TimeB, nullptr, &Column1); // we care about all allocs up to Time B
			IterateCells(OutCells, 0, Column1);
		}
		break;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::IterateCells(TArray<const FSbTreeCell*>& OutCells, int32 Column) const
{
	if (Column < 0 || Column > (int32)CurrentColumn)
	{
		// not an error; just early out
		return;
	}

	const uint32 LocalColumn = static_cast<uint32>(Column);

	const uint32 MaxDepth = FSbTreeUtils::GetMaxDepth(CurrentColumn);

	const uint32 NumCells = static_cast<uint32>(Cells.Num());
	for (uint32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
		const uint32 CellIndex = FSbTreeUtils::GetCellAtDepth(LocalColumn, Depth);
		if (CellIndex < NumCells)
		{
			const FSbTreeCell* CellPtr = Cells[CellIndex];
			if (CellPtr)
			{
				OutCells.Add(CellPtr);
			}
		}
	}

#if USE_OFFSETTED_CELLS
	const uint32 NumOffsettedCells = static_cast<uint32>(OffsettedCells.Num());
	for (uint32 Depth = 1; Depth <= MaxDepth; ++Depth) // offsetted cells doesn't exist on Depth 0
	{
		const uint32 HalfCellWidth = (1 << Depth) >> 1;
		if (HalfCellWidth > LocalColumn)
		{
			break;
		}
		const uint32 CellIndex = FSbTreeUtils::GetCellAtDepth(LocalColumn - HalfCellWidth, Depth);
		if (CellIndex < NumOffsettedCells)
		{
			const FSbTreeCell* CellPtr = OffsettedCells[CellIndex];
			if (CellPtr)
			{
				OutCells.Add(CellPtr);
			}
		}
	}
#endif // USE_OFFSETTED_CELLS
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::IterateCells(TArray<const FSbTreeCell*> & OutCells, int32 StartColumn, int32 EndColumn) const
{
	if (StartColumn == EndColumn)
	{
		IterateCells(OutCells, StartColumn);
		return;
	}

	if (EndColumn < 0 || StartColumn > (int32)CurrentColumn || StartColumn > EndColumn)
	{
		// not an error; just early out
		return;
	}

	const uint32 LocalStartColumn = (StartColumn < 0) ? 0 : static_cast<uint32>(StartColumn);
	const uint32 LocalEndColumn = (static_cast<uint32>(EndColumn) > CurrentColumn) ? CurrentColumn : static_cast<uint32>(EndColumn);
	check(LocalStartColumn <= LocalEndColumn);

	const uint32 MaxDepth = FSbTreeUtils::GetMaxDepth(CurrentColumn);

	const uint32 NumCells = static_cast<uint32>(Cells.Num());
	for (uint32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
		const uint32 FirstCellIndex = FSbTreeUtils::GetCellAtDepth(LocalStartColumn, Depth);
		const uint32 LastCellIndex = FMath::Min(FSbTreeUtils::GetCellAtDepth(LocalEndColumn, Depth) + 1, NumCells);
		const uint32 CellIncrement = 1 << (Depth + 1);
		for (uint32 CellIndex = FirstCellIndex; CellIndex < LastCellIndex; CellIndex += CellIncrement)
		{
			const FSbTreeCell* CellPtr = Cells[CellIndex];
			if (CellPtr)
			{
				OutCells.Add(CellPtr);
			}
		}
	}

#if USE_OFFSETTED_CELLS
	const uint32 NumOffsettedCells = static_cast<uint32>(OffsettedCells.Num());
	for (uint32 Depth = 1; Depth <= MaxDepth; ++Depth) // offsetted cells doesn't exist on Depth 0
	{
		const uint32 HalfCellWidth = (1 << Depth) >> 1;
		const uint32 OffsettedStartColumn = (LocalStartColumn > HalfCellWidth) ? LocalStartColumn - HalfCellWidth : 0;
		const uint32 FirstCellIndex = FSbTreeUtils::GetCellAtDepth(OffsettedStartColumn, Depth);
		const uint32 OffsettedEndColumn = (LocalEndColumn > HalfCellWidth) ? LocalEndColumn - HalfCellWidth : 0;
		const uint32 LastCellIndex = FMath::Min(FSbTreeUtils::GetCellAtDepth(OffsettedEndColumn, Depth) + 1, NumOffsettedCells);
		const uint32 CellIncrement = 1 << (Depth + 1);
		for (uint32 CellIndex = FirstCellIndex; CellIndex < LastCellIndex; CellIndex += CellIncrement)
		{
			const FSbTreeCell* CellPtr = OffsettedCells[CellIndex];
			if (CellPtr)
			{
				OutCells.Add(CellPtr);
			}
		}
	}
#endif // USE_OFFSETTED_CELLS
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::DebugPrint() const
{
	uint32 TotalColumns = CurrentColumn + 1;
	uint32 MaxDepth = FSbTreeUtils::GetMaxDepth(TotalColumns);

	uint32 TotalAllocs = 0;
	uint32 LocalMaxAllocCountPerCell = 0;

	uint32 NotEmptyCellCount = 0;
	const uint32 CellCount = static_cast<uint32>(Cells.Num());
	for (uint32 CellIndex = 0; CellIndex < CellCount; ++CellIndex)
	{
		const FSbTreeCell* CellPtr = Cells[CellIndex];
		if (CellPtr != nullptr)
		{
			++NotEmptyCellCount;
			const uint32 CellAllocCount = CellPtr->GetAllocCount();
			TotalAllocs += CellAllocCount;
			if (CellAllocCount > LocalMaxAllocCountPerCell)
			{
				LocalMaxAllocCountPerCell = CellAllocCount;
			}
		}
	}

	uint32 TotalCells = CellCount;

#if USE_OFFSETTED_CELLS
	uint32 NotEmptyOffsettedCellCount = 0;
	const uint32 OffsettedCellCount = static_cast<uint32>(OffsettedCells.Num());
	for (uint32 CellIndex = 0; CellIndex < OffsettedCellCount; ++CellIndex)
	{
		const FSbTreeCell* CellPtr = OffsettedCells[CellIndex];
		if (CellPtr != nullptr)
		{
			++NotEmptyOffsettedCellCount;
			const uint32 CellAllocCount = CellPtr->GetAllocCount();
			TotalAllocs += CellAllocCount;
			if (CellAllocCount > LocalMaxAllocCountPerCell)
			{
				LocalMaxAllocCountPerCell = CellAllocCount;
			}
		}
	}

	TotalCells += OffsettedCellCount;
#endif // USE_OFFSETTED_CELLS

	//#define SbTreePrint(x) FPlatformMisc::LowLevelOutputDebugString(TEXT(x))
	//#define SbTreePrintF(fmt, ...) FPlatformMisc::LowLevelOutputDebugStringf(TEXT(fmt), __VA_ARGS__)

	FString StringBuffer;
	#define SbTreePrint(x) StringBuffer += TEXT(x)
	#define SbTreePrintF(fmt, ...) StringBuffer.Appendf(TEXT(fmt), __VA_ARGS__)

	SbTreePrintF("Column Width:\t%u\n", (1 << ColumnShift));
	SbTreePrintF("Allocs:\t%u\n", TotalAllocs);
	SbTreePrintF("Columns:\t%u\n", TotalColumns);
	SbTreePrintF("Max Depth:\t%u\n", MaxDepth);
	SbTreePrintF("Cells:\t%u\n", TotalCells);
	SbTreePrintF("Max Alloc Count Per Cell:\t%u\n", LocalMaxAllocCountPerCell);
	SbTreePrintF("Not Empty Cells:\t%u\n", NotEmptyCellCount);
#if USE_OFFSETTED_CELLS
	SbTreePrintF("Not Empty Offsetted Cells:\t%u\n", NotEmptyOffsettedCellCount);
#endif // USE_OFFSETTED_CELLS

	SbTreePrint("\n");

	for (uint32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
#if USE_OFFSETTED_CELLS
		SbTreePrintF("\t%u\t%u*", Depth, Depth);
#else // USE_OFFSETTED_CELLS
		SbTreePrintF("\t%u", Depth);
#endif // USE_OFFSETTED_CELLS
	}
	SbTreePrint("\n");

	TArray<uint32> PrevCellIndex;
	PrevCellIndex.AddDefaulted(MaxDepth + 1);
	PrevCellIndex[0] = 1;

	for (uint32 Column = 0; Column < TotalColumns; ++Column)
	{
		SbTreePrintF("%u", Column);

		for (uint32 Depth = 0; Depth <= MaxDepth; ++Depth)
		{
			uint32 CellIndex = FSbTreeUtils::GetCellAtDepth(Column, Depth);
			if (CellIndex != PrevCellIndex[Depth])
			{
				const FSbTreeCell* CellPtr = (CellIndex < CellCount) ? Cells[CellIndex] : nullptr;
				uint32 AllocCount = (CellPtr != nullptr) ? CellPtr->GetAllocCount() : 0;

#if USE_OFFSETTED_CELLS
				const FSbTreeCell* OffsettedCellPtr = (CellIndex < OffsettedCellCount) ? OffsettedCells[CellIndex] : nullptr;
				uint32 OffsettedAllocCount = (OffsettedCellPtr != nullptr) ? OffsettedCellPtr->GetAllocCount() : 0;

				SbTreePrintF("\t%u\t%u", AllocCount, OffsettedAllocCount);
#else // USE_OFFSETTED_CELLS
				SbTreePrintF("\t%u", AllocCount);
#endif // USE_OFFSETTED_CELLS

				PrevCellIndex[Depth] = CellIndex;
			}
			else
			{
				SbTreePrint("\t\t");
			}
		}
		SbTreePrint("\n");
	}

	SbTreePrint("\n");

	FString FilePath = TEXT("D:/work/sbif.tab");
	FFileHelper::SaveStringToFile(StringBuffer, *FilePath);

	#undef SbTreePrint
	#undef SbTreePrintF
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSbTree::Validate() const
{
	const uint32 MaxDepth = FSbTreeUtils::GetMaxDepth(CurrentColumn);

	const uint32 NumCells = static_cast<uint32>(Cells.Num());
	for (uint32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
		uint32 EventIndex = 0;
		double Time = 0.0;
		const uint32 CellIncrement = 1 << (Depth + 1);
		uint32 CellIndex = FSbTreeUtils::GetCellAtDepth(0, Depth);
		while (CellIndex < NumCells)
		{
			const FSbTreeCell* CellPtr = Cells[CellIndex];
			if (CellPtr)
			{
				check(CellPtr->GetMaxEndEventIndex() > CellPtr->GetMinStartEventIndex());
				check(CellPtr->GetMinStartEventIndex() >= EventIndex);
				EventIndex = CellPtr->GetMaxEndEventIndex() + 1;

				check(CellPtr->GetMaxEndTime() >= CellPtr->GetMinStartTime());
				check(CellPtr->GetMinStartTime() >= Time);
				Time = CellPtr->GetMinStartTime();
			}
			CellIndex += CellIncrement;
		}
	}

#if USE_OFFSETTED_CELLS
	const uint32 NumOffsettedCells = static_cast<uint32>(OffsettedCells.Num());
	for (uint32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
		uint32 EventIndex = 0;
		double Time = 0.0;
		const uint32 CellIncrement = 1 << (Depth + 1);
		uint32 CellIndex = FSbTreeUtils::GetCellAtDepth(0, Depth);
		while (CellIndex < NumOffsettedCells)
		{
			const FSbTreeCell* CellPtr = OffsettedCells[CellIndex];
			if (CellPtr)
			{
				check(CellPtr->GetMaxEndEventIndex() > CellPtr->GetMinStartEventIndex());
				check(CellPtr->GetMinStartEventIndex() >= EventIndex);
				EventIndex = CellPtr->GetMaxEndEventIndex() + 1;

				check(CellPtr->GetMaxEndTime() >= CellPtr->GetMinStartTime());
				check(CellPtr->GetMinStartTime() >= Time);
				Time = CellPtr->GetMinStartTime();
			}
			CellIndex += CellIncrement;
		}
	}
	check(NumCells == NumOffsettedCells);
#endif // USE_OFFSETTED_CELLS
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef USE_OFFSETTED_CELLS

} // namespace TraceServices
