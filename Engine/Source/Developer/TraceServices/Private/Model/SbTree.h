// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "Common/PagedArray.h"
#include "TraceServices/Model/AllocationsProvider.h"

namespace TraceServices
{

struct FAllocationItem;
class ILinearAllocator;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSbTreeUtils
{
public:
	static uint32 GetMaxDepth(uint32 TotalColumns);
	static uint32 GetCellAtDepth(uint32 Column, uint32 Depth);
	static uint32 GetCommonDepth(uint32 ColumnA, uint32 ColumnB);
	static uint32 GetCellWidth(uint32 CellIndex);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSbTreeCell
{
public:
	FSbTreeCell(ILinearAllocator& InAllocator);
	~FSbTreeCell();

	uint32 GetAllocCount() const { return Allocs.Num(); }

	void AddAlloc(const FAllocationItem* Alloc); // takes ownership of memory allocated by Alloc pointer

	uint32 GetMinStartEventIndex() const { return MinStartEventIndex; }
	uint32 GetMaxEndEventIndex() const { return MaxEndEventIndex; }

	double GetMinStartTime() const { return MinStartTime; }
	double GetMaxEndTime() const { return MaxEndTime; }

	void Query(TArray<const FAllocationItem*>& OutAllocs, const IAllocationsProvider::FQueryParams& Params) const;

private:
	ILinearAllocator& Allocator;

	//TODO: TPagedArray<FAllocationItem> Allocs;
	TArray<const FAllocationItem*> Allocs;

	uint32 MinStartEventIndex;
	uint32 MaxEndEventIndex;
	double MinStartTime;
	double MaxEndTime;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSbTree
{
public:
	FSbTree(ILinearAllocator& InAllocator, uint32 InColumnShift);
	~FSbTree();

	void SetTimeForEvent(uint32 EventIndex, double Time);

	void AddAlloc(const FAllocationItem* Alloc); // takes ownership of memory allocated by Alloc pointer

	uint32 GetColumnWidth() const { return 1 << ColumnShift; }
	uint32 GetCurrentColumn() const { return CurrentColumn; }

	void Query(TArray<const FSbTreeCell*>& OutCells, const IAllocationsProvider::FQueryParams& Params) const;

	void IterateCells(TArray<const FSbTreeCell*>& OutCells, int32 Column) const;
	void IterateCells(TArray<const FSbTreeCell*>& OutCells, int32 StartColumn, int32 EndColumn) const;

	void DebugPrint() const;
	void Validate() const;

private:
	int32 GetColumnsAtTime(double Time, int32* OutStartColumn, int32* OutEndColumn) const;

private:
	ILinearAllocator& Allocator;

	// Normal cells.
	TArray<FSbTreeCell*> Cells;

	// Offsetted cells. Note: Depth 0 doesn't use offsetted cells. Those will always be empty.
	TArray<FSbTreeCell*> OffsettedCells;

	// Array with time values of the first event in each column.
	// Because multiple columns can have same time, CST[i] can also be assumed the time of the last event in previous column (i-1).
	// Time range for events in column i is inclusive interval [CST[i], CST[i+1]], with CST[N] = infinity.
	TArray<double> ColumnStartTimes;

	// It defines the width of a column (as number of events).
	// ColumnWidth = 1 << ColumnShift
	uint32 ColumnShift;

	// Index of the current (last) column.
	uint32 CurrentColumn;

	double LastAllocEndTime = 0.0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
