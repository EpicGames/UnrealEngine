// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Screenshot.h"
#include "Model/ScreenshotProviderPrivate.h"

#include "AnalysisServicePrivate.h"
#include "Common/FormatArgs.h"

namespace TraceServices
{

FScreenshotProvider::FScreenshotProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
}

TSharedPtr<FScreenshot> FScreenshotProvider::AddScreenshot(uint32 Id)
{
	TSharedPtr<FScreenshot> NewScreenshot = Screenshots.Add(Id, MakeShared<FScreenshot>());
	return NewScreenshot;
}

void FScreenshotProvider::AddScreenshotChunk(uint32 Id, uint32 ChunkNum, uint16 Size, const TArrayView<const uint8>& ChunkData)
{
	TSharedPtr<FScreenshot> *ScreenshotPtr = Screenshots.Find(Id);

	if (ScreenshotPtr == nullptr)
	{
		return;
	}

	TSharedPtr<FScreenshot> Screenshot = *ScreenshotPtr;

	constexpr uint32 ChunkSize = TNumericLimits<uint16>::Max();
	check(Screenshot->Data.Num() == ChunkNum * ChunkSize);

	Screenshot->Data.Append(ChunkData.GetData(), Size);
}

const TSharedPtr<const FScreenshot> FScreenshotProvider::GetScreenshot(uint32 Id) const
{
	const TSharedPtr<FScreenshot>* ScreenshotPtr = Screenshots.Find(Id);

	if (ScreenshotPtr == nullptr)
	{
		return nullptr;
	}

	return *ScreenshotPtr;
}

FName GetScreenshotProviderName()
{
	static const FName Name("ScreenshotProvider");
	return Name;
}

const IScreenshotProvider& ReadScreenshotProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<IScreenshotProvider>(GetScreenshotProviderName());
}

} // namespace TraceServices
