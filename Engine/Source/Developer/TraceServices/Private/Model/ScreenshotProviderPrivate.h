// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "TraceServices/Model/Screenshot.h"

namespace TraceServices
{

class FScreenshotProvider
	: public IScreenshotProvider
{
public:
	explicit FScreenshotProvider(IAnalysisSession& Session);
	virtual ~FScreenshotProvider() = default;

	TSharedPtr<FScreenshot> AddScreenshot(uint32 Id);
	void AddScreenshotChunk(uint32 Id, uint32 ChunkNum, uint16 Size, const TArrayView<const uint8>& ChunkData);
	virtual const TSharedPtr<const FScreenshot> GetScreenshot(uint32 Id) const override;

private:
	TMap<uint32, TSharedPtr<FScreenshot>> Screenshots;
	IAnalysisSession& Session;
};

} // namespace TraceServices
