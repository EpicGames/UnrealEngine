// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/StackSamples.h"
#include "Model/StackSamplesPrivate.h"

#include "Algo/Sort.h"
#include "HAL/PlatformMath.h"

// TraceServices
#include "AnalysisServicePrivate.h"
#include "Common/Utils.h"
#include "TraceServices/Model/Modules.h"

namespace TraceServices
{

thread_local FProviderLock::FThreadLocalState GStackSamplesProviderLockState;

FStackSamplesProvider::FStackSamplesProvider(IAnalysisSession& InSession)
	: Session(InSession)
	, Threads(Session.GetLinearAllocator(), 256)
	, StackFrames(Session.GetLinearAllocator(), 4 << 10)
{
	SetSamplingInterval(1000); // 1ms
}

FStackSamplesProvider::~FStackSamplesProvider()
{
}

void FStackSamplesProvider::SetSamplingInterval(uint32 Microseconds)
{
	SamplingIntervalUsec = Microseconds;
	MaxSampleDuration = (double)Microseconds / 1000000.0; // [seconds]
}

uint32 FStackSamplesProvider::GetStackFrameCount() const
{
	ReadAccessCheck();

	return static_cast<uint32>(StackFrames.Num());
}

void FStackSamplesProvider::EnumerateStackFrames(TFunctionRef<void(const FStackSampleFrame& Frame)> Callback) const
{
	ReadAccessCheck();

	auto It = StackFrames.GetIterator();
	const FStackSampleFrame* StackFrame = It.GetCurrentItem();
	while (StackFrame)
	{
		Callback(*StackFrame);
		StackFrame = It.NextItem();
	}
}

uint32 FStackSamplesProvider::GetThreadCount() const
{
	ReadAccessCheck();

	return static_cast<uint32>(Threads.Num());
}

void FStackSamplesProvider::EnumerateThreads(TFunctionRef<void(const FStackSampleThread& Thread)> Callback) const
{
	ReadAccessCheck();

	auto It = Threads.GetIterator();
	const FStackSamplesProviderThread* Thread = It.GetCurrentItem();
	while (Thread)
	{
		Callback(FStackSampleThread{ Thread->GetSystemThreadId(), Thread->GetName(), Thread->GetTimeline() });
		Thread = It.NextItem();
	}
}

uint32 FStackSamplesProvider::GetTimelineCount() const
{
	ReadAccessCheck();

	return static_cast<uint32>(Timelines.Num());
}

void FStackSamplesProvider::EnumerateTimelines(TFunctionRef<void(const FStackSampleTimeline& Timeline, const FStackSampleThread& Thread)> Callback) const
{
	ReadAccessCheck();

	for (const FStackSamplesProviderThread* Thread : Timelines)
	{
		check(Thread->GetTimeline() != nullptr);
		Callback(*Thread->GetTimeline(), FStackSampleThread{ Thread->GetSystemThreadId(), Thread->GetName(), Thread->GetTimeline() });
	}
}

const FStackSampleTimeline* FStackSamplesProvider::GetTimeline(uint32 SystemThreadId) const
{
	ReadAccessCheck();

	FStackSamplesProviderThread*const* FoundThread = ThreadsBySystemId.Find(SystemThreadId);
	if (!FoundThread)
	{
		return nullptr;
	}

	return (*FoundThread)->GetTimeline();
}

FStackSamplesProviderThread& FStackSamplesProvider::GetOrAddThread(uint32 SystemThreadId)
{
	FStackSamplesProviderThread** FoundThread = ThreadsBySystemId.Find(SystemThreadId);
	if (FoundThread)
	{
		return **FoundThread;
	}

	FStackSamplesProviderThread& NewThread = Threads.EmplaceBack(FStackSamplesProviderThread{ SystemThreadId });
	ThreadsBySystemId.Add(SystemThreadId, &NewThread);
	return NewThread;
}

void FStackSamplesProvider::AddThreadName(uint32 SystemThreadId, uint32 SystemProcessId, FStringView Name)
{
	EditAccessCheck();

	FStackSamplesProviderThread& Thread = GetOrAddThread(SystemThreadId);
	Thread.Name = Session.StoreString(Name);
}

FStackSampleFrame& FStackSamplesProvider::GetOrAddStackFrame(uint64 Address)
{
	FStackSampleFrame** FoundStackFrame = StackFramesByAddress.Find(Address);
	if (FoundStackFrame)
	{
		return **FoundStackFrame;
	}

	FStackSampleFrame& NewStackFrame = StackFrames.EmplaceBack(FStackSampleFrame{ Address });
	StackFramesByAddress.Add(Address, &NewStackFrame);

	if (ModuleProvider)
	{
		// This will return immediately. The result will be empty if the symbol
		// has not been encountered before, and resolution has been queued up.
		NewStackFrame.Symbol = ModuleProvider->GetSymbol(Address);
	}
	else
	{
		static const FResolvedSymbol NeverResolveSymbol(ESymbolQueryResult::NotLoaded, nullptr, nullptr, nullptr, 0, EResolvedSymbolFilterStatus::NotFiltered);
		NewStackFrame.Symbol = &NeverResolveSymbol;
	}

	CreateTimer(NewStackFrame);

	return NewStackFrame;
}

void FStackSamplesProvider::AddStackSample(uint32 SystemThreadId, double Time, uint32 Count, const uint64* Addresses)
{
	EditAccessCheck();

	if (!EditableTimingProfilerProvider)
	{
		EditableTimingProfilerProvider = EditTimingProfilerProvider(Session);
	}
	if (!ModuleProvider)
	{
		ModuleProvider = const_cast<IModuleProvider*>(ReadModuleProvider(Session));
	}

	FStackSamplesProviderThread& Thread = GetOrAddThread(SystemThreadId);
	if (!Thread.Timeline.IsValid())
	{
		Thread.Timeline = MakeShared<FStackSamplesTimeline>(Session.GetLinearAllocator());
		Timelines.Add(&Thread);
	}

	TArray<uint64> Callstack;
	Callstack.Reserve(Count);
	for (int32 StackIndex = int32(Count) - 1; StackIndex >= 0; --StackIndex)
	{
		Callstack.Add(Addresses[StackIndex]);
	}

	const int32 LastSampleStackSize = Thread.LastSampleStack.Num();
	const int32 NewSampleStackSize = Callstack.Num();

	// Ensure monotonic increase of time values.
	if (!ensure(Time >= Thread.LastSampleTime))
	{
		UE_LOGF(LogTraceServices, Warning, "[StackSampling] Time should increase monotonically (SysThreadId=0x%X, Time=%f < %f)!", SystemThreadId, Time, Thread.LastSampleTime);
		Time = Thread.LastSampleTime;
	}

	const double LastSampleEndTime = Thread.LastSampleTime + MaxSampleDuration/2;
	const double CurrentSampleStartTime = FMath::Max(0.0, Time - MaxSampleDuration/2);

	constexpr double Tolerance = 0.5; // 50%; i.e. 0.5ms for 1ms intervals

	int32 StartNewStackIndex = 0;
	if (CurrentSampleStartTime - MaxSampleDuration * Tolerance <= LastSampleEndTime)
	{
		for (int32 StackIndex = 0; StackIndex < LastSampleStackSize; ++StackIndex)
		{
			if ((StackIndex >= NewSampleStackSize) ||
				(Callstack[StackIndex] != Thread.LastSampleStack[StackIndex]))
			{
				break;
			}
			StartNewStackIndex = StackIndex + 1;
		}
	}

	const double MidTime = (Thread.LastSampleTime + Time) / 2.0;
	const double EndTime = FMath::Min(LastSampleEndTime, MidTime);
	const double StartTime = FMath::Max(CurrentSampleStartTime, MidTime);

	for (int32 StackIndex = StartNewStackIndex; StackIndex < LastSampleStackSize; ++StackIndex)
	{
		Thread.Timeline->AppendEndEvent(EndTime);
	}
	for (int32 StackIndex = StartNewStackIndex; StackIndex < NewSampleStackSize; ++StackIndex)
	{
		FStackSampleFrame& StackFrame = GetOrAddStackFrame(Callstack[StackIndex]);
		FTimingProfilerEvent Event;
		Event.TimerIndex = StackFrame.TimerId;
		Thread.Timeline->AppendBeginEvent(StartTime, Event);
	}

	// LastSampleStack = Callstack
	if (StartNewStackIndex < LastSampleStackSize)
	{
		Thread.LastSampleStack.RemoveAt(StartNewStackIndex, LastSampleStackSize - StartNewStackIndex, EAllowShrinking::No);
	}
	for (int32 StackIndex = StartNewStackIndex; StackIndex < NewSampleStackSize; ++StackIndex)
	{
		Thread.LastSampleStack.Push(Callstack[StackIndex]);
	}

	Thread.LastSampleTime = Time;

	// Update stats...
	MaxStackSize = FMath::Max(MaxStackSize, NewSampleStackSize);
	TotalNumFrames += static_cast<uint32>(NewSampleStackSize);
	TotalNumSamples++;
}

void FStackSamplesProvider::CreateTimer(FStackSampleFrame& StackFrame)
{
	EndEdit();

	{
		FAnalysisSessionEditScope _(Session);
		StackFrame.TimerId = EditableTimingProfilerProvider->AddTimer(ETimingProfilerTimerType::CpuSampling);
	}

	if (!UpdateTimerName(StackFrame, true))
	{
		PendingTimers.Add(&StackFrame);
	}

	BeginEdit();
}

bool FStackSamplesProvider::UpdateTimerName(FStackSampleFrame& StackFrame, bool bFirstTime)
{
	bool bIsPending = false;

	FString TimerName;
	if (!StackFrame.Symbol)
	{
		TimerName = FString::Printf(TEXT("0x%llX"), StackFrame.Address);
	}
	else
	{
		switch (StackFrame.Symbol->GetResult())
		{
			case TraceServices::ESymbolQueryResult::Pending:
				if (bFirstTime)
				{
					TimerName = FString::Printf(TEXT("0x%llX pending..."), StackFrame.Address);
				}
				bIsPending = true;
				break;

			case TraceServices::ESymbolQueryResult::OK:
				TimerName = FString::Printf(TEXT("%s (%s / 0x%llX)"),
					StackFrame.Symbol->Name ? StackFrame.Symbol->Name : TEXT("N/A"),
					StackFrame.Symbol->Module ? StackFrame.Symbol->Module : TEXT("N/A"),
					StackFrame.Address);
				{
					FAnalysisSessionEditScope _(Session);
					EditableTimingProfilerProvider->SetTimerLocation(StackFrame.TimerId, StackFrame.Symbol->File, StackFrame.Symbol->Line);
				}
				break;

			case TraceServices::ESymbolQueryResult::NotLoaded:
				TimerName = FString::Printf(TEXT("%s / 0x%llX NotLoaded"),
					StackFrame.Symbol->Module ? StackFrame.Symbol->Module : TEXT("N/A"),
					StackFrame.Address);
				break;

			case TraceServices::ESymbolQueryResult::Mismatch:
				TimerName = FString::Printf(TEXT("%s / 0x%llX Mismatch"),
					StackFrame.Symbol->Module ? StackFrame.Symbol->Module : TEXT("N/A"),
					StackFrame.Address);
				break;

			case TraceServices::ESymbolQueryResult::NotFound:
				TimerName = FString::Printf(TEXT("%s / 0x%llX NotFound"),
					StackFrame.Symbol->Module ? StackFrame.Symbol->Module : TEXT("N/A"),
					StackFrame.Address);
				break;

			default:
				TimerName = FString::Printf(TEXT("0x%llX Invalid !!!"), StackFrame.Address);
				break;
		}
	}

	if (!bIsPending || bFirstTime)
	{
		FAnalysisSessionEditScope _(Session);
		EditableTimingProfilerProvider->SetTimerName(StackFrame.TimerId, TimerName);
	}

	return !bIsPending;
}

bool FStackSamplesProvider::UpdatePendingTimers()
{
	for (int32 Index = 0; Index < PendingTimers.Num(); ++Index)
	{
		FStackSampleFrame& Frame = *PendingTimers[Index];

		if (UpdateTimerName(Frame, false))
		{
			PendingTimers[Index] = PendingTimers[PendingTimers.Num() - 1];
			PendingTimers.RemoveAt(PendingTimers.Num() - 1, EAllowShrinking::No);
			--Index;
		}
	}

	return PendingTimers.Num() > 0;
}

double FStackSamplesProvider::OnAnalysisEnd()
{
	EditAccessCheck();

	double MaxEndTime = 0.0;

	for (FStackSamplesProviderThread* Thread : Timelines)
	{
		check(Thread->Timeline);

		const double LastSampleEndTime = Thread->LastSampleTime + MaxSampleDuration / 2;
		MaxEndTime = FMath::Max(MaxEndTime, LastSampleEndTime);

		for (int32 Index = Thread->LastSampleStack.Num(); Index > 0; --Index)
		{
			Thread->Timeline->AppendEndEvent(LastSampleEndTime);
		}
		Thread->LastSampleStack.Reset();
	}

	UE_LOGF(LogTraceServices, Log, "[StackSampling] max stack size: %d frames", MaxStackSize);
	UE_LOGF(LogTraceServices, Log, "[StackSampling] Analysis completed (%llu timers, %d timelines, %llu threads, %llu stack samples, %llu stack frames).",
		StackFrames.Num(), Timelines.Num(), Threads.Num(), TotalNumSamples, TotalNumFrames);

	return MaxEndTime;
}

FName GetStackSamplesProviderName()
{
	static const FName Name("StackSamplesProvider");
	return Name;
}

const IStackSamplesProvider* ReadStackSamplesProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IStackSamplesProvider>(GetStackSamplesProviderName());
}

} // namespace TraceServices
