// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"

// TraceServices
#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "Model/MonotonicTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/TimingProfiler.h"

namespace TraceServices
{

class IModuleProvider;

extern thread_local FProviderLock::FThreadLocalState GStackSamplesProviderLockState;

typedef TMonotonicTimeline<FTimingProfilerEvent> FStackSamplesTimeline;

class FStackSamplesProviderThread
{
	friend class FStackSamplesProvider;

public:
	FStackSamplesProviderThread(uint32 InSystemThreadId)
		: SystemThreadId(InSystemThreadId)
	{
	}
	~FStackSamplesProviderThread() = default;

	uint32 GetSystemThreadId() const { return SystemThreadId; }
	const TCHAR* GetName() const { return Name; }
	FStackSamplesTimeline* GetTimeline() const { return Timeline.Get(); }

private:
	uint32 SystemThreadId = 0;
	const TCHAR* Name = nullptr;
	TSharedPtr<FStackSamplesTimeline> Timeline;
	TArray<uint64> LastSampleStack;
	double LastSampleTime = 0.0;
};

class FStackSamplesProvider
	: public IStackSamplesProvider
	, public IEditableProvider
{
public:
	explicit FStackSamplesProvider(IAnalysisSession& Session);
	virtual ~FStackSamplesProvider();

	//////////////////////////////////////////////////
	// Read operations

	virtual void BeginRead() const override { Lock.BeginRead(GStackSamplesProviderLockState); }
	virtual void EndRead() const override { Lock.EndRead(GStackSamplesProviderLockState); }
	virtual void ReadAccessCheck() const override { Lock.ReadAccessCheck(GStackSamplesProviderLockState); }

	virtual uint32 GetStackFrameCount() const override;
	virtual void EnumerateStackFrames(TFunctionRef<void(const FStackSampleFrame& Frame)> Callback) const override;

	virtual uint32 GetThreadCount() const override;
	virtual void EnumerateThreads(TFunctionRef<void(const FStackSampleThread& Thread)> Callback) const override;

	virtual uint32 GetTimelineCount() const override;
	virtual void EnumerateTimelines(TFunctionRef<void(const FStackSampleTimeline& Timeline, const FStackSampleThread& Thread)> Callback) const override;
	virtual const FStackSampleTimeline* GetTimeline(uint32 SystemThreadId) const override;

	//////////////////////////////////////////////////
	// Edit operations

	virtual void BeginEdit() const override { Lock.BeginWrite(GStackSamplesProviderLockState); }
	virtual void EndEdit() const override { Lock.EndWrite(GStackSamplesProviderLockState); }
	virtual void EditAccessCheck() const override { Lock.WriteAccessCheck(GStackSamplesProviderLockState); }

	void SetSamplingInterval(uint32 Microseconds);
	void AddThreadName(uint32 SystemThreadId, uint32 SystemProcessId, FStringView Name);
	void AddStackSample(uint32 SystemThreadId, double Time, uint32 Count, const uint64* Addresses);
	double OnAnalysisEnd();

	bool UpdatePendingTimers();

	//////////////////////////////////////////////////

private:
	FStackSamplesProviderThread& GetOrAddThread(uint32 SystemThreadId);
	FStackSampleFrame& GetOrAddStackFrame(uint64 Address);
	void CreateTimer(FStackSampleFrame& StackFrame);
	bool UpdateTimerName(FStackSampleFrame& StackFrame, bool bFirstTime);

private:
	mutable FProviderLock Lock;

	IAnalysisSession& Session;
	IEditableTimingProfilerProvider* EditableTimingProfilerProvider = nullptr;
	IModuleProvider* ModuleProvider = nullptr;

	TPagedArray<FStackSamplesProviderThread> Threads;
	TArray<FStackSamplesProviderThread*> Timelines; // the threads with a valid timeline
	TMap<uint32, FStackSamplesProviderThread*> ThreadsBySystemId; // SystemThreadId --> FThreadSamplingData*
	TPagedArray<FStackSampleFrame> StackFrames;
	TMap<uint64, FStackSampleFrame*> StackFramesByAddress; // Address --> FStackSampleFrame*
	TMap<uint32, FStackSampleFrame*> StackFramesByTimerId; // TimerId --> FStackSampleFrame*
	TArray<FStackSampleFrame*> PendingTimers;

	double MaxSampleDuration = 0.0;// [seconds]
	uint32 SamplingIntervalUsec = 0;// [microseconds]

	int32 MaxStackSize = 0;
	uint64 TotalNumFrames = 0;
	uint64 TotalNumSamples = 0;
};

} // namespace TraceServices
