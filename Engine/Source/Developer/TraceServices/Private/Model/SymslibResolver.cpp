// Copyright Epic Games, Inc. All Rights Reserved.

#include "SymslibResolver.h"

// TraceServices
#include "Common/SymbolHelper.h"

#if UE_SYMSLIB_AVAILABLE

#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Async/MappedFileHandle.h"
#include "Async/ParallelFor.h"
#include "Containers/StringView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTLS.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CString.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"

// TraceServices
#include "Common/SymbolStringAllocator.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Diagnostics.h"

#include <atomic>

#if PLATFORM_WINDOWS
	#define USE_DBG_HELP_UNDECORATOR 1
	#if USE_DBG_HELP_UNDECORATOR
		#include <Microsoft/AllowMicrosoftPlatformTypes.h>
		#include <DbgHelp.h>
		#include <Microsoft/HideMicrosoftPlatformTypes.h>
	#endif
#else
	#define USE_DBG_HELP_UNDECORATOR 0
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(LogSymslib);

namespace TraceServices
{

static const TCHAR* GUnknownModuleTextSymsLib = TEXT("Unknown");

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace
{
	struct FAutoMappedFile
	{
		FString FilePath;

		TUniquePtr<IMappedFileHandle> Handle;
		TUniquePtr<IMappedFileRegion> Region;

		bool Load(const TCHAR* FileName)
		{
			FOpenMappedResult Result = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(FileName);
			if (Result.HasValue())
			{
				Handle = Result.StealValue();
				Region.Reset(Handle->MapRegion(0, Handle->GetFileSize()));
			}
			else
			{
				Handle.Reset();
				Region.Reset();
			}
			return Region.IsValid();
		}

		SYMS_String8 GetData() const
		{
			return Region.IsValid()
				? syms_str8((SYMS_U8*)Region->GetMappedPtr(), Region->GetMappedSize())
				: syms_str8(nullptr, 0);
		}
	};

	static FString FindBinaryFileInPath(const FString& File, const FString& SearchPath, const FString& Platform)
	{
		IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();

		// On Linux and Mac find the non-stripped binary if available
		// todo: We cannot actually rely on the Platform string being set yet, we can enable that check when that's part of trace files metadata.
		//       for now we always override the binary if a .debug file exists
		//if (Platform.Equals(TEXT("Linux")) || Platform.Equals(TEXT("Mac")))
		{
			FString NonStrippedFile = FPaths::SetExtension(File, TEXT("debug"));
			FString Result = FPaths::Combine(SearchPath, NonStrippedFile);
			if (PlatformFile->FileExists(*Result))
			{
				return Result;
			}
		}

		FString FileName = FPaths::GetCleanFilename(File);

		// If the search path is an absolute path to a file use this. Filenames does
		// not need to match (e.g. eboot.bin <-> gamename.self);
		// but except the *.pdb files (in which case, we need to load the exe or dll binary instead).
		if (!SearchPath.EndsWith(TEXT(".pdb")))
		{
			if (PlatformFile->FileExists(*SearchPath))
			{
				return SearchPath;
			}
		}

		// Look for exact filename of the module in the provided search path.
		{
			FString Result = FPaths::Combine(SearchPath, FileName);
			if (PlatformFile->FileExists(*Result))
			{
				return Result;
			}
		}

		// Look for exact filename of the module in the provided search path
		// (case where the search path includes a filename).
		{
			FString Result = FPaths::Combine(FPaths::GetPath(SearchPath), FileName);
			if (PlatformFile->FileExists(*Result))
			{
				return Result;
			}
		}

		// In case File is relative
		if (FPaths::IsRelative(File))
		{
			FString Result = FPaths::Combine(SearchPath, File);
			if (PlatformFile->FileExists(*Result))
			{
				return Result;
			}
		}

		// Finally try the file path itself
		if (PlatformFile->FileExists(*File))
		{
			return File;
		}

		// File not found
		return FString();
	}

	static FString FindSymbolFileInPath(const FString& File, const FString& SearchPath)
	{
		IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
		FString SearchPathBase;

		// If search path is an absolute path to a file
		if (PlatformFile->FileExists(*SearchPath))
		{
			// For symbol files, the filename do need to match.
			if (FPaths::GetCleanFilename(SearchPath) == FPaths::GetCleanFilename(File))
			{
				return SearchPath;
			}

			// Strip the filename from the search path
			SearchPathBase = FPaths::GetPath(SearchPath);
		}
		else
		{
			SearchPathBase = SearchPath;
		}

		// Extract only filename part in case Path is absolute path
		FString FileName = FPaths::GetCleanFilename(File);

		// Look in the search path
		FString Result = FPaths::Combine(SearchPathBase, FileName);
		if (PlatformFile->FileExists(*Result))
		{
			return Result;
		}

		// In case File is relative
		Result = FPaths::Combine(SearchPathBase, File);
		if (PlatformFile->FileExists(*Result))
		{
			return Result;
		}

		// File not found
		return FString();
	}

	// Find a file assuming that search path is a root engine folder.
	static FString FindFileInEngineFolder(const FString& FilePath, const FString& SearchPath, const FString& Platform, const FString& Project)
	{
		IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();

		FString File = FPaths::GetCleanFilename(FilePath);

		// Look in engine directory
		FString Result = FPaths::Combine(SearchPath, TEXT("Engine"), TEXT("Binaries"), Platform, File);
		if (PlatformFile->FileExists(*Result))
		{
			return Result;
		}

		// Look in project directory
		Result = FPaths::Combine(SearchPath, Project, TEXT("Binaries"), Platform, File);
		if (PlatformFile->FileExists(*Result))
		{
			return Result;
		}

		return FString();
	}

	// use _NT_SYMBOL_PATH environment variable format
	// for more information see: https://docs.microsoft.com/en-us/windows-hardware/drivers/debugger/advanced-symsrv-use
	// to explicitly download symbol from MS Symbol Server for specific dll file, run:
	// "C:\Program Files (x86)\Windows Kits\10\Debuggers\x64\symchk.exe" /r C:\Windows\system32\kernel32.dll /s srv*C:\Symbols*https://msdl.microsoft.com/download/symbols
	static FString FindWindowsSymbolFile(const FString& GuidPath, const FString& SearchPath)
	{
		IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();

		if (SearchPath.StartsWith(TEXT("srv*")) || SearchPath.StartsWith(TEXT("cache*")))
		{
			TArray<FString> Srv;
			SearchPath.RightChop(4).ParseIntoArray(Srv, TEXT("*"));
			for (const FString& Path : Srv)
			{
				FString FilePath = FPaths::Combine(Path, GuidPath);
				if (PlatformFile->FileExists(*FilePath))
				{
					return FilePath;
				}
			}
		}
		else if (SearchPath.Contains(TEXT("*")))
		{
			FString FilePath = FPaths::Combine(SearchPath, GuidPath);
			if (PlatformFile->FileExists(*FilePath))
			{
				return FilePath;
			}
		}

		return FString();
	}

	static bool LoadBinary(const TCHAR* ModuleFullName, SYMS_Arena* Arena, SYMS_ParseBundle& Bundle, TArray<FAutoMappedFile>& Files, const FString& SearchPath, const FString& Platform, const FString& AppName)
	{
		FString FilePath = ModuleFullName;
		bool bFileFound = false;

		// Remap known renamed binaries
		if (FilePath.EndsWith(TEXT("eboot.bin")) && !Platform.IsEmpty() && !AppName.IsEmpty())
		{
			FilePath = FPaths::Combine(FPaths::GetPath(FilePath), FString::Printf(TEXT("%s.self"), *AppName));
		}

		// First lookup file in symbol path
		FString BinaryPath = FindBinaryFileInPath(FilePath, SearchPath, Platform);
		if (BinaryPath.IsEmpty() && !Platform.IsEmpty())
		{
			BinaryPath = FindFileInEngineFolder(FilePath, SearchPath, Platform, AppName);
		}
		bFileFound = !BinaryPath.IsEmpty();

		if (!bFileFound)
		{
			UE_LOGF(LogSymslib, Verbose, "Binary file '%ls' not found", *FilePath);
			return false;
		}

		FAutoMappedFile& File = Files.AddDefaulted_GetRef();
		if (!File.Load(*BinaryPath))
		{
			UE_LOGF(LogSymslib, Warning, "Failed to load binary file '%ls'", *BinaryPath);
			return false;
		}

		SYMS_FileAccel* Accel = syms_file_accel_from_data(Arena, File.GetData());
		SYMS_BinAccel* BinAccel = syms_bin_accel_from_file(Arena, File.GetData(), Accel);
		if (!syms_accel_is_good(BinAccel))
		{
			UE_LOGF(LogSymslib, Warning, "Cannot parse binary file '%ls'", *BinaryPath);
			return false;
		}

		// remember full path where binary file was found, so debug file can be looked up next to it
		File.FilePath = BinaryPath;

		Bundle.bin_data = File.GetData();
		Bundle.bin = BinAccel;

		return true;
	}

	static bool LoadDebug(SYMS_Arena* Arena, SYMS_ParseBundle& Bundle, TArray<FAutoMappedFile>& Files, const FString& SearchPath)
	{
		if (syms_bin_is_dbg(Bundle.bin))
		{
			// binary has debug info built-in (like dwarf file)
			Bundle.dbg = syms_dbg_accel_from_bin(Arena, Files[0].GetData(), Bundle.bin);
			Bundle.dbg_data = Bundle.bin_data;
			UE_LOGF(LogSymslib, Verbose, "Binary file '%ls' has debug info built-in", *Files[0].FilePath);
			return true;
		}

		// we're loading extra file (pdb for exe)
		SYMS_ExtFileList List = syms_ext_file_list_from_bin(Arena, Files[0].GetData(), Bundle.bin);
		if (!List.first)
		{
			UE_LOGF(LogSymslib, Warning, "Binary file '%ls' built without debug info", *Files[0].FilePath);
			return false;
		}
		SYMS_ExtFile ExtFile = List.first->ext_file;

		// debug file path from metadata in executable
		FString FilePath = ANSI_TO_TCHAR(reinterpret_cast<char*>(ExtFile.file_name.str));

		// try in the same path as the binary file
		FString BinaryPath = FPaths::GetPath(Files[0].FilePath);
		FString DebugPath = FindSymbolFileInPath(FilePath, BinaryPath);

		if (DebugPath.IsEmpty())
		{
			// try in provided search path
			DebugPath = FindSymbolFileInPath(FilePath, SearchPath);
		}

		if (DebugPath.IsEmpty())
		{
			// if executable is PE format, try Windows symbol path format with guid in path
			if (Bundle.bin->format == SYMS_FileFormat_PE && SearchPath.Contains(TEXT("*")))
			{
				SYMS_PeBinAccel* PeAccel = reinterpret_cast<SYMS_PeBinAccel*>(Bundle.bin);

				SYMS_PeGuid* Guid = &PeAccel->dbg_guid;
				SYMS_U32 Age = PeAccel->dbg_age;

				FString FileName = FPaths::GetCleanFilename(FilePath);
				FString GuidPath = FString::Printf(TEXT("%s/%08X%04X%04X%02X%02X%02X%02X%02X%02X%02X%02X%X/%s"),
					*FileName,
					Guid->data1, Guid->data2, Guid->data3,
					Guid->data4[0], Guid->data4[1], Guid->data4[2], Guid->data4[3], Guid->data4[4], Guid->data4[5], Guid->data4[6], Guid->data4[7],
					Age,
					*FileName);

				DebugPath = FindWindowsSymbolFile(GuidPath, SearchPath);
			}
		}

		if (DebugPath.IsEmpty())
		{
			UE_LOGF(LogSymslib, Verbose, "Debug symbols file '%ls' not found", *FilePath);
			return false;
		}

		FAutoMappedFile& File = Files.AddDefaulted_GetRef();
		if (!File.Load(*DebugPath))
		{
			UE_LOGF(LogSymslib, Warning, "Failed to load debug symbols file '%ls'", *DebugPath);
			return false;
		}

		SYMS_FileAccel* Accel = syms_file_accel_from_data(Arena, File.GetData());
		SYMS_DbgAccel* DbgAccel = syms_dbg_accel_from_file(Arena, File.GetData(), Accel);
		if (!syms_accel_is_good(DbgAccel))
		{
			// syms_dbg_accel_from_file only handles PDB. For ELF .debug files (separate DWARF),
			// we need to go through the bin path: parse as binary, then extract debug info from it.
			if (syms_file_is_bin(Accel))
			{
				SYMS_BinAccel* DbgBin = syms_bin_accel_from_file(Arena, File.GetData(), Accel);
				if (syms_accel_is_good(DbgBin) && syms_bin_is_dbg(DbgBin))
				{
					DbgAccel = syms_dbg_accel_from_bin(Arena, File.GetData(), DbgBin);
				}
			}

			if (!syms_accel_is_good(DbgAccel))
			{
				UE_LOGF(LogSymslib, Warning, "Cannot parse debug symbols file '%ls'", *DebugPath);
				return false;
			}
		}

		File.FilePath = DebugPath;

		Bundle.dbg = DbgAccel;
		Bundle.dbg_data = File.GetData();

		return true;
	}

	static bool MatchImageId(const TArray<uint8>& ImageId, SYMS_ParseBundle DataParsed)
	{
		if (DataParsed.dbg->format == SYMS_FileFormat_PDB)
		{
			// for Pdbs checksum is a 16 byte guid and 4 byte unsigned integer for age, but usually age is not used for matching debug file to exe
			static_assert(sizeof(FGuid) == 16, "Expected 16 byte FGuid");
			check(ImageId.Num() == 20);
			FGuid* ModuleGuid = (FGuid*)ImageId.GetData();

			SYMS_ExtMatchKey MatchKey = syms_ext_match_key_from_dbg(DataParsed.dbg_data, DataParsed.dbg);
			FGuid* PdbGuid = (FGuid*)MatchKey.v;

			if (*ModuleGuid != *PdbGuid)
			{
				// mismatch
				return false;
			}
		}
		else if (DataParsed.bin->format == SYMS_FileFormat_ELF)
		{
			// try different ways of getting build id from elf binary
			SYMS_String8 FoundId = { 0, 0 };

			SYMS_String8 Bin = DataParsed.bin_data;
			SYMS_ElfSectionArray Sections = DataParsed.bin->elf_accel.sections;
			for (SYMS_U64 SectionIndex = 0; SectionIndex < Sections.count; SectionIndex += 1)
			{
				SYMS_U64 SectionOffset = Sections.v[SectionIndex].file_range.min;
				SYMS_U64 SectionSize = Sections.v[SectionIndex].file_range.max - SectionOffset;

				if (syms_string_match(Sections.v[SectionIndex].name, syms_str8_lit(".note.gnu.build-id"), 0))
				{
					if (SectionSize > 12)
					{
						SYMS_U32 NameSize = *(SYMS_U32*)&Bin.str[SectionOffset + 0];
						SYMS_U32 DescSize = *(SYMS_U32*)&Bin.str[SectionOffset + 4];
						SYMS_U32 Type = *(SYMS_U32*)&Bin.str[SectionOffset + 8];

						const SYMS_U32 NT_GNU_BUILD_ID = 3;
						// name must be "GNU\0", contents must be at least 16 bytes, and type must be 3
						if (NameSize == 4 && DescSize >= 16 && Type == NT_GNU_BUILD_ID)
						{
							SYMS_U64 NameOffset = sizeof(NameSize) + sizeof(DescSize) + sizeof(Type);
							SYMS_String8 NameStr = syms_str8(&Bin.str[SectionOffset + NameOffset], 4);
							if (NameSize <= SectionSize && syms_string_match(NameStr, syms_str8((SYMS_U8*)"GNU", 4), 0))
							{
								SYMS_U64 DescOffset = NameOffset + SYMS_AlignPow2(NameSize, 4);
								if (DescOffset + 16 <= SectionSize)
								{
									FoundId = syms_str8(&Bin.str[SectionOffset + DescOffset], 16);
								}
							}
						}
					}
					break;
				}
				else if (syms_hash_djb2(Sections.v[SectionIndex].name) == 0xaab84f54dfa67dee)
				{
					if (SectionSize >= 16)
					{
						FoundId = syms_str8(&Bin.str[SectionOffset], 16);
					}
					break;
				}
			}

			if (FoundId.size == 16 && FoundId.size == ImageId.Num())
			{
				if (FMemory::Memcmp(ImageId.GetData(), FoundId.str, FoundId.size) != 0)
				{
					// mismatch
					return false;
				}
			}
		}

		// either ID's are matching, or ID is not found in which case return "success" case
		return true;
	}

} // namespace

////////////////////////////////////////////////////////////////////////////////////////////////////

FSymslibResolver::FSymslibResolver(IAnalysisSession& InSession, IResolvedSymbolFilter& InSymbolFilter)
	: Modules(InSession.GetLinearAllocator(), 128)
	, CancelTasks(false)
	, Session(InSession)
	, SymbolFilter(InSymbolFilter)
{
	// Setup search paths. Paths are searched in the following order:
	// 1. Any new path entered by the user this session
	// 2. Path of the executable (if available)
	// 3. Paths from UE_INSIGHTS_SYMBOL_PATH environment variable
	// 4. Paths from _NT_SYMBOL_PATH environment variable
	// 5. Paths from the user configuration file

	// Get the pre-configured symbol search paths (environment variables + user configuration file).
	FSymbolHelper::GetSymbolSearchPaths(LogSymslib, ConfigSymbolSearchPaths);

	UpdateSessionInfo();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSymslibResolver::~FSymslibResolver()
{
	CancelTasks = true;
	// Wait for any existing cleanup tasks to finish
	if (CleanupTask)
	{
		CleanupTask->Wait();
	}
	// Wait for module reload task to finish
	if (ModuleReloadTask)
	{
		ModuleReloadTask->Wait();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::QueueModuleLoad(const uint8* ImageId, uint32 ImageIdSize, FModule* Module)
{
	check(Module != nullptr);
	ensure(GetModuleEntry(Module) == nullptr);

	FWriteScopeLock _(ModulesLock);

	check(!CleanupTask); // No queued addresses should be queued after cleanup task is started

	// Add the new module entry.
	FModuleEntry* Entry = &Modules.PushBack();
	Entry->Module = Module;
	Entry->ImageId = TArrayView<const uint8>(ImageId, ImageIdSize);

	// Sort list according to base address.
	SortedModules.Add(Entry);
	Algo::Sort(SortedModules, [](const FModuleEntry* Lhs, const FModuleEntry* Rhs) { return Lhs->Module->Base < Rhs->Module->Base; });

	++ModulesDiscovered;

	// Reset stats for module.
	Module->Stats.Discovered.store(0u);
	Module->Stats.Resolved.store(0u);
	Module->Stats.Failed.store(0u);
	Module->Stats.Available.store(0u);

	// Set the Pending state before scheduling the background task (to allow calling code to wait, if needed).
	Module->Status.store(EModuleStatus::Pending);

	// Queue up module to have symbols loaded.
	// Run as background task as to not interfere with Slate.
	++TasksInFlight;
	FFunctionGraphTask::CreateAndDispatchWhenReady(
		[this, Entry]
		{
			LoadModuleTracked(Entry, FStringView());
			--TasksInFlight;
		}, TStatId{}, nullptr, ENamedThreads::AnyBackgroundThreadNormalTask);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::QueueModuleReload(FModule* Module, const TCHAR* Path, TFunction<void(SymbolArray&)> ResolveOnSuccess)
{
	check(Module != nullptr);

	// Find the entry
	FModuleEntry* Entry = GetModuleEntry(Module);
	if (!Entry)
	{
		return;
	}
	check(Entry->Module == Module);

	// No use in trying reload already loaded modules
	if (Module->Status == EModuleStatus::Loaded)
	{
		return;
	}

	// Reset stats for reloaded module.
	Module->Stats.Discovered.store(0u);
	Module->Stats.Resolved.store(0u);
	Module->Stats.Failed.store(0u);
	Module->Stats.Available.store(0u);

	// Set the Pending state before scheduling the background task (to allow calling code to wait, if needed).
	EModuleStatus PreviousStatus = Module->Status.exchange(EModuleStatus::Pending);
	if (PreviousStatus >= EModuleStatus::FailedStatusStart)
	{
		--ModulesFailed;
	}

	FString PathStr(Path);
	FPaths::NormalizeDirectoryName(PathStr);
	const TCHAR* OverrideSearchPath = Session.StoreString(PathStr);

	// Can only launch one task at a time
	if (ModuleReloadTask && !ModuleReloadTask->IsComplete())
	{
		ModuleReloadTask->Wait();
	}

	FFunctionGraphTask::CreateAndDispatchWhenReady([this, Entry, OverrideSearchPath, ResolveOnSuccess]
	{
		FScopeLock CleanupExclusive(&CleanupLock);

		LoadModuleTracked(Entry, OverrideSearchPath);

		if (Entry->Module->Status.load() == EModuleStatus::Loaded)
		{
			// If an additional search path was specified and successful, add to the permanent
			// list of search paths.
			{
				FWriteScopeLock _(CustomSymbolSearchPathsLock);
				// Get the base path if path points to a file
				IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
				const FString Directory = PlatformFile->DirectoryExists(OverrideSearchPath) ? OverrideSearchPath : FPaths::GetPath(OverrideSearchPath);
				if (!Directory.IsEmpty())
				{
					CustomSymbolSearchPaths.AddUnique(Directory);
				}
			}

			// Reset stats for reloaded module.
			// Note: If there are symbols pending to resolve (ex. busy spinning in ResolveSymbol()), the Stats.Discovered
			//       might be already incremented (see ResolveSymbolTracked()). So, resetting the stats here might result
			//       in slightly wrong stats where Discovered < Resolved + Failed.
			FModule::SymbolStats& ModuleStats = Entry->Module->Stats;
			ModuleStats.Discovered.store(0u);
			ModuleStats.Resolved.store(0u);
			ModuleStats.Failed.store(0u);

			// Ask the caller for a list of symbols that should be resolved, now that module is properly loaded and
			// resolve them immediately.
			{
				FSymbolStringAllocator StringAllocator(Session.GetLinearAllocator(), (64 << 10) / sizeof(TCHAR)); // using 64 KiB page size

				SymbolArray SymbolsToResolve;
				ResolveOnSuccess(SymbolsToResolve);
				for (TTuple<uint64, FResolvedSymbol*> Pair : SymbolsToResolve)
				{
					if (CancelTasks.load(std::memory_order_relaxed))
					{
						break;
					}
					ResolveSymbolTracked(Pair.Get<0>(), *Pair.Get<1>(), StringAllocator);
				}

				uint64 BytesAllocated = StringAllocator.GetAllocatedSize();
				uint64 BytesUsed = StringAllocator.GetUsedSize();
				uint64 BytesWasted = BytesAllocated - BytesUsed;
				UE_LOGF(LogSymslib, VeryVerbose, "String allocator: %.02f KiB used (+ %.02f KiB wasted) in %d blocks",
					(double)BytesUsed / 1024.0,
					(double)BytesWasted / 1024.0,
					StringAllocator.GetNumAllocatedBlocks());
				SymbolBytesAllocated.fetch_add(BytesAllocated);
				SymbolBytesWasted.fetch_add(BytesWasted);
			}

			// Finally if the cleanup task has already been dispatched, no other resolve request will come in and we
			// can safely release our resources. CleanupTask and a reload task cannot run at the same time due to CleanupLock.
			if (CleanupTask)
			{
				// Need to wait for tasks, since there could be queued symbol resolves for this module waiting
				// in the queue.
				WaitForTasks();

				Algo::ForEach(Entry->Instance.Arenas, syms_arena_release);
				Entry->Instance.Arenas.Empty();
			}
		}
	}, TStatId{}, nullptr, ENamedThreads::AnyBackgroundThreadNormalTask);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::QueueSymbolResolve(uint64 Address, FResolvedSymbol* Symbol)
{
	FScopeLock _(&SymbolsQueueLock);
	check(!CleanupTask); // No queued addresses should be queued after cleanup task is started
	MaybeDispatchQueuedAddresses();
	ResolveQueue.Add(FQueuedAddress{ Address, Symbol });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::OnAnalysisComplete()
{
	// Dispatch any remaining requests.
	{
		 FScopeLock _(&SymbolsQueueLock);
		 DispatchQueuedAddresses();
	}

	CleanupTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this]
	{
		FScopeLock CleanupExclusive(&CleanupLock);

		// Wait for outstanding batches to complete.
		WaitForTasks();

		// Release memory used by syms library
		{
			FReadScopeLock _(ModulesLock);
			for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
			{
				FModuleEntry& Entry = Modules[ModuleIndex];
				Algo::ForEach(Entry.Instance.Arenas, syms_arena_release);
				Entry.Instance.Arenas.Empty();
			}
		}

		UE_LOGF(LogSymslib, Display, "Allocated %.02f MiB of strings (%.02f MiB wasted).",
			(double)SymbolBytesAllocated / (1024.0 * 1024.0), (double)SymbolBytesWasted / (1024.0 * 1024.0));
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::GetStats(IModuleProvider::FStats* OutStats) const
{
	FReadScopeLock _(ModulesLock);
	FMemory::Memzero(*OutStats);
	for (uint32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
	{
		const FModule::SymbolStats& ModuleStats = Modules[ModuleIndex].Module->Stats;
		OutStats->SymbolsDiscovered += ModuleStats.Discovered.load();
		OutStats->SymbolsCached += ModuleStats.Cached.load();
		OutStats->SymbolsResolved += ModuleStats.Resolved.load();
		OutStats->SymbolsFailed += ModuleStats.Failed.load();
	}
	OutStats->SymbolsFailed += NoModuleSymbols;
	OutStats->ModulesDiscovered = ModulesDiscovered.load();
	OutStats->ModulesFailed = ModulesFailed.load();
	OutStats->ModulesLoaded = ModulesLoaded.load();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FSymslibResolver::HasFinishedResolving() const
{
	if (CleanupTask)
	{
		return CleanupTask->IsComplete();
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const
{
	{
		FReadScopeLock _(CustomSymbolSearchPathsLock);
		Algo::ForEach(CustomSymbolSearchPaths, Callback);
	}

	Algo::ForEach(ConfigSymbolSearchPaths, Callback);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Checks if there are no modules in flight and that the queue has reached
 * the threshold for dispatching. Note that is up to the caller to synchronize.
 */
void FSymslibResolver::MaybeDispatchQueuedAddresses()
{
	const bool bModulesInFlight = (ModulesDiscovered.load() - ModulesFailed.load() - ModulesLoaded.load()) > 0;
	if (!bModulesInFlight && (ResolveQueue.Num() >= QueuedAddressLength))
	{
		DispatchQueuedAddresses();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Dispatches the currently queued addresses to be resolved. Note that is up to the caller to synchronize.
 */
void FSymslibResolver::DispatchQueuedAddresses()
{
	if (!ResolveQueue.IsEmpty())
	{
		TArray<FQueuedAddress> WorkingSet(ResolveQueue);

		uint32 Stride = (WorkingSet.Num() - 1) / SymbolTasksInParallel + 1;
		constexpr uint32 MinStride = 4;
		Stride = FMath::Max(Stride, MinStride);
		const uint32 ActualSymbolTasksInParallel = (WorkingSet.Num() + Stride - 1) / Stride;
		TasksInFlight += ActualSymbolTasksInParallel;

		// Use background priority in order to not interfere with Slate.
		ParallelFor(ActualSymbolTasksInParallel, [this, &WorkingSet, Stride](uint32 Index) {
			const uint32 StartIndex = Index * Stride;
			const uint32 EndIndex = FMath::Min(StartIndex + Stride, (uint32)WorkingSet.Num());
			TArrayView<FQueuedAddress> QueuedWork(&WorkingSet[StartIndex], EndIndex - StartIndex);
			ResolveSymbols(QueuedWork);
			--TasksInFlight;
		}, EParallelForFlags::BackgroundPriority);

		ResolveQueue.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::ResolveSymbols(TArrayView<FQueuedAddress>& QueuedWork)
{
	// Create a local string allocator. We don't use the session string store due to contention when going wide. Since
	// the ModuleProvider already de-duplicates symbols we do not need this feature from the string store.
	FSymbolStringAllocator StringAllocator(Session.GetLinearAllocator(), (64 << 10) / sizeof(TCHAR)); // using 64 KiB page size

	for (const FQueuedAddress& ToResolve : QueuedWork)
	{
		if (CancelTasks.load(std::memory_order_relaxed))
		{
			break;
		}
		ResolveSymbolTracked(ToResolve.Address, *ToResolve.Target, StringAllocator);
	}

	uint64 BytesAllocated = StringAllocator.GetAllocatedSize();
	uint64 BytesUsed = StringAllocator.GetUsedSize();
	uint64 BytesWasted = BytesAllocated - BytesUsed;
	UE_LOGF(LogSymslib, VeryVerbose, "String allocator: %.02f KiB used (+ %.02f KiB wasted) in %d blocks",
		(double)BytesUsed / 1024.0,
		(double)BytesWasted / 1024.0,
		StringAllocator.GetNumAllocatedBlocks());
	SymbolBytesAllocated.fetch_add(BytesAllocated);
	SymbolBytesWasted.fetch_add(BytesWasted);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSymslibResolver::FModuleEntry* FSymslibResolver::GetModuleEntry(FModule* Module) const
{
	FReadScopeLock _(ModulesLock);
	for (FModuleEntry* Entry : SortedModules)
	{
		if (Entry->Module == Module)
		{
			return Entry;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSymslibResolver::FModuleEntry* FSymslibResolver::GetModuleForAddress(uint64 Address) const
{
	FReadScopeLock _(ModulesLock);
	const int32 EntryIdx = Algo::LowerBoundBy(SortedModules, Address, [](const FModuleEntry* Entry) { return Entry->Module->Base; }) - 1;
	if (EntryIdx < 0 || EntryIdx >= SortedModules.Num())
	{
		return nullptr;
	}
	return SortedModules[EntryIdx];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::UpdateResolvedSymbol(FResolvedSymbol& Symbol, ESymbolQueryResult Result, const TCHAR* Module, const TCHAR* Name, const TCHAR* File, uint16 Line)
{
	Symbol.Module = Module;
	Symbol.Name = Name;
	Symbol.File = File;
	Symbol.Line = Line;
	Symbol.Result.store(Result, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::LoadModuleTracked(FModuleEntry* Entry, FStringView OverrideSearchPath)
{
	FModule* Module = Entry->Module;

	// Reset stats
	Module->Stats.Discovered.store(0u);
	Module->Stats.Resolved.store(0u);
	Module->Stats.Failed.store(0u);
	Module->Stats.Available.store(0u);

	Module->Status.store(EModuleStatus::Pending);

	// Build search paths
	TArray<FString> SearchPaths;
	if (OverrideSearchPath.IsEmpty())
	{
		// 1. Any new path entered by the user this session
		{
			FReadScopeLock _(CustomSymbolSearchPathsLock);
			for (const FString& SearchPath : CustomSymbolSearchPaths)
			{
				SearchPaths.AddUnique(SearchPath);
			}
		}

		// 2. Path of the executable (if available)
		FString ModuleNamePath = FPaths::GetPath(Module->FullName);
		FPaths::NormalizeDirectoryName(ModuleNamePath);
		if (!ModuleNamePath.IsEmpty())
		{
			SearchPaths.AddUnique(ModuleNamePath);
		}

		// 3. Paths from UE_INSIGHTS_SYMBOL_PATH
		// 4. Paths from _NT_SYMBOL_PATH
		// 5. Paths from the user configuration file
		for (const FString& SearchPath : ConfigSymbolSearchPaths)
		{
			SearchPaths.AddUnique(SearchPath);
		}
	}
	else
	{
		SearchPaths.Add(FString(OverrideSearchPath));
	}

	// Exhaust all the search paths
	EModuleStatus Status = EModuleStatus::Failed;
	TStringBuilder<128> StatusMessage;
	for (const FString& SearchPath : SearchPaths)
	{
		if (CancelTasks.load())
		{
			break;
		}
		StatusMessage.Reset();
		Status = LoadModule(Entry, SearchPath, StatusMessage);
		UE_LOGF(LogSymslib, Display, "%ls", StatusMessage.ToString());
		if (Status == EModuleStatus::Loaded)
		{
			break;
		}
	}

	if (Status == EModuleStatus::Loaded)
	{
		++ModulesLoaded;
	}
	else
	{
		++ModulesFailed;
	}

	// Make the final status visible to the world
	Module->StatusMessage = Session.StoreString(StatusMessage.ToView());
	Module->Status.store(Status);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EModuleStatus FSymslibResolver::LoadModule(FModuleEntry* Entry, FStringView SearchPathView, FStringBuilderBase& OutStatusMessage) const
{
	FModule::SymbolStats& ModuleStats = Entry->Module->Stats;

	// how many symbols are loaded
	ModuleStats.Available = 0;

	const FString SearchPath(SearchPathView);

	// temporary memory used for loading
	SYMS_Group* Group = syms_group_alloc();

	// memory-mapped binary & debug files
	TArray<FAutoMappedFile> Files;

	// contents of binary & debug file
	SYMS_ParseBundle Bundle;

	if (Platform.IsEmpty())
	{
		UpdateSessionInfo();
	}

	if (!LoadBinary(Entry->Module->FullName, Group->arena, Bundle, Files, SearchPath, Platform, AppName))
	{
		syms_group_release(Group);
		OutStatusMessage.Appendf(TEXT("Failed to load binary for '%s' in '%s'."), Entry->Module->Name, *SearchPath);
		return EModuleStatus::Failed;
	}

	if (!LoadDebug(Group->arena, Bundle, Files, SearchPath))
	{
		syms_group_release(Group);
		OutStatusMessage.Appendf(TEXT("Failed to load debug symbols for '%s' in '%s'."), Entry->Module->Name, *SearchPath);
		return EModuleStatus::Failed;
	}

	// check debug data mismatch to captured module
	if (!Entry->ImageId.IsEmpty() && !MatchImageId(Entry->ImageId, Bundle))
	{
		syms_group_release(Group);
		OutStatusMessage << TEXT("Symbols for '") << Entry->Module->Name << TEXT("' found in '") << SearchPath << TEXT("' does not match trace binary.");
		return EModuleStatus::VersionMismatch;
	}

	FSymsInstance* Instance = &Entry->Instance;

	// initialize group
	syms_set_lane(0);
	syms_group_init(Group, &Bundle);

	// unit storage
	SYMS_U64 UnitCount = syms_group_unit_count(Group);
	Instance->Units.SetNum(static_cast<int32>(UnitCount));

	// per-thread arena storage (at least one)
	int32 WorkerThreadCount = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());
	Instance->Arenas.SetNum(WorkerThreadCount);

	// parse debug info in multiple threads
	{
		uint32 LaneSlot = FPlatformTLS::AllocTlsSlot();

		std::atomic<uint32> LaneCount = 0;
		syms_group_begin_multilane(Group, WorkerThreadCount);
		ParallelFor(static_cast<int32>(UnitCount), [Instance, Group, LaneSlot, &LaneCount, &ModuleStats](int32 Index)
		{
			SYMS_Arena* Arena;
			uint32 LaneValue = uint32(reinterpret_cast<intptr_t>(FPlatformTLS::GetTlsValue(LaneSlot)));
			if (LaneValue == 0)
			{
				// first time we are on this thread
				LaneValue = ++LaneCount;
				FPlatformTLS::SetTlsValue(LaneSlot, reinterpret_cast<void*>(intptr_t(LaneValue)));

				// syms lane index is 0-based
				uint32 LaneIndex = LaneValue - 1;
				syms_set_lane(LaneIndex);
				Arena = Instance->Arenas[LaneIndex] = syms_arena_alloc();
			}
			else
			{
				uint32 LaneIndex = LaneValue - 1;
				syms_set_lane(LaneIndex);
				Arena = Instance->Arenas[LaneIndex];
			}

			SYMS_ArenaTemp Scratch = syms_get_scratch(0, 0);

			SYMS_UnitID UnitID = static_cast<SYMS_UnitID>(Index) + 1; // syms unit id's are 1-based
			FSymsUnit* Unit = &Instance->Units[Index];

			SYMS_SpatialMap1D* ProcSpatialMap = syms_group_proc_map_from_uid(Group, UnitID);
			Unit->ProcMap = syms_spatial_map_1d_copy(Arena, ProcSpatialMap);

			SYMS_String8Array* FileTable = syms_group_file_table_from_uid_with_fallbacks(Group, UnitID);
			Unit->FileTable = syms_string_array_copy(Arena, 0, FileTable);

			SYMS_LineParseOut* LineParse = syms_group_line_parse_from_uid(Group, UnitID);
			Unit->LineTable = syms_line_table_with_indexes_from_parse(Arena, LineParse);

			SYMS_SpatialMap1D* LineSpatialMap = syms_group_line_sequence_map_from_uid(Group, UnitID);
			Unit->LineMap = syms_spatial_map_1d_copy(Arena, LineSpatialMap);

			SYMS_UnitAccel* UnitAccel = syms_group_unit_from_uid(Group, UnitID);

			SYMS_IDMap ProcIdMap = syms_id_map_alloc(Scratch.arena, 4093);

			SYMS_SymbolIDArray* ProcArray = syms_group_proc_sid_array_from_uid(Group, UnitID);
			SYMS_U64 ProcCount = ProcArray->count;

			FSymsSymbol* Symbols = syms_push_array(Arena, FSymsSymbol, ProcCount);
			for (SYMS_U64 ProcIndex = 0; ProcIndex < ProcCount; ProcIndex++)
			{
				SYMS_SymbolID SymbolID = ProcArray->ids[ProcIndex];

				SYMS_String8 Name = syms_group_symbol_name_from_sid(Arena, Group, UnitAccel, SymbolID);
				Symbols[ProcIndex].Name = reinterpret_cast<char*>(Name.str);

				syms_id_map_insert(Scratch.arena, &ProcIdMap, SymbolID, &Symbols[ProcIndex]);
			}

			SYMS_SpatialMap1D* ProcMap = &Unit->ProcMap;
			for (SYMS_SpatialMap1DRange* Range = ProcMap->ranges, *EndRange = ProcMap->ranges + ProcMap->count; Range < EndRange; Range++)
			{
				void* SymbolPtr = syms_id_map_ptr_from_u64(&ProcIdMap, Range->val);
				Range->val = SYMS_U64(reinterpret_cast<intptr_t>(SymbolPtr));
			}

			syms_release_scratch(Scratch);

			ModuleStats.Available += static_cast<uint32>(ProcCount);
		});
		syms_group_end_multilane(Group);

		FPlatformTLS::FreeTlsSlot(LaneSlot);
	}

	SYMS_Arena* Arena = Instance->Arenas[0];
	if (!Arena)
	{
		Arena = Instance->Arenas[0] = syms_arena_alloc();
	}

	// store stripped format symbols
	{
		SYMS_LinkNameRecArray StrippedInfo = syms_group_link_name_records(Group);

		FSymsSymbol* StrippedSymbols = syms_push_array(Arena, FSymsSymbol, StrippedInfo.count);
		for (SYMS_U64 Index = 0; Index < StrippedInfo.count; Index++)
		{
			SYMS_LinkNameRec* Info = &StrippedInfo.recs[Index];
			FSymsSymbol* StrippedSymbol = &StrippedSymbols[Index];

			SYMS_String8 Name = syms_push_string_copy(Arena, Info->name);
			StrippedSymbol->Name = reinterpret_cast<char*>(Name.str);
		}

		Instance->StrippedMap = syms_spatial_map_1d_copy(Arena, syms_group_link_name_spatial_map(Group));
		Instance->StrippedSymbols = StrippedSymbols;

		ModuleStats.Available += (uint32)StrippedInfo.count;
	}

	Instance->UnitMap = syms_spatial_map_1d_copy(Arena, syms_group_unit_map(Group));
	Instance->DefaultBase = syms_group_default_vbase(Group);

	syms_group_release(Group);
	Instance->Arenas.RemoveAll([](SYMS_Arena* arena) { return arena == nullptr; });

	// In order to avoid listing all files, use only the last. On dwarf/elf this will be the main binary, on Windows
	// this will be the pdb (the first is the exe/dll).
	OutStatusMessage.Append(*Files.Last().FilePath);
	return EModuleStatus::Loaded;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::ResolveSymbolTracked(uint64 Address, FResolvedSymbol& Target, FSymbolStringAllocator& StringAllocator)
{
	const ESymbolQueryResult PreviousResult = Target.Result.load();
	if (PreviousResult == ESymbolQueryResult::OK)
	{
		++AlreadyResolvedSymbols;
		return;
	}

	FModuleEntry* Entry = GetModuleForAddress(Address);
	if (!Entry)
	{
		UE_LOGF(LogSymslib, Warning, "No module mapped to address 0x%llX.", Address);
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotLoaded,
			GUnknownModuleTextSymsLib,
			GUnknownModuleTextSymsLib,
			GUnknownModuleTextSymsLib,
			0);
		SymbolFilter.Update(Target);
		++NoModuleSymbols;
		return;
	}

	FModule::SymbolStats& ModuleStats = Entry->Module->Stats;
	++ModuleStats.Discovered;
	if (!ResolveSymbol(Address, Target, StringAllocator, Entry))
	{
		++ModuleStats.Failed;
	}
	else
	{
		++ModuleStats.Resolved;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#if USE_DBG_HELP_UNDECORATOR
FORCEINLINE void UndecorateAndCopySymbolName(ANSICHAR* OutStr, const char* Name, uint32 MaxStringLength)
{
	// In case of stripped names we use the UnDecorateSymbolName function from the DbgHelp.
	if (Name[0] == '?')
	{
		// todo: To get full parity with non-stripped pdbs we shouldn't print the argument types. There is a flag
		//		 UNDNAME_NO_ARGUMENTS however whenever I tried that the symbol was not undecorated at all?
		constexpr DWORD Flags =
			UNDNAME_NO_MS_KEYWORDS |
			UNDNAME_NO_FUNCTION_RETURNS |
			UNDNAME_NO_ALLOCATION_MODEL |
			UNDNAME_NO_ALLOCATION_LANGUAGE |
			UNDNAME_NO_THISTYPE |
			UNDNAME_NO_ACCESS_SPECIFIERS |
			UNDNAME_NO_THROW_SIGNATURES |
			UNDNAME_NO_MEMBER_TYPE |
			UNDNAME_NO_RETURN_UDT_MODEL;

		// It is unclear from official documentation if UnDecorateSymbolName is thread safe or not, so
		// we take a local lock just in case.
		static FCriticalSection Cs;
		FScopeLock Lock(&Cs);
		DWORD Length = UnDecorateSymbolName(Name, OutStr, MaxStringLength, Flags);
	}
	else
	{
		FCStringAnsi::Strncpy(OutStr, Name, MaxStringLength);
	}
}
#endif // USE_DBG_HELP_UNDECORATOR

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FSymslibResolver::ResolveSymbol(uint64 Address, FResolvedSymbol& Target, FSymbolStringAllocator& StringAllocator, FModuleEntry* Entry) const
{
	EModuleStatus Status = Entry->Module->Status.load();
	while ((Status == EModuleStatus::Pending || Status == EModuleStatus::Discovered) && !CancelTasks.load())
	{
		FPlatformProcess::YieldThread();
		Status = Entry->Module->Status.load();
	}

	switch (Status)
	{
	case EModuleStatus::Failed:
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotLoaded,
			Entry->Module->Name,
			GUnknownModuleTextSymsLib,
			GUnknownModuleTextSymsLib,
			0);
		SymbolFilter.Update(Target);
		return false;

	case EModuleStatus::VersionMismatch:
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::Mismatch,
			Entry->Module->Name,
			GUnknownModuleTextSymsLib,
			GUnknownModuleTextSymsLib,
			0);
		SymbolFilter.Update(Target);
		return false;

	default:
		break;
	}

	// Find procedure and source file for address

	FSymsInstance* Instance = &Entry->Instance;
	SYMS_U64 VirtualOffset = Address + Instance->DefaultBase - Entry->Module->Base;

	FSymsSymbol* SymsSymbol = nullptr;

	constexpr uint32 MaxStringSize = 1024;
	const TCHAR* SourceFilePersistent = nullptr;
	uint32 SourceFileLine = 0;

	SYMS_UnitID UnitID = syms_spatial_map_1d_value_from_point(&Instance->UnitMap, VirtualOffset);
	if (UnitID)
	{
		FSymsUnit* Unit = &Instance->Units[static_cast<uint32>(UnitID) - 1];

		SYMS_U64 Value = syms_spatial_map_1d_value_from_point(&Unit->ProcMap, VirtualOffset);
		if (Value)
		{
			SymsSymbol = reinterpret_cast<FSymsSymbol*>(Value);

			SYMS_U64 SeqNumber = syms_spatial_map_1d_value_from_point(&Unit->LineMap, VirtualOffset);
			if (SeqNumber)
			{
				SYMS_Line Line = syms_line_from_sequence_voff(&Unit->LineTable, SeqNumber, VirtualOffset);
				if (Line.src_coord.file_id)
				{
					SYMS_String8 FileName = Unit->FileTable.strings[Line.src_coord.file_id - 1];

					ANSICHAR SourceFile[MaxStringSize];
					FCStringAnsi::Strncpy(SourceFile, reinterpret_cast<char*>(FileName.str), MaxStringSize);
					SourceFile[MaxStringSize - 1] = 0;
					SourceFilePersistent = StringAllocator.Store(ANSI_TO_TCHAR(SourceFile));

					SourceFileLine = Line.src_coord.line;
				}
			}
		}
	}

	if (SymsSymbol == nullptr)
	{
		// try lookup into stripped format symbols
		SYMS_U64 Value = syms_spatial_map_1d_value_from_point(&Instance->StrippedMap, VirtualOffset);
		if (Value)
		{
			SymsSymbol = &Instance->StrippedSymbols[Value - 1];

			// use module name as filename
			SourceFilePersistent = StringAllocator.Store(Entry->Module->Name);
		}
	}

	// this includes skipping symbols without name (empty string)
	if (!SymsSymbol || !SourceFilePersistent || (SymsSymbol && SymsSymbol->Name[0] == 0))
	{
		UpdateResolvedSymbol(Target,
			ESymbolQueryResult::NotFound,
			Entry->Module->Name,
			GUnknownModuleTextSymsLib,
			GUnknownModuleTextSymsLib,
			0);
		SymbolFilter.Update(Target);
		return false;
	}

	ANSICHAR SymbolName[MaxStringSize];
#if USE_DBG_HELP_UNDECORATOR
	UndecorateAndCopySymbolName(SymbolName, SymsSymbol->Name, MaxStringSize);
#else
	FCStringAnsi::Strncpy(SymbolName, SymsSymbol->Name, MaxStringSize);
#endif
	SymbolName[MaxStringSize - 1] = 0;
	const TCHAR* SymbolNamePersistent =  StringAllocator.Store(ANSI_TO_TCHAR(SymbolName));

	// Store the strings and update the target data
	UpdateResolvedSymbol(Target,
		ESymbolQueryResult::OK,
		Entry->Module->Name,
		SymbolNamePersistent,
		SourceFilePersistent,
		static_cast<uint16>(SourceFileLine));
	SymbolFilter.Update(Target);

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::WaitForTasks()
{
	uint32 OutstandingTasks = TasksInFlight.load(std::memory_order_acquire);
	do
	{
		OutstandingTasks = TasksInFlight.load(std::memory_order_acquire);
		FPlatformProcess::Sleep(0.0);
	}
	while (OutstandingTasks > 0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSymslibResolver::UpdateSessionInfo() const
{
	// Try to get session information.
	// Depending on how module tracing and session info tracing is implemented on different platforms this may not
	// be available at this point.
	FAnalysisSessionReadScope _(Session);
	const IDiagnosticsProvider* DiagnosticsProvider = ReadDiagnosticsProvider(Session);
	if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
	{
		const FSessionInfo Info = DiagnosticsProvider->GetSessionInfo();
		Platform = Info.Platform;
		AppName = Info.AppName;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices

#endif // UE_SYMSLIB_AVAILABLE
