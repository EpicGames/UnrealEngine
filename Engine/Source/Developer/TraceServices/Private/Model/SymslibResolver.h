// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

// We currently only have symslib available on selected platforms.
#if PLATFORM_DESKTOP
	#define UE_SYMSLIB_AVAILABLE 1
#else
	#define UE_SYMSLIB_AVAILABLE 0
#endif

#if UE_SYMSLIB_AVAILABLE

#include "Async/MappedFileHandle.h"
#include "Async/TaskGraphInterfaces.h"
#include "Common/PagedArray.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "symslib.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Modules.h"

#include <atomic>

DECLARE_LOG_CATEGORY_EXTERN(LogSymslib, Log, All);

namespace TraceServices
{

class FSymslibResolver
{
public:
	typedef TArray<TTuple<uint64, FResolvedSymbol*>> SymbolArray;

	FSymslibResolver(IAnalysisSession& InSession, IResolvedSymbolFilter& InSymbolFilter);
	~FSymslibResolver();

	void QueueModuleLoad(const uint8* ImageId, uint32 ImageIdSize, FModule* Module);
	void QueueModuleReload(FModule* Module, const TCHAR* Path, TFunction<void(SymbolArray&)> ResolveOnSuccess);
	void QueueSymbolResolve(uint64 Address, FResolvedSymbol* Symbol);
	void OnAnalysisComplete();
	void GetStats(IModuleProvider::FStats* OutStats) const;
	bool HasFinishedResolving() const;
	void EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const;

private:
	struct FSymsSymbol
	{
		const char* Name;
	};

	struct FSymsUnit
	{
		SYMS_SpatialMap1D ProcMap;
		SYMS_String8Array FileTable;
		SYMS_LineTable    LineTable;
		SYMS_SpatialMap1D LineMap;
	};

	struct FSymsInstance
	{
		TArray<SYMS_Arena*> Arenas;
		TArray<FSymsUnit>   Units;
		SYMS_SpatialMap1D   UnitMap;
		SYMS_SpatialMap1D   StrippedMap;
		FSymsSymbol*        StrippedSymbols;
		uint64              DefaultBase;
	};

	struct FModuleEntry
	{
		FModule* Module;
		TArray<uint8> ImageId;
		FSymsInstance Instance;
	};

	struct FQueuedAddress
	{
		uint64 Address;
		FResolvedSymbol* Target;
	};

	enum : uint32 {
		MaxNameLen = 512,
		QueuedAddressLength = 2048,
		SymbolTasksInParallel = 8
	};

	/**
	 * Checks if there are no modules in flight and that the queue has reached
	 * the threshold for dispatching. Note that is up to the caller to synchronize.
	 */
	void MaybeDispatchQueuedAddresses();
	/**
	 * Dispatches the currently queued addresses to be resolved. Note that is up to the caller to synchronize.
	 */
	void DispatchQueuedAddresses();
	void ResolveSymbols(TArrayView<FQueuedAddress>& QueuedWork);
	FModuleEntry* GetModuleEntry(FModule* Module) const;
	FModuleEntry* GetModuleForAddress(uint64 Address) const;
	static void UpdateResolvedSymbol(FResolvedSymbol& Symbol, ESymbolQueryResult Result, const TCHAR* Module, const TCHAR* Name, const TCHAR* File, uint16 Line);
	void LoadModuleTracked(FModuleEntry* Entry, FStringView OverrideSearchPath);
	EModuleStatus LoadModule(FModuleEntry* Entry, FStringView SearchPathView, FStringBuilderBase& OutStatusMessage) const;
	void ResolveSymbolTracked(uint64 Address, FResolvedSymbol& Target, class FSymbolStringAllocator& StringAllocator);
	bool ResolveSymbol(uint64 Address, FResolvedSymbol& Target, FSymbolStringAllocator& StringAllocator, FModuleEntry* Entry) const;
	void WaitForTasks();
	void UpdateSessionInfo() const;

	mutable FRWLock ModulesLock;
	TPagedArray<FModuleEntry> Modules;
	TArray<FModuleEntry*> SortedModules;
	FCriticalSection SymbolsQueueLock;
	TArray<FQueuedAddress, TInlineAllocator<QueuedAddressLength>> ResolveQueue;
	std::atomic<uint32> TasksInFlight;
	FCriticalSection CleanupLock;
	FGraphEventRef CleanupTask;
	FGraphEventRef ModuleReloadTask;
	std::atomic<bool> CancelTasks;

	mutable FRWLock CustomSymbolSearchPathsLock;
	TArray<FString> CustomSymbolSearchPaths; // search paths added by user from UI in the current session
	TArray<FString> ConfigSymbolSearchPaths; // search paths specified in environment variables and config files

	mutable FString Platform;
	mutable FString AppName;

	std::atomic<uint32> ModulesDiscovered;
	std::atomic<uint32> ModulesFailed;
	std::atomic<uint32> ModulesLoaded;
	std::atomic<uint32> AlreadyResolvedSymbols;
	std::atomic<uint32> NoModuleSymbols;
	std::atomic<uint64> SymbolBytesAllocated;
	std::atomic<uint64> SymbolBytesWasted;

	IAnalysisSession& Session;
	IResolvedSymbolFilter& SymbolFilter;
};

} // namespace TraceServices

#endif //UE_SYMSLIB_AVAILABLE
