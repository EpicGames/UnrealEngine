// Copyright Epic Games, Inc. All Rights Reserved.

#include "TableImportTask.h"

#include "Async/TaskGraphInterfaces.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/FileHelper.h"
#include "Tasks/Task.h"

#include <limits>

namespace TraceServices
{

#define LOCTEXT_NAMESPACE "TableImportTask"

FTableImportTask::FTableImportTask(const FString& InFilePath, FName InTableId, FTableImportService::TableImportCallback InCallback)
	: Callback(InCallback)
	, FilePath(InFilePath)
	, TableId(InTableId)
{
}

FTableImportTask::~FTableImportTask()
{
}

void FTableImportTask::operator()()
{
	ETableImportResult Result = ImportTable();

	TSharedPtr<FTableImportCallbackParams> Params = MakeShared<FTableImportCallbackParams>();
	Params->TableId = TableId;
	Params->Result = Result;
	Params->Table = Table;
	Params->Messages = Messages;

	FFunctionGraphTask::CreateAndDispatchWhenReady(
		[Callback = this->Callback, Params]()
		{
			Callback(Params);
		},
		TStatId(),
		nullptr,
		ENamedThreads::GameThread);
}

ETableImportResult FTableImportTask::ImportTable()
{
	Table = MakeShared<TImportTable<FImportTableRow>>();

	TArray<FString> Lines;
	bool Result = LoadFileToStringArray(FilePath, Lines);

	if (!Result)
	{
		AddError(FText::Format(LOCTEXT("FailedToReadMsg", "Import failed because file {0} could not be read:"), FText::FromString(FilePath)));
		return ETableImportResult::EFail;
	}

	if (Lines.Num() < 2)
	{
		AddError(FText::Format(LOCTEXT("NotEnoughtLinesMsg", "Import failed because the files did not contain a minimum of 2 lines."), FText::FromString(FilePath)));
		return ETableImportResult::EFail;
	}

	if (FilePath.EndsWith(TEXT(".csv")))
	{
		Separator = TEXT(",");
	}
	else if (FilePath.EndsWith(TEXT(".tsv")))
	{
		Separator = TEXT("\t");
	}

	if (!ParseHeader(Lines[0]))
	{
		return ETableImportResult::EFail;
	}

	if (!CreateLayout(Lines[1]))
	{
		return ETableImportResult::EFail;
	}

	if (!ParseData(Lines))
	{
		return ETableImportResult::EFail;
	}

	return ETableImportResult::ESuccess;
}

bool FTableImportTask::ParseHeader(const FString& HeaderLine)
{
	HeaderLine.ParseIntoArray(ColumnNames, *Separator, false);
	if (ColumnNames.Num() == 0)
	{
		AddError(FText::Format(LOCTEXT("NoColumnsMsg", "Import failed because the file did not contain any columns."), FText::FromString(FilePath)));
		return false;
	}

	return true;
}

bool FTableImportTask::CreateLayout(const FString& Line)
{
	TArray<FString> Values;
	SplitLineIntoValues(Line, Values);

	if (Values.Num() != ColumnNames.Num())
	{
		AddError(LOCTEXT("ValuesMismatchMsg1", "Import failed because the number of values on line 1 does not match the number of columns in the header line."));
		return false;
	}

	TTableLayout<FImportTableRow>& Layout = Table->EditLayout();
	for (int32 Index = 0; Index < Values.Num(); ++Index)
	{
		auto ProjectorFunc = [Index](const FImportTableRow& Row)
		{
			return Row.GetValue(Index);
		};
		
		if (ColumnNames[Index].IsEmpty())
		{
			ColumnNames[Index] = FString::Format(TEXT("Column {0}"), {Index});
		}

		const FString& Value = Values[Index];
		if (Value.IsEmpty())
		{
			// If the first line has an empty value assume it is an int, the most restrictive type and downgrade if we encounter other types.  
			Layout.AddColumn<uint32>(*ColumnNames[Index], ProjectorFunc, TableColumnDisplayHint_Summable);
		}
		else if (Value.Equals(TEXT("inf")) || Value.Equals(TEXT("-inf")) || Value.Equals(TEXT("infinity")) || Value.Equals(TEXT("-infinity")) || Value.Equals(TEXT("nan")))
		{
			Layout.AddColumn<double>(*ColumnNames[Index], ProjectorFunc, TableColumnDisplayHint_Summable);
		}
		else if (Value.IsNumeric())
		{
			if (Value.Contains(TEXT(".")))
			{
				Layout.AddColumn<double>(*ColumnNames[Index], ProjectorFunc, TableColumnDisplayHint_Summable);
			}
			else
			{
				Layout.AddColumn<uint32>(*ColumnNames[Index], ProjectorFunc, TableColumnDisplayHint_Summable);
			}
		}
		else
		{
			Layout.AddColumn<const TCHAR*>(*ColumnNames[Index], ProjectorFunc);
		}
	}

	return true;
}

bool FTableImportTask::ParseData(TArray<FString>& Lines)
{
	TTableLayout<FImportTableRow>& Layout = Table->EditLayout();
	bool Restart = false;

	TArray<bool> HasNonEmptyValues;
	HasNonEmptyValues.AddDefaulted(ColumnNames.Num());

	for (int32 LineIndex = 1; LineIndex < Lines.Num(); ++LineIndex)
	{
		TArray<FString> Values;
		SplitLineIntoValues(Lines[LineIndex], Values);

		if (Values.Num() != ColumnNames.Num())
		{
			AddError(FText::Format(LOCTEXT("ValuesMismatchMsg2", "Import failed because the number of values on line {0} does not match the number of columns in the header line."), LineIndex + 1));
			return false;
		}

		FImportTableRow& NewRow = Table->AddRow();
		NewRow.SetNumValues(Values.Num());
		for (int32 ValueIndex = 0; ValueIndex < Values.Num(); ++ValueIndex)
		{
			ETableColumnType ColumnType = Layout.GetColumnType(ValueIndex);
			const FString& Value = Values[ValueIndex];
			if (ColumnType == TableColumnType_CString)
			{
				HasNonEmptyValues[ValueIndex] = true;
				const TCHAR* StoredValue = Table->GetStringStore().Store(*Value);
				NewRow.SetValue(ValueIndex, StoredValue);
			}
			else if (ColumnType == TableColumnType_Double)
			{
				if (Value.IsEmpty())
				{
					NewRow.SetValue(ValueIndex, 0.0f);
					continue;
				}
				else if (Value.IsNumeric())
				{
					HasNonEmptyValues[ValueIndex] = true;
					NewRow.SetValue(ValueIndex, FCString::Atod(*Value));
					continue;
				}
				else if (Value.Equals(TEXT("inf")) || Value.Equals(TEXT("infinity")))
				{
					HasNonEmptyValues[ValueIndex] = true;
					NewRow.SetValue(ValueIndex, std::numeric_limits<double>::infinity());
					continue;
				}
				else if (Value.Equals(TEXT("-inf")) || Value.Equals(TEXT("-infinity")))
				{
					HasNonEmptyValues[ValueIndex] = true;
					NewRow.SetValue(ValueIndex, -std::numeric_limits<double>::infinity());
					continue;
				}
				else if (Value.Equals(TEXT("nan")))
				{
					HasNonEmptyValues[ValueIndex] = true;
					NewRow.SetValue(ValueIndex, std::numeric_limits<double>::quiet_NaN());
					continue;
				}

				Layout.SetColumnType(ValueIndex, TableColumnType_CString);
				Restart = true;
				break;
			}
			else if (ColumnType == TableColumnType_Int)
			{
				if (Value.IsEmpty())
				{
					NewRow.SetValue(ValueIndex, 0);
					continue;
				}
				else if (!Value.IsNumeric())
				{
					Layout.SetColumnType(ValueIndex, TableColumnType_CString);
					Restart = true;
					break;
				}
				else if (Value.Contains(TEXT(".")))
				{
					Layout.SetColumnType(ValueIndex, TableColumnType_Double);
					Restart = true;
					break;
				}

				HasNonEmptyValues[ValueIndex] = true;
				NewRow.SetValue(ValueIndex, FCString::Atoi64(*Value));
			}
		}

		if (Restart)
		{
			TSharedPtr<TImportTable<FImportTableRow >> NewTable = MakeShared<TImportTable<FImportTableRow>>();
			NewTable->EditLayout() = Layout;
			Table = NewTable;
			ParseData(Lines);

			return true;
		}
	}

	// If we have columns with only empty values, switch their type to string and reprocess the data.
	const TCHAR* EmptyValue = Table->GetStringStore().Store(TEXT(""));
	for (int32 Index = 0; Index < HasNonEmptyValues.Num(); ++Index)
	{
		if (HasNonEmptyValues[Index] == false)
		{
			Layout.SetColumnType(Index, TableColumnType_CString);
			Restart = true;
		}
	}

	if (Restart)
	{
		TSharedPtr<TImportTable<FImportTableRow >> NewTable = MakeShared<TImportTable<FImportTableRow>>();
		NewTable->EditLayout() = Layout;
		Table = NewTable;
		ParseData(Lines);
	}

	return true;
}

void FTableImportTask::SplitLineIntoValues(const FString& InLine, TArray<FString>& OutValues)
{
	//Parse the line into values. Separators inside quotes are ignored.
	int Start = 0;
	bool IsInQuotes = false;
	int Index;
	for (Index = 0; Index < InLine.Len(); ++Index)
	{
		if (InLine[Index] == TEXT('"') && (Index == 0 || InLine[Index - 1] != TEXT('\\')))
		{
			IsInQuotes = !IsInQuotes;
		}
		else if(!IsInQuotes && InLine[Index] == Separator[0])
		{
			FString Value = InLine.Mid(Start, Index - Start).TrimQuotes();
			OutValues.Add(std::move(Value));
			Start = Index + 1;
		}
	}

	FString Value = InLine.Mid(Start, Index - Start).TrimQuotes();
	OutValues.Add(std::move(Value));
}

bool FTableImportTask::LoadFileToStringArray(const FString &InFilePath, TArray<FString>& Lines)
{
	// We use this function instead of FFileHelper::LoadFileToArray because that one does not support very large files.

	auto Visitor = [&Lines](FStringView Line)
	{
		Lines.Add(FString(Line));
	};

	return FFileHelper::LoadFileToStringWithLineVisitor(*FilePath, Visitor);
}

void FTableImportTask::AddError(const FText& Msg)
{
	Messages.Add(FTokenizedMessage::Create(EMessageSeverity::Error, Msg));
}

void FTableImportService::ImportTable(const FString& InPath, FName TableId, FTableImportService::TableImportCallback InCallback)
{
	UE::Tasks::Launch(UE_SOURCE_LOCATION, FTableImportTask(InPath, TableId, InCallback));
}

#undef LOCTEXT_NAMESPACE

} // namespace TraceServices
