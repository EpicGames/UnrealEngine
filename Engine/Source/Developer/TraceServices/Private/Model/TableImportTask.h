// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceServices
#include "Common/StringStore.h"
#include "Model/Tables.h"
#include "TraceServices/Model/TableImport.h"

class FTokenizedMessage;

namespace TraceServices
{

struct FColumnValueContainer;

class FImportTableRow
{
public:
	FColumnValueContainer GetValue(uint64 ColumnIndex) const
	{
		return Values[static_cast<int32>(ColumnIndex)];
	}

	template<typename T>
	void SetValue(uint64 ColumnIndex, T Value)
	{
		Values[static_cast<int32>(ColumnIndex)] = Value;
	}

	void SetNumValues(uint64 NumColumns)
	{
		Values.AddUninitialized(static_cast<int32>(NumColumns));
	}

private:
	TArray<FColumnValueContainer> Values;
};

constexpr int GImportTableAllocatorSlabSize = 32 << 20;

template<typename RowType>
class TImportTable
	: public TTable<RowType, GImportTableAllocatorSlabSize>
{
public:
	TImportTable()
		: TTable<RowType, GImportTableAllocatorSlabSize>()
		, StringStore(TTable<RowType, GImportTableAllocatorSlabSize>::Allocator)
	{
	}
	virtual ~TImportTable() = default;

	FStringStore& GetStringStore() { return StringStore; }

private:
	FStringStore StringStore;
};

class FTableImportTask
{
public:
	FTableImportTask(const FString& InFilePath, FName InTableId, FTableImportService::TableImportCallback InCallback);
	~FTableImportTask();

	void operator()();

	bool ParseHeader(const FString& HeaderLine);
	bool CreateLayout(const FString& Line);
	bool ParseData(TArray<FString>& Lines);

private:
	ETableImportResult ImportTable();

	void SplitLineIntoValues(const FString& InLine, TArray<FString>& OutValues);
	bool LoadFileToStringArray(const FString& InFilePath, TArray<FString>& Lines);

	void AddError(const FText& Msg);

	FTableImportService::TableImportCallback Callback;
	TSharedPtr<TImportTable<FImportTableRow>> Table;

	TArray<TSharedRef<FTokenizedMessage>> Messages;
	TArray<FString> ColumnNames;
	FString FilePath;
	FName TableId;
	FString Separator;
};

} // namespace TraceServices
