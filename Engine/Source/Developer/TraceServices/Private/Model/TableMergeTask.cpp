// Copyright Epic Games, Inc. All Rights Reserved.

#include "TableMergeTask.h"

#include "Logging/TokenizedMessage.h"
#include "Tasks/Task.h"
#include "Async/TaskGraphInterfaces.h"

namespace TraceServices
{

#define LOCTEXT_NAMESPACE "TableMergeTask"
	
FTableMergeTask::FTableMergeTask(const TSharedPtr<IUntypedTable>& InTableA, const TSharedPtr<IUntypedTable>& InTableB, FTableMergeService::TableDiffCallback InCallback)
	: TableA(InTableA)
	, TableB(InTableB)
	, Callback(InCallback)
{
}

void FTableMergeTask::operator()()
{
	const ETableDiffResult Result = MergeTables();

	TSharedPtr<FTableDiffCallbackParams> Params = MakeShared<FTableDiffCallbackParams>();
	Params->Result = Result;
	Params->Table = TableC;
	Params->Messages = Messages;

	FFunctionGraphTask::CreateAndDispatchWhenReady(
		[Callback = this->Callback, Params]()
		{
			Callback(Params);
		},
		TStatId(),
		nullptr,
		ENamedThreads::GameThread);
}

TraceServices::ETableDiffResult FTableMergeTask::MergeTables()
{
	const ITableLayout& LayoutA = TableA->GetLayout();
	const ITableLayout& LayoutB = TableB->GetLayout();

	if (LayoutA.GetColumnCount() != LayoutB.GetColumnCount())
	{
		AddError(LOCTEXT("IncompatibleLayoutColumnCount", "Layout incompatible: column count is different."));
		return ETableDiffResult::EFail;
	}

	TableC = MakeShared<TImportTable<FImportTableRow>>();
		
	if (!BuildCLayout(LayoutA, LayoutB, TableC->EditLayout()))
	{
		return ETableDiffResult::EFail;
	}

	const TSharedPtr<IUntypedTableReader> TableReaderA = MakeShareable(TableA->CreateReader());
	const TSharedPtr<IUntypedTableReader> TableReaderB = MakeShareable(TableB->CreateReader());
	FillResultUsing<FOnlyA, Sign::Positive>(*TableReaderA);
	FillResultUsing<FOnlyB, Sign::Negative>(*TableReaderB);

	return ETableDiffResult::ESuccess;
}

bool FTableMergeTask::BuildCLayout(const ITableLayout& LayoutA, const ITableLayout& LayoutB, TTableLayout<FImportTableRow>& LayoutC)
{
	for (uint64 ColIndex = 0; ColIndex < LayoutA.GetColumnCount(); ++ColIndex)
	{
		const ETableColumnType ColumnType = LayoutA.GetColumnType(ColIndex);
		const TCHAR* ColumnName = LayoutA.GetColumnName(ColIndex);
		const uint32 Flags = LayoutA.GetColumnDisplayHintFlags(ColIndex);

		ensure(ColumnType == LayoutB.GetColumnType(ColIndex));
		ensure(FCString::Stricmp(ColumnName, LayoutB.GetColumnName(ColIndex)) == 0);

		auto ProjectorFuncCreator = [&LayoutC]
		{
			return [ColIndex = LayoutC.GetColumnCount()](const FImportTableRow& Row)
			{
				return Row.GetValue(ColIndex);
			};
		};

		FString DiffColumnName = FString::Format(TEXT("{0} (A-B)"), {ColumnName});
		FString AColumnName = FString::Format(TEXT("{0} (A)"), {ColumnName});
		FString BColumnName = FString::Format(TEXT("{0} (B)"), {ColumnName});

		switch (ColumnType)
		{
		case TableColumnType_CString:
			LayoutC.AddColumn<const TCHAR*>(ColumnName, ProjectorFuncCreator(), Flags);
			ColumnsMap.AddDefaulted_GetRef().Emplace<FBoth>(ColIndex);
			break;
		case TableColumnType_Bool:
			{
				if ((Flags & TableColumnDisplayHint_Summable) != 0)
				{
					// DIFF column
					LayoutC.AddColumn<bool>(*DiffColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FMerged>(ColIndex);
					// OriginalA
					LayoutC.AddColumn<bool>(*AColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyA>(ColIndex);
					// OriginalB
					LayoutC.AddColumn<bool>(*BColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyB>(ColIndex);
				}
				else
				{
					LayoutC.AddColumn<bool>(ColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FBoth>(ColIndex);
				}
			}
			break;
		case TableColumnType_Int:
			{
				if ((Flags & TableColumnDisplayHint_Summable) != 0)
				{
					// DIFF column
					LayoutC.AddColumn<int64>(*DiffColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FMerged>(ColIndex);
					// OriginalA
					LayoutC.AddColumn<uint64>(*AColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyA>(ColIndex);
					// OriginalB
					LayoutC.AddColumn<uint64>(*BColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyB>(ColIndex);
				}
				else
				{
					AddError(FText::Format(
						LOCTEXT("LayoutInclompatibleIntNotSummmable", "Column {0} has int values but is not summable."), 
						{FText::FromStringView(ColumnName)}));
					return false;
				}
			}
			break;
		case TableColumnType_Float:
			{
				if ((Flags & TableColumnDisplayHint_Summable) != 0)
				{
					// DIFF column
					LayoutC.AddColumn<float>(*DiffColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FMerged>(ColIndex);
					// OriginalA
					LayoutC.AddColumn<float>(*AColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyA>(ColIndex);
					// OriginalB
					LayoutC.AddColumn<float>(*BColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyB>(ColIndex);
				}
				else
				{
					AddError(FText::Format(
						LOCTEXT("LayoutInclompatibleFloatNotSummmable", "Column {0} has float values but is not summable."), 
						{FText::FromStringView(ColumnName)}));
					return false;
				}
			}
			break;
		case TableColumnType_Double:
			{
				if ((Flags & TableColumnDisplayHint_Summable) != 0)
				{
					// DIFF column
					LayoutC.AddColumn<double>(*DiffColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FMerged>(ColIndex);
					// OriginalA
					LayoutC.AddColumn<double>(*AColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyA>(ColIndex);
					// OriginalB
					LayoutC.AddColumn<double>(*BColumnName, ProjectorFuncCreator(), Flags);
					ColumnsMap.AddDefaulted_GetRef().Emplace<FOnlyB>(ColIndex);
				}
				else
				{
					AddError(FText::Format(
						LOCTEXT("LayoutInclompatibleDoubleNotSummmable", "Column {0} has double values but is not summable."), 
						{FText::FromStringView(ColumnName)}));
					return false;
				}
			}
			break;
		case TableColumnType_Invalid: 
			AddError(FText::Format(
				LOCTEXT("LayoutInclompatibleInvalidColumn", "Column {0} has invalid type."), 
				{FText::FromStringView(ColumnName)}));
			return false;
		default: ;
		}
	}

	return true;
}

void FTableMergeTask::AddError(const FText& Msg)
{
	Messages.Add(FTokenizedMessage::Create(EMessageSeverity::Error, Msg));
}

void FTableMergeService::MergeTables(const TSharedPtr<IUntypedTable>& TableA, const TSharedPtr<IUntypedTable>& TableB, TableDiffCallback InCallback)
{
	UE::Tasks::Launch(UE_SOURCE_LOCATION, FTableMergeTask(TableA, TableB, InCallback));
}

#undef LOCTEXT_NAMESPACE

} // namespace TraceServices

