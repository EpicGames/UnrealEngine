// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TableImportTask.h"
#include "Tables.h"
#include "TraceServices/Model/TableMerge.h"

#include "Misc/TVariant.h"

namespace TraceServices
{
class FImportTableRow;

class FTableMergeTask
{
public:
	FTableMergeTask(const TSharedPtr<IUntypedTable>& InTableA, const TSharedPtr<IUntypedTable>& InTableB, FTableMergeService::TableDiffCallback InCallback);

	void operator()();

private:
	ETableDiffResult MergeTables();

	struct FBoth { FBoth(uint64 Index) : ColIndex(Index) {} uint64 ColIndex; };
	struct FOnlyA { FOnlyA(uint64 Index) : ColIndex(Index) {} uint64 ColIndex; };
	struct FOnlyB { FOnlyB(uint64 Index) : ColIndex(Index) {} uint64 ColIndex; };
	struct FMerged { FMerged(uint64 Index) : ColIndex(Index) {} uint64 ColIndex; };
	enum class Sign { Positive, Negative };

	bool BuildCLayout(const ITableLayout& LayoutA, const ITableLayout& LayoutB, TTableLayout<FImportTableRow>& LayoutC);

	template<typename Variant, Sign Operation>
	void FillResultUsing(IUntypedTableReader& SourceTableReader) const;

	void AddError(const FText& Msg);

private:
	const TSharedPtr<IUntypedTable> TableA;
	const TSharedPtr<IUntypedTable> TableB;
	const FTableMergeService::TableDiffCallback Callback;

	TSharedPtr<TImportTable<FImportTableRow>> TableC;
	TArray<TSharedRef<FTokenizedMessage>> Messages;
	TArray<TVariant<FEmptyVariantState, FBoth, FOnlyA, FOnlyB, FMerged>> ColumnsMap;
};

template <typename Variant, FTableMergeTask::Sign Operation>
void FTableMergeTask::FillResultUsing(IUntypedTableReader& SourceTableReader) const
{
	const uint64 ColumnsMapCount = static_cast<uint64>(ColumnsMap.Num());

	for (SourceTableReader.SetRowIndex(0); SourceTableReader.IsValid(); SourceTableReader.NextRow())
	{
		FImportTableRow& Row = TableC->AddRow();
		Row.SetNumValues(ColumnsMapCount);

		for (uint64 DestIndex = 0; DestIndex < ColumnsMapCount; ++DestIndex)
		{
			const ETableColumnType ColumnType = TableC->GetLayout().GetColumnType(DestIndex);

			uint64 SourceIndex;
			int64 Multiplier = 1;
			if (const auto* Column = ColumnsMap[static_cast<int32>(DestIndex)].TryGet<FBoth>())
			{
				SourceIndex = Column->ColIndex;
			}
			else if (const auto* ColumnA = ColumnsMap[static_cast<int32>(DestIndex)].TryGet<Variant>())
			{
				SourceIndex = ColumnA->ColIndex;
			}
			else if (const auto* ColumnM = ColumnsMap[static_cast<int32>(DestIndex)].TryGet<FMerged>())
			{
				SourceIndex = ColumnM->ColIndex;
				Multiplier = Operation == Sign::Negative ? -1 : 1;
			}
			else
			{
				switch (ColumnType)
				{
				case TableColumnType_Bool:
					Row.SetValue(DestIndex, false);
					break;
				case TableColumnType_Int:
					Row.SetValue(DestIndex, 0);
					break;
				case TableColumnType_Float:
					Row.SetValue(DestIndex, 0.0f);
					break;
				case TableColumnType_Double:
					Row.SetValue(DestIndex, 0.0);
					break;
				case TableColumnType_CString:
					Row.SetValue(DestIndex, TEXT("<>"));
					break;
				case TableColumnType_Invalid: break;
				default: break;
				}
				continue;
			}

			switch (ColumnType)
			{
			case TableColumnType_Bool:
				Row.SetValue(DestIndex, SourceTableReader.GetValueBool(SourceIndex));
				break;
			case TableColumnType_Int:
				Row.SetValue(DestIndex, SourceTableReader.GetValueInt(SourceIndex) * Multiplier);
				break;
			case TableColumnType_Float:
				Row.SetValue(DestIndex, SourceTableReader.GetValueFloat(SourceIndex) * (float)Multiplier);
				break;
			case TableColumnType_Double:
				Row.SetValue(DestIndex, SourceTableReader.GetValueDouble(SourceIndex) * (double)Multiplier);
				break;
			case TableColumnType_CString:
				Row.SetValue(DestIndex, TableC->GetStringStore().Store(SourceTableReader.GetValueCString(SourceIndex)));
				break;
			case TableColumnType_Invalid: break;
			default: break;
			}
		}
	}
	SourceTableReader.SetRowIndex(0);
}
} // namespace TraceServices
