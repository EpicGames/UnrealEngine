// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "TraceServices/Containers/Tables.h"
#include "Templates/Function.h"
#include "Common/PagedArray.h"
#include "TraceServices/Containers/SlabAllocator.h"

namespace TraceServices
{

template<typename Type>
static constexpr ETableColumnType GetColumnTypeFromNativeType();

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<bool>()
{
	return TableColumnType_Bool;
}

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<int8>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<uint8>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<int16>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<uint16>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<int32>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<uint32>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<int64>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<uint64>()
{
	return TableColumnType_Int;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<float>()
{
	return TableColumnType_Float;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<double>()
{
	return TableColumnType_Double;
};

template<>
constexpr ETableColumnType GetColumnTypeFromNativeType<const TCHAR*>()
{
	return TableColumnType_CString;
}

struct FColumnValueContainer
{
	FColumnValueContainer(bool Value)
	{
		BoolValue = Value;
	}

	FColumnValueContainer(int8 Value)
	{
		IntValue = Value;
	}

	FColumnValueContainer(uint8 Value)
	{
		IntValue = Value;
	}

	FColumnValueContainer(int32 Value)
	{
		IntValue = Value;
	}

	FColumnValueContainer(uint32 Value)
	{
		IntValue = Value;
	}

	FColumnValueContainer(int64 Value)
	{
		IntValue = Value;
	}

	FColumnValueContainer(uint64 Value)
	{
		IntValue = int64(Value);
	}

	FColumnValueContainer(float Value)
	{
		FloatValue = Value;
	}

	FColumnValueContainer(double Value)
	{
		DoubleValue = Value;
	}

	FColumnValueContainer(const TCHAR* Value)
	{
		StringValue = Value;
	}

	union
	{
		bool BoolValue;
		int64 IntValue;
		float FloatValue;
		double DoubleValue;
		const TCHAR* StringValue;
	};
};

template<typename RowType>
class TTableLayout
	: public ITableLayout
{
public:
	template<typename ColumnNativeType>
	TTableLayout<RowType>& AddColumn(ColumnNativeType RowType::* MemberVariableColumn, const TCHAR* ColumnName, uint32 DisplayHintFlags = 0)
	{
		Columns.Add({
			ColumnName,
			GetColumnTypeFromNativeType<ColumnNativeType>(),
			DisplayHintFlags,
			[MemberVariableColumn](const RowType& Row) -> FColumnValueContainer
			{
				return FColumnValueContainer(Row.*MemberVariableColumn);
			}
			});
		return *this;
	}

	template<typename ColumnNativeType>
	TTableLayout<RowType>& AddColumn(ColumnNativeType(RowType::* MemberFunctionColumn)() const, const TCHAR* ColumnName, uint32 DisplayHintFlags = 0)
	{
		Columns.Add({
			ColumnName,
			GetColumnTypeFromNativeType<ColumnNativeType>(),
			DisplayHintFlags,
			[MemberFunctionColumn](const RowType& Row) -> FColumnValueContainer
			{
				return FColumnValueContainer((Row.*MemberFunctionColumn)());
			}
			});
		return *this;
	}

	template<typename ColumnNativeType>
	TTableLayout<RowType>& AddColumn(ColumnNativeType(*FunctionColumn)(const RowType&), const TCHAR* ColumnName, uint32 DisplayHintFlags = 0)
	{
		Columns.Add({
			ColumnName,
			GetColumnTypeFromNativeType<ColumnNativeType>(),
			DisplayHintFlags,
			[FunctionColumn](const RowType& Row) -> FColumnValueContainer
			{
				return FColumnValueContainer(FunctionColumn(Row));
			}
			});
		return *this;
	}

	template<typename ColumnNativeType>
	TTableLayout<RowType>& AddColumn(const TCHAR* ColumnName, TFunction<FColumnValueContainer(const RowType&)> Projector, uint32 DisplayHintFlags = 0)
	{
		Columns.Add({
			ColumnName,
			GetColumnTypeFromNativeType<ColumnNativeType>(),
			DisplayHintFlags,
			Projector });
		return *this;
	}

	uint64 GetColumnCount() const override
	{
		return static_cast<uint64>(Columns.Num());
	}

	const TCHAR* GetColumnName(uint64 ColumnIndex) const override
	{
		return *Columns[static_cast<int32>(ColumnIndex)].Name;
	}

	ETableColumnType GetColumnType(uint64 ColumnIndex) const override
	{
		return Columns[static_cast<int32>(ColumnIndex)].Type;
	}

	void SetColumnType(uint64 ColumnIndex, ETableColumnType ColumnType)
	{
		Columns[static_cast<int32>(ColumnIndex)].Type = ColumnType;
	}

	uint32 GetColumnDisplayHintFlags(uint64 ColumnIndex) const override
	{
		return Columns[static_cast<int32>(ColumnIndex)].DisplayHintFlags;
	}

	FColumnValueContainer GetColumnValue(const RowType& Row, uint64 ColumnIndex) const
	{
		return Columns[static_cast<int32>(ColumnIndex)].Projector(Row);
	}

private:
	struct FColumnDeclaration
	{
		FString Name;
		ETableColumnType Type;
		uint32 DisplayHintFlags;
		TFunction<FColumnValueContainer(const RowType&)> Projector;
	};

	TArray<FColumnDeclaration> Columns;
};

template<typename RowType>
class TTableReader
	: public ITableReader<RowType>
{
public:
	TTableReader(const TTableLayout<RowType>& InLayout, const TPagedArray<RowType>& InRows)
		: Layout(InLayout)
		, Iterator(InRows.GetIteratorFromItem(0))
	{
		CurrentRow = Iterator.GetCurrentItem();
	}

	virtual bool IsValid() const override
	{
		return CurrentRow != nullptr;
	}

	virtual void NextRow() override
	{
		CurrentRow = Iterator.NextItem();
	}

	virtual void SetRowIndex(uint64 RowIndex) override
	{
		CurrentRow = Iterator.SetPosition(RowIndex);
	}

	virtual const RowType* GetCurrentRow() const override
	{
		return CurrentRow;
	}

	virtual bool GetValueBool(uint64 ColumnIndex) const override
	{
		if (!CurrentRow)
		{
			return false;
		}
		ETableColumnType ColumnType = Layout.GetColumnType(ColumnIndex);
		switch (ColumnType)
		{
		case TableColumnType_Bool:
			return Layout.GetColumnValue(*CurrentRow, ColumnIndex).BoolValue;
		case TableColumnType_Int:
			return static_cast<bool>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).IntValue);
		case TableColumnType_Float:
			return static_cast<bool>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).FloatValue);
		case TableColumnType_Double:
			return static_cast<bool>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).DoubleValue);
		}
		return false;
	}

	virtual int64 GetValueInt(uint64 ColumnIndex) const override
	{
		if (!CurrentRow)
		{
			return 0;
		}
		ETableColumnType ColumnType = Layout.GetColumnType(ColumnIndex);
		switch (ColumnType)
		{
		case TableColumnType_Bool:
			return static_cast<int64>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).BoolValue);
		case TableColumnType_Int:
			return Layout.GetColumnValue(*CurrentRow, ColumnIndex).IntValue;
		case TableColumnType_Float:
			return static_cast<int64>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).FloatValue);
		case TableColumnType_Double:
			return static_cast<int64>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).DoubleValue);
		}
		return 0;
	}

	virtual float GetValueFloat(uint64 ColumnIndex) const override
	{
		if (!CurrentRow)
		{
			return 0.0;
		}
		ETableColumnType ColumnType = Layout.GetColumnType(ColumnIndex);
		switch (ColumnType)
		{
		case TableColumnType_Bool:
			return static_cast<float>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).BoolValue);
		case TableColumnType_Int:
			return static_cast<float>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).IntValue);
		case TableColumnType_Float:
			return Layout.GetColumnValue(*CurrentRow, ColumnIndex).FloatValue;
		case TableColumnType_Double:
			return static_cast<float>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).DoubleValue);
		}
		return 0.0;
	}

	virtual double GetValueDouble(uint64 ColumnIndex) const override
	{
		if (!CurrentRow)
		{
			return 0.0;
		}
		ETableColumnType ColumnType = Layout.GetColumnType(ColumnIndex);
		switch (ColumnType)
		{
		case TableColumnType_Bool:
			return static_cast<double>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).BoolValue);
		case TableColumnType_Int:
			return static_cast<double>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).IntValue);
		case TableColumnType_Float:
			return static_cast<double>(Layout.GetColumnValue(*CurrentRow, ColumnIndex).FloatValue);
		case TableColumnType_Double:
			return Layout.GetColumnValue(*CurrentRow, ColumnIndex).DoubleValue;
		}
		return 0.0;
	}

	virtual const TCHAR* GetValueCString(uint64 ColumnIndex) const override
	{
		if (!CurrentRow)
		{
			return TEXT("");
		}
		ETableColumnType ColumnType = Layout.GetColumnType(ColumnIndex);
		if (ColumnType == TableColumnType_CString)
		{
			const TCHAR* CStringValue = Layout.GetColumnValue(*CurrentRow, ColumnIndex).StringValue;
			return CStringValue ? CStringValue : TEXT("");
		}
		return TEXT("");
	}

private:
	const TTableLayout<RowType>& Layout;
	typename TPagedArray<RowType>::TIterator Iterator;
	const RowType* CurrentRow;
};

template<typename RowType>
class TTableBase
	: public ITable<RowType>
{
public:
	TTableBase() = default;

	TTableBase(TTableLayout<RowType> InLayout)
		: Layout(InLayout)
	{

	}

	virtual ~TTableBase() = default;

	const ITableLayout& GetLayout() const override
	{
		return Layout;
	}

	uint64 GetRowCount() const override
	{
		return GetRows().Num();
	}

	ITableReader<RowType>* CreateReader() const override
	{
		return new TTableReader<RowType>(Layout, GetRows());
	}

	TTableLayout<RowType>& EditLayout()
	{
		return Layout;
	}

private:
	virtual const TPagedArray<RowType>& GetRows() const = 0;

	TTableLayout<RowType> Layout;
};

template<typename RowType>
class TTableView
	: public TTableBase<RowType>
{
public:
	TTableView(const TPagedArray<RowType>& InRows)
		: Rows(InRows)
	{
	}

private:
	virtual const TPagedArray<RowType>& GetRows() const override
	{
		return Rows;
	}

	const TPagedArray<RowType>& Rows;
};

template<typename RowType, int AllocatorSlabSize = 2 << 20>
class TTable
	: public TTableBase<RowType>
{
public:
	TTable()
		: Allocator(AllocatorSlabSize)
		, Rows(Allocator, 1024)
	{

	}

	TTable(TTableLayout<RowType> Layout)
		: TTableBase<RowType>(Layout)
		, Allocator(AllocatorSlabSize)
		, Rows(Allocator, 1024)
	{

	}

	RowType& AddRow()
	{
		return Rows.PushBack();
	}

protected:
	FSlabAllocator Allocator;

private:
	virtual const TPagedArray<RowType>& GetRows() const override
	{
		return Rows;
	}

	TPagedArray<RowType> Rows;
};

} // namespace TraceServices
