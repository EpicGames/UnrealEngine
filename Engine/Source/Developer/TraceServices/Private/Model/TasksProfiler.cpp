// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/TasksProfiler.h"
#include "Model/TasksProfilerPrivate.h"
#include "AnalysisServicePrivate.h"
#include "Misc/ScopeLock.h"
#include "Misc/Timespan.h"
#include "Algo/BinarySearch.h"
#include "Common/Utils.h"
#include "Async/TaskGraphInterfaces.h"

//////////////////////////////////////////
// Handling of task traces
// To reduce the amount of traces, or when a particular trace event is not possible for a particular task type, they are often omitted. 
// In this case omitted traces are assumed and restored here.
// Different task types trace differently. Notable examples are:
// * Tasks System regular tasks: (almost) no surprises here, 
// ["created"] -> "launched" -> ["scheduled"] -> "started" -> "finished" -> "completed" -> "destroyed",
// where [trace] can be ommitted and assumed to happen at the same time and in the same thread as the subsequent trace.
// The only surprise here is that "started" can race "scheduled" traces during task retraction.
// * Tasks System task events (`FTaskEvent`): are never scheduled or executed, "launched" -> "completed" -> "destroyed". 
// "created" == "launched", "scheduled" == "started" == "finished" == "completed". 
// FTaskEvent is detected by "invalid" "started" timestamp when "completed" timestamp arrives.
// * TaskGraph regular tasks: behave mostly as Tasks System regular tasks except for the case when in the moment of their completion 
// they have incomplete "don't complete until" tasks (aka "nested" tasks). In this case they are traced as two different tasks: 
// the original one and the new one created for flagging the graph event completed when all nested tasks are completed. 
// the original task will have only "completed" trace at the moment when the second task is created, the second task will behave as a regular task.
// * TaskGraph standalone FGraphEvent (created by `FGraphEvent::CreateGraphEvent()`): similar to regular TaskGraph tasks, except w/o nested
// tasks - "completed" -> "destroyed".
// * ParallelFor tasks: usually - "launched" -> "started" -> "completed" -> "destroyed". "created" == "launched", "scheduled" == "launched", 
// "finished" == "completed". but if it was cancelled: "launched" -> "destroyed"

#if !defined(UE_TASK_TRACES_ENABLE_OLD_TASKS_SUPPORT)
#define UE_TASK_TRACES_ENABLE_OLD_TASKS_SUPPORT 1
#endif

namespace TraceServices
{
	FTasksProvider::FTasksProvider(IAnalysisSession& InSession)
		: Session(InSession)
		, EditableCounterProvider(EditCounterProvider(Session))
	{
#if 0
		////////////////////////////////
		// tests

		FAnalysisSessionEditScope _(Session);

		const int32 ValidThreadId = 42;
		const int32 InvalidThreadId = 0;
		const double InvalidTimestamp = 0;

		TArray64<TaskTrace::FId>& Thread = ExecutionThreads.FindOrAdd(ValidThreadId);

		auto MockTaskExecution = [this, Thread = &Thread](TaskTrace::FId TaskId, double StartedTimestamp, double FinishedTimestamp) -> FTaskInfo&
		{
			FTaskInfo& Task = GetOrCreateTask(TaskId);
			Task.Id = TaskId;
			Task.StartedTimestamp = StartedTimestamp;
			Task.FinishedTimestamp = FinishedTimestamp;
			Thread->Add(Task.Id);
			return Task;
		};

		FTaskInfo& Task1 = MockTaskExecution(0, 5, 10);
		FTaskInfo& Task2 = MockTaskExecution(1, 15, 20);
		
		check(GetTask(InvalidThreadId, InvalidTimestamp) == nullptr);
		check(GetTask(ValidThreadId, InvalidTimestamp) == nullptr);
		check(GetTask(ValidThreadId, 5) == &Task1);
		check(GetTask(ValidThreadId, 7) == &Task1);
		check(GetTask(ValidThreadId, 10) == nullptr);
		check(GetTask(ValidThreadId, 12) == nullptr);
		check(GetTask(ValidThreadId, 17) == &Task2);
		check(GetTask(ValidThreadId, 22) == nullptr);

		// reset
		FirstTaskId = TaskTrace::InvalidId;
		ExecutionThreads.Empty();
		Tasks.Empty();
#endif
	}

	void FTasksProvider::CreateCounters()
	{
		check(bCountersCreated == false);
		FAnalysisSessionEditScope _(Session);

		WaitingForPrerequisitesTasksCounter = EditableCounterProvider.CreateEditableCounter();
		WaitingForPrerequisitesTasksCounter->SetName(TEXT("Tasks::WaitingForPrerequisitesTasks"));
		WaitingForPrerequisitesTasksCounter->SetDescription(TEXT("Tasks: the number of tasks waiting for prerequisites (blocked by dependency)"));
		WaitingForPrerequisitesTasksCounter->SetIsFloatingPoint(false);

		TaskLatencyCounter = EditableCounterProvider.CreateEditableCounter();
		TaskLatencyCounter->SetName(TEXT("Tasks::TaskLatency"));
		TaskLatencyCounter->SetDescription(TEXT("Tasks: tasks latency - the time from scheduling to execution start"));
		TaskLatencyCounter->SetIsFloatingPoint(true);

		ScheduledTasksCounter = EditableCounterProvider.CreateEditableCounter();
		ScheduledTasksCounter->SetName(TEXT("Tasks::ScheduledTasks"));
		ScheduledTasksCounter->SetDescription(TEXT("Tasks: number of scheduled tasks excluding named threads (the size of the queue)"));
		ScheduledTasksCounter->SetIsFloatingPoint(false);

		NamedThreadsScheduledTasksCounter = EditableCounterProvider.CreateEditableCounter();
		NamedThreadsScheduledTasksCounter->SetName(TEXT("Tasks::NamedThreadsScheduledTasks"));
		NamedThreadsScheduledTasksCounter->SetDescription(TEXT("Tasks: number of scheduled tasks for named threads"));
		NamedThreadsScheduledTasksCounter->SetIsFloatingPoint(false);

		RunningTasksCounter = EditableCounterProvider.CreateEditableCounter();
		RunningTasksCounter->SetName(TEXT("Tasks::RunningTasks"));
		RunningTasksCounter->SetDescription(TEXT("Tasks: level of parallelism - the number of tasks being executed"));
		RunningTasksCounter->SetIsFloatingPoint(false);

		ExecutionTimeCounter = EditableCounterProvider.CreateEditableCounter();
		ExecutionTimeCounter->SetName(TEXT("Tasks::ExecutionTime"));
		ExecutionTimeCounter->SetDescription(TEXT("Tasks: execution time"));
		ExecutionTimeCounter->SetIsFloatingPoint(true);

		AliveTasksCounter = EditableCounterProvider.CreateEditableCounter();
		AliveTasksCounter->SetName(TEXT("Tasks::AliveTasks"));
		AliveTasksCounter->SetDescription(TEXT("Tasks: the numbers of tasks alive (created but not destroyed)"));
		AliveTasksCounter->SetIsFloatingPoint(false);

		AliveTasksSizeCounter = EditableCounterProvider.CreateEditableCounter();
		AliveTasksSizeCounter->SetName(TEXT("Tasks::AliveTasksSize"));
		AliveTasksSizeCounter->SetDescription(TEXT("Tasks: the total size of alive tasks (created but not destroyed)"));
		AliveTasksSizeCounter->SetIsFloatingPoint(false);

		bCountersCreated = true;
	}

	void FTasksProvider::Init(uint32 InVersion)
	{
		Version = InVersion;

		if (!bCountersCreated)
		{
			CreateCounters();
		}
	}

	void FTasksProvider::TaskCreated(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId, uint64 TaskSize)
	{
		UE_LOG(LogTraceServices, Verbose, TEXT("TaskCreated(TaskId: %" UINT64_FMT ", Timestamp %.6f)"), TaskId, Timestamp);

		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskCreated(TaskId %" UINT64_FMT ", Timestamp %.6f) skipped"), TaskId, Timestamp);
			return;
		}

		// some task events can be duplicates, as the result of generating them here if they are missing (some task events are not sent to reduce trace 
		// file size), but there's no way to distinguish missing and late events (task events can race and come out of order). A generated event comes
		// always before a real one, so the real one overrides the generated one silently. but as counters were already updated for a generated event,
		// we don't update them for a duplicate.
		bool bDuplicateEvent = Task->CreatedTimestamp != FTaskInfo::InvalidTimestamp;

		Task->CreatedTimestamp = Timestamp;
		Task->CreatedThreadId = ThreadId;
		Task->TaskSize = TaskSize;

		if (!bDuplicateEvent)
		{
			if (!bCountersCreated)
			{
				CreateCounters();
			}

			AliveTasksCounter->SetValue(Timestamp, ++AliveTasksNum);
			AliveTasksSizeCounter->SetValue(Timestamp, AliveTasksSize += TaskSize);
		}
	}

	void FTasksProvider::TaskLaunched(TaskTrace::FId TaskId, const TCHAR* DebugName, bool bTracked, int32 ThreadToExecuteOn, double Timestamp, uint32 ThreadId, uint64 TaskSize)
	{
		UE_LOG(LogTraceServices, Verbose, TEXT("TaskLaunched(TaskId: %" UINT64_FMT ", DebugName: %s, bTracked: %d, Timestamp %.6f)"), TaskId, DebugName, bTracked, Timestamp);

		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskLaunched(TaskId %" UINT64_FMT ", DebugName %s, bTracked %d, Timestamp %.6f) skipped"), TaskId, DebugName, bTracked, Timestamp);
			return;
		}

		if (Task->CreatedTimestamp == FTaskInfo::InvalidTimestamp) // created and launched in one go
		{
			TaskCreated(TaskId, Timestamp, ThreadId, TaskSize);
		}

		bool bDuplicateEvent = Task->LaunchedTimestamp != FTaskInfo::InvalidTimestamp;

		Task->DebugName = DebugName;
		Task->bTracked = bTracked;
		Task->ThreadToExecuteOn = ThreadToExecuteOn;
		Task->LaunchedTimestamp = Timestamp;
		Task->LaunchedThreadId = ThreadId;

		if (!bDuplicateEvent)
		{
			if (!bCountersCreated)
			{
				CreateCounters();
			}

			WaitingForPrerequisitesTasksCounter->SetValue(Timestamp, ++WaitingForPrerequisitesTasksNum);
		}
	}

	void FTasksProvider::TaskScheduled(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId)
	{
		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskScheduled(TaskId %" UINT64_FMT ", Timestamp %.6f) skipped"), TaskId, Timestamp);
			return;
		}

		bool bDuplicateEvent = Task->ScheduledTimestamp != FTaskInfo::InvalidTimestamp;

		Task->ScheduledTimestamp = Timestamp;
		Task->ScheduledThreadId = ThreadId;

		if (!bDuplicateEvent)
		{
			if (!bCountersCreated)
			{
				CreateCounters();
			}

			WaitingForPrerequisitesTasksCounter->SetValue(Timestamp, --WaitingForPrerequisitesTasksNum);
			if (IsNamedThread(Task->ThreadToExecuteOn))
			{
				NamedThreadsScheduledTasksCounter->SetValue(Timestamp, ++ScheduledTasksNum);
			}
			else
			{
				ScheduledTasksCounter->SetValue(Timestamp, ++ScheduledTasksNum);
			}
		}
	}

	void FTasksProvider::SubsequentAdded(TaskTrace::FId TaskId, TaskTrace::FId SubsequentId, double Timestamp, uint32 ThreadId)
	{
		InitTaskIdToIndexConversion(TaskId);

		// when FGraphEvent is used to wait for a notification, it doesn't have an associated task and so is not created or launched. 
		// In this case we need to create it and initialise it before registering the completion event
		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("SubsequentAdded(TaskId %" UINT64_FMT ", SubsequentId %" UINT64_FMT ", Timestamp %.6f) skipped, task %" UINT64_FMT " doesn't exists"), TaskId, SubsequentId, Timestamp, TaskId);
			return;
		}

		FTaskInfo* SubsequentTask = TryGetOrCreateTask(SubsequentId);
		if (SubsequentTask == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("SubsequentAdded(TaskId %" UINT64_FMT ", SubsequentId %" UINT64_FMT ", Timestamp %.6f) skipped, subsequent task %" UINT64_FMT " doesn't exists"), TaskId, SubsequentId, Timestamp, SubsequentId);
			return;
		}

		// if a subsequent is added before this task execution started, it's a subsequent that have this task as a (execution) prerequisite.
		// otherwise, it's a subsequent that have this task as a nested task (a completion prerequisite)
		if (SubsequentTask->StartedTimestamp == FTaskInfo::InvalidTimestamp || SubsequentTask->StartedTimestamp >= Timestamp)
		{
			AddRelative(TEXT("Subsequent"), TaskId, &FTaskInfo::Subsequents, SubsequentId, Timestamp, ThreadId);
			// make a backward link from the subsequent task to this task (prerequisite)
			AddRelative(TEXT("Prerequisite"), SubsequentId, &FTaskInfo::Prerequisites, TaskId, Timestamp, ThreadId);
		}
		else
		{
			AddRelative(TEXT("Parent"), TaskId, &FTaskInfo::ParentTasks, SubsequentId, Timestamp, ThreadId);
			// make a backward link from the parent task to this nested task
			AddRelative(TEXT("Nested"), SubsequentId, &FTaskInfo::NestedTasks, TaskId, Timestamp, ThreadId);
		}
	}

	void FTasksProvider::TaskStarted(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId)
	{
		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskStarted(TaskId %" UINT64_FMT ", Timestamp %.6f) skipped"), TaskId, Timestamp);
			return;
		}

		if (Task->ScheduledTimestamp == FTaskInfo::InvalidTimestamp && Task->LaunchedTimestamp != FTaskInfo::InvalidTimestamp)
		{
			// omitted "scheduled" event, generate it
			TaskScheduled(TaskId, Timestamp, Task->LaunchedThreadId); // `LaunchedThreadId` is just a value that potentially is closest to the truth
		}

		bool bDuplicateEvent = Task->StartedTimestamp != FTaskInfo::InvalidTimestamp;

		Task->StartedTimestamp = Timestamp;
		Task->StartedThreadId = ThreadId;

		ExecutionThreads.FindOrAdd(ThreadId).Add(TaskId);

		if (!bDuplicateEvent)
		{
			if (!bCountersCreated)
			{
				CreateCounters();
			}

			if (IsNamedThread(Task->ThreadToExecuteOn))
			{
				NamedThreadsScheduledTasksCounter->SetValue(Timestamp, --ScheduledTasksNum);
			}
			else
			{
				ScheduledTasksCounter->SetValue(Timestamp, --ScheduledTasksNum);
			}
			RunningTasksCounter->SetValue(Timestamp, ++RunningTasksNum);

			TaskLatencyCounter->SetValue(Timestamp, Task->StartedTimestamp - Task->ScheduledTimestamp);
		}
	}

	void FTasksProvider::TaskFinished(TaskTrace::FId TaskId, double Timestamp)
	{
		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskFinished(TaskId %" UINT64_FMT ", Timestamp %.6f) skipped"), TaskId, Timestamp);
			return;
		}

		if (Task->StartedTimestamp == FTaskInfo::InvalidTimestamp && Task->LaunchedTimestamp != FTaskInfo::InvalidTimestamp)
		{
			// missing "started" event for a task covered by this trace
			TaskStarted(TaskId, Timestamp, Task->LaunchedThreadId);
		}

		bool bDuplicateEvent = Task->FinishedTimestamp != FTaskInfo::InvalidTimestamp;

		Task->FinishedTimestamp = Timestamp;

		if (!bDuplicateEvent)
		{
			if (!bCountersCreated)
			{
				CreateCounters();
			}

			RunningTasksCounter->SetValue(Timestamp, --RunningTasksNum);
			ExecutionTimeCounter->SetValue(Timestamp, Task->FinishedTimestamp - Task->StartedTimestamp);
		}
	}

	void FTasksProvider::TaskCompleted(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId)
	{
		InitTaskIdToIndexConversion(TaskId);

		// when FGraphEvent is used to wait for a notification, it doesn't have an associated task and so is not created or launched. 
		// In this case we need to create it and initialise it before registering the completion event
		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskCompleted(TaskId %" UINT64_FMT ", Timestamp %.6f) skipped"), TaskId, Timestamp);
			return;
		}

		if (Task->FinishedTimestamp == FTaskInfo::InvalidTimestamp && Task->LaunchedTimestamp != FTaskInfo::InvalidTimestamp)
		{
			// missing "finished" event for a task that is covered by this trace, generate it
			TaskFinished(TaskId, Timestamp);
		}

		Task->CompletedTimestamp = Timestamp;
		Task->CompletedThreadId = ThreadId;
	}

	void FTasksProvider::TaskDestroyed(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId)
	{
		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetOrCreateTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("TaskDestroyed(TaskId %" UINT64_FMT ", Timestamp %.6f) skipped"), TaskId, Timestamp);
			return;
		}

		if (Task->CompletedTimestamp == FTaskInfo::InvalidTimestamp && Task->LaunchedTimestamp != FTaskInfo::InvalidTimestamp)
		{
			// missing "completed" event for a task that is covered by this trace
			TaskCompleted(TaskId, Timestamp, ThreadId);
		}

		bool bUpdateCounters = Task->LaunchedTimestamp != FTaskInfo::InvalidTimestamp; // we accounted this task in counters

		Task->DestroyedTimestamp = Timestamp;
		Task->DestroyedThreadId = ThreadId;

		if (bUpdateCounters)
		{
			if (!bCountersCreated)
			{
				CreateCounters();
			}

			AliveTasksCounter->SetValue(Timestamp, --AliveTasksNum);
			AliveTasksSizeCounter->SetValue(Timestamp, AliveTasksSize -= Task->TaskSize);
		}
	}

	void FTasksProvider::WaitingStarted(TArray<TaskTrace::FId> InTasks, double Timestamp, uint32 ThreadId)
	{
		FWaitingForTasks Waiting;
		Waiting.Tasks = MoveTemp(InTasks);
		Waiting.StartedTimestamp = Timestamp;

		WaitingThreads.FindOrAdd(ThreadId).Add(MoveTemp(Waiting));
	}

	void FTasksProvider::WaitingFinished(double Timestamp, uint32 ThreadId)
	{
		TArray64<FWaitingForTasks>* Thread = WaitingThreads.Find(ThreadId);
		if (Thread == nullptr)
		{
			UE_LOGF(LogTraceServices, Log, "WaitingFinished task event (Thread %d, Timestamp %.6f) skipped.", ThreadId, Timestamp);
			return;
		}

		Thread->Last().FinishedTimestamp = Timestamp;
	}

	void FTasksProvider::InitTaskIdToIndexConversion(TaskTrace::FId InFirstTaskId)
	{
		check(InFirstTaskId != TaskTrace::InvalidId);
		if (FirstTaskId == TaskTrace::InvalidId)
		{
			FirstTaskId = InFirstTaskId;
		}
	}

	int64 FTasksProvider::GetTaskIndex(TaskTrace::FId TaskId) const
	{
		return (int64)TaskId - FirstTaskId;
	}

	const FTaskInfo* FTasksProvider::TryGetTask(TaskTrace::FId TaskId) const
	{
		check(TaskId != TaskTrace::InvalidId);
		int64 TaskIndex = GetTaskIndex(TaskId);
		return Tasks.IsValidIndex(TaskIndex) ? &Tasks[TaskIndex] : nullptr;
	}

	FTaskInfo* FTasksProvider::TryGetTask(TaskTrace::FId TaskId)
	{
		return const_cast<FTaskInfo*>(const_cast<const FTasksProvider*>(this)->TryGetTask(TaskId)); // reuse the const version
	}

	FTaskInfo* FTasksProvider::TryGetOrCreateTask(TaskTrace::FId TaskId)
	{
		int64 TaskIndex = GetTaskIndex(TaskId);
		// traces can race, it's possible a trace with `TaskId = X` can come first, initialize `FirstTaskId` and only then a trace with 
		// `TaskId = X - 1` arrives. This will produce `TaskIndex < 0`. Such traces can happen only at the very beginning of the capture 
		// and are ignored
		if (TaskIndex < 0)
		{
			return nullptr;
		}

		if (TaskIndex >= Tasks.Num())
		{
			int64 PrevNum = Tasks.Num();
			Tasks.AddDefaulted(TaskIndex - Tasks.Num() + 1);
			for (int64 Index = PrevNum; Index != Tasks.Num(); ++Index)
			{
				Tasks[Index].Id = Index + FirstTaskId;
			}
		}

		return &Tasks[TaskIndex];
	}

	bool FTasksProvider::IsNamedThread(int32 Thread)
	{
		return ENamedThreads::GetThreadIndex((ENamedThreads::Type)Thread) != ENamedThreads::AnyThread;
	}

	void FTasksProvider::AddRelative(const TCHAR* RelationType, TaskTrace::FId TaskId, TArray<FTaskInfo::FRelationInfo> FTaskInfo::* RelationsPtr, TaskTrace::FId RelativeId, double Timestamp, uint32 ThreadId)
	{
		UE_LOG(LogTraceServices, Verbose, TEXT("%s (%" UINT64_FMT ") added to TaskId: %" UINT64_FMT ", Timestamp %.6f)"), RelationType, RelativeId, TaskId, Timestamp);

		InitTaskIdToIndexConversion(TaskId);

		FTaskInfo* Task = TryGetTask(TaskId);
		if (Task == nullptr)
		{
			UE_LOG(LogTraceServices, Log, TEXT("Add%s(TaskId %" UINT64_FMT ", OtherId: %" UINT64_FMT ", Timestamp %.6f) skipped"), RelationType, TaskId, RelativeId, Timestamp);
			return;
		}

		(Task->*RelationsPtr).Emplace(RelativeId, Timestamp, ThreadId);
	}

	/////////////////////////////////////////////////////////////////////////////////
	// ITasksProvider impl

	const FTaskInfo* FTasksProvider::TryGetTask(uint32 ThreadId, double Timestamp) const
	{
		const TArray64<TaskTrace::FId>* Thread = ExecutionThreads.Find(ThreadId);
		if (Thread == nullptr)
		{
			return nullptr;
		}

		int64 NextTaskIndex = Algo::LowerBound(*Thread, Timestamp, 
			[this](TaskTrace::FId TaskId, double Timestamp)
			{
				const FTaskInfo* Task = TryGetTask(TaskId);
				return Task != nullptr && Task->StartedTimestamp <= Timestamp;
			}
		);

		if (NextTaskIndex == 0)
		{
			return nullptr;
		}

		TaskTrace::FId TaskId = (*Thread)[NextTaskIndex - 1];
		const FTaskInfo* Task = TryGetTask(TaskId);
		return Task != nullptr && Task->FinishedTimestamp > Timestamp ? Task : nullptr;
	}

	const FWaitingForTasks* FTasksProvider::TryGetWaiting(const TCHAR* TimerName, uint32 ThreadId, double Timestamp) const
	{
		if (FCString::Strcmp(TimerName, TEXT("WaitUntilTasksComplete")) != 0 && 
			FCString::Strcmp(TimerName, TEXT("GameThreadWaitForTask")) != 0 &&
			FCString::Strcmp(TimerName, TEXT("Tasks::Wait")) != 0 &&
			FCString::Strcmp(TimerName, TEXT("Tasks::BusyWait")) != 0)
		{
			return nullptr;
		}

		const TArray64<FWaitingForTasks>* Thread = WaitingThreads.Find(ThreadId);
		if (Thread == nullptr)
		{
			return nullptr;
		}

		int64 NextWaitingIndex = Algo::LowerBound(*Thread, Timestamp,
			[this](const FWaitingForTasks& Waiting, double Timestamp)
			{
				return Waiting.StartedTimestamp <= Timestamp;
			}
		);

		if (NextWaitingIndex == 0)
		{
			return nullptr;
		}

		const FWaitingForTasks& Waiting = (*Thread)[NextWaitingIndex - 1];
		return Waiting.FinishedTimestamp > Timestamp || Waiting.FinishedTimestamp == FTaskInfo::InvalidTimestamp ? &Waiting : nullptr;
	}

	TArray<TaskTrace::FId> FTasksProvider::TryGetParallelForTasks(const TCHAR* TimerName, uint32 ThreadId, double StartTime, double EndTime) const
	{
		TArray<TaskTrace::FId> Result;

		if (FCString::Strcmp(TimerName, TEXT("ParallelFor")) != 0)
		{
			return Result;
		}

		EnumerateTasks(StartTime, EndTime, ETaskEnumerationOption::Alive,
			[ThreadId, StartTime, EndTime, &Result] (const FTaskInfo& Task)
			{
				if (Task.LaunchedThreadId == ThreadId && Task.LaunchedTimestamp > StartTime && Task.FinishedTimestamp < EndTime)
				{
					Result.Add(Task.Id);
				}
				return ETaskEnumerationResult::Continue;
			}
		);

		return Result;
	}

	int64 FTasksProvider::GetNumTasks() const
	{
		return Tasks.Num();
	}

	void FTasksProvider::EnumerateTasks(double StartTime, double EndTime, ETaskEnumerationOption EnumerationOption, TaskCallback Callback) const
	{
		using FSelector = bool(*)(double, double, const FTaskInfo&);
		FSelector Selectors[(uint32)ETaskEnumerationOption::Count] = 
		{
			// Alive
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// created before `StartTime` and destroyed after `EndTime`
				return (Task.CreatedTimestamp <= EndTime || Task.CreatedTimestamp == FTaskInfo::InvalidTimestamp) &&
					(Task.DestroyedTimestamp >= StartTime || Task.DestroyedTimestamp == FTaskInfo::InvalidTimestamp);
			},
			// Launched
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// launched between `StartTime` and `EndTime`
				return Task.LaunchedTimestamp >= StartTime && Task.LaunchedTimestamp <= EndTime;
			},
			// Active
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// completed before `StartTime` and destroyed after `EndTime`
				return (Task.StartedTimestamp < EndTime && Task.StartedTimestamp != FTaskInfo::InvalidTimestamp) &&
					(Task.FinishedTimestamp > StartTime || Task.FinishedTimestamp == FTaskInfo::InvalidTimestamp);
			},
			// WaitingForPrerequisites
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// launched before `StartTime` and scheduled after `EndTime`
				return (Task.LaunchedTimestamp <= StartTime && Task.LaunchedTimestamp != FTaskInfo::InvalidTimestamp) && 
					(Task.ScheduledTimestamp >= EndTime || Task.ScheduledTimestamp == FTaskInfo::InvalidTimestamp);
			},
			// Queued
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// scheduled before `StartTime` and started execution after `EndTime`
				return (Task.ScheduledTimestamp <= StartTime && Task.ScheduledTimestamp != FTaskInfo::InvalidTimestamp) &&
					(Task.StartedTimestamp >= EndTime || Task.StartedTimestamp == FTaskInfo::InvalidTimestamp);
			},
			// Executing
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// started execution before `StartTime` and finished after `EndTime`
				return (Task.StartedTimestamp <= StartTime && Task.StartedTimestamp != FTaskInfo::InvalidTimestamp) &&
					(Task.FinishedTimestamp >= EndTime || Task.FinishedTimestamp == FTaskInfo::InvalidTimestamp);
			},
			// WaitingForNested
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// finished execution before `StartTime` and completed after `EndTime`
				return (Task.FinishedTimestamp <= StartTime && Task.FinishedTimestamp != FTaskInfo::InvalidTimestamp) &&
					(Task.CompletedTimestamp >= EndTime || Task.CompletedTimestamp == FTaskInfo::InvalidTimestamp);
			},
			// Completed
			[](double StartTime, double EndTime, const FTaskInfo& Task)
			{	// completed before `StartTime` and destroyed after `EndTime`
				return (Task.CompletedTimestamp <= StartTime && Task.CompletedTimestamp != FTaskInfo::InvalidTimestamp) &&
					(Task.DestroyedTimestamp >= EndTime || Task.DestroyedTimestamp == FTaskInfo::InvalidTimestamp);
			},
		};

		auto Enumerate = [StartTime, EndTime, Callback = MoveTemp(Callback), this](auto Selector)
		{
			for (const TPair<uint32, TArray64<TaskTrace::FId>>& KeyValue : ExecutionThreads)
			{
				const TArray64<TaskTrace::FId>& TaskIds = KeyValue.Value;
				for (int64 i = 0; i != TaskIds.Num(); ++i)
				{
					const FTaskInfo* Task = TryGetTask(TaskIds[i]);
					check(Task);
					if (Task != nullptr && Selector(StartTime, EndTime, *Task))
					{
						if (Callback(*Task) == ETaskEnumerationResult::Stop)
						{
							break;
						}
					}
				}
			}
		};

		Enumerate(Selectors[(uint32)EnumerationOption]);
	}
}
