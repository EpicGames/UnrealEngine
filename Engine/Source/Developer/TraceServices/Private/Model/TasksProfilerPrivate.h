// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/TasksProfiler.h"
#include "TraceServices/Model/Counters.h"
#include "HAL/CriticalSection.h"
#include "Async/TaskTrace.h"
#include "Containers/Map.h"

namespace TraceServices
{
	// tasks database, filled by the analyser and consumed by UI.
	// Creates and fills a number of counters. Unprocessed info can be queried by `ITasksProvider` interface
	class FTasksProvider : public ITasksProvider
	{
	public:
		explicit FTasksProvider(IAnalysisSession& Session);
		virtual ~FTasksProvider() = default;

		void Init(uint32 Version);

		// traces consumption
		void TaskCreated(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId, uint64 TaskSize);
		void TaskLaunched(TaskTrace::FId TaskId, const TCHAR* DebugName, bool bTracked, int32 ThreadToExecuteOn, double Timestamp, uint32 ThreadId, uint64 TaskSize);
		void TaskScheduled(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId);
		void SubsequentAdded(TaskTrace::FId TaskId, TaskTrace::FId SubsequentId, double Timestamp, uint32 ThreadId);
		void TaskStarted(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId);
		void TaskFinished(TaskTrace::FId TaskId, double Timestamp);
		void TaskCompleted(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId);
		void TaskDestroyed(TaskTrace::FId TaskId, double Timestamp, uint32 ThreadId);

		void WaitingStarted(TArray<TaskTrace::FId> Tasks, double Timestamp, uint32 ThreadId);
		void WaitingFinished(double Timestamp, uint32 ThreadId);

		// ITasksProvider impl
		virtual const FTaskInfo* TryGetTask(uint32 ThreadId, double Timestamp) const override;
		virtual const FTaskInfo* TryGetTask(TaskTrace::FId TaskId) const;
		virtual const FWaitingForTasks* TryGetWaiting(const TCHAR* TimerName, uint32 ThreadId, double Timestamp) const override;
		virtual TArray<TaskTrace::FId> TryGetParallelForTasks(const TCHAR* TimerName, uint32 ThreadId, double StartTime, double EndTime) const override;
		virtual int64 GetNumTasks() const override;
		virtual void EnumerateTasks(double StartTime, double EndTime, ETaskEnumerationOption EnumerationOption, TaskCallback Callback) const override;

	private:
		// initializes conversion of task id to its index in the internal container, if it's not initialized already
		void InitTaskIdToIndexConversion(TaskTrace::FId FirstTaskId);

		// translates `TaskId` to the index into `Tasks` array, 
		int64 GetTaskIndex(TaskTrace::FId TaskId) const; // `FistTaskId` must be initialised before the first use

		FTaskInfo* TryGetTask(TaskTrace::FId TaskId);
		// default-constructs a task if required
		FTaskInfo* TryGetOrCreateTask(TaskTrace::FId TaskId);

		static bool IsNamedThread(int32 Thread);

		void CreateCounters();

		void AddRelative(const TCHAR* RelativeType, TaskTrace::FId TaskId, TArray<FTaskInfo::FRelationInfo> FTaskInfo::* RelationsPtr, TaskTrace::FId RelativeId, double Timestamp, uint32 ThreadId);

	private:
		IAnalysisSession& Session; // for synchronisation with consumers

		uint32 Version = ~(uint32)0;

		// task IDs are auto-incremented almost monotonic numbers, they are used as indices into tasks array minus the first task ID
		// "almost monotonic" because a trace with TaskId==N+1 can arrive before all traces with TaskId==N
		TArray64<FTaskInfo> Tasks;
		
		// on execution start, when it becomes known which thread will execute a task, all tasks are grouped by their execution threads, where
		// they are naturally sorted by `StartedTimestamp`
		TMap<uint32, TArray64<TaskTrace::FId>> ExecutionThreads; // ThreadId -> [TaskId]

		TMap<uint32, TArray64<FWaitingForTasks>> WaitingThreads; // ThreadId -> [Waiting]

		// task ID of the first trace. as traces can race, it's possible some traces of the previous task will be lost, which is not a probem
		TaskTrace::FId FirstTaskId = TaskTrace::InvalidId;

		// number of tasks waiting to be scheduled, because they are blocked by dependencies or they will be immediately scheduled
		// and reported by another task trace
		int64 WaitingForPrerequisitesTasksNum = 0;
		// number of non named threads tasks sheduled for execution
		int64 ScheduledTasksNum = 0;
		// number of named threads tasks scheduled for execution
		int64 NamedThreadsScheduledTasksNum = 0;
		// number of tasks being executed
		int64 RunningTasksNum = 0;
		// a bool that is set when the counters are created
		bool bCountersCreated = false;
		// number of alive tasks
		int64 AliveTasksNum = 0;
		int64 AliveTasksSize = 0;

		IEditableCounterProvider& EditableCounterProvider;
		IEditableCounter* TaskLatencyCounter;
		IEditableCounter* WaitingForPrerequisitesTasksCounter;
		IEditableCounter* ScheduledTasksCounter;
		IEditableCounter* NamedThreadsScheduledTasksCounter;
		IEditableCounter* RunningTasksCounter;
		IEditableCounter* ExecutionTimeCounter;
		IEditableCounter* AliveTasksCounter;
		IEditableCounter* AliveTasksSizeCounter;
	};
} // namespace TraceServices
