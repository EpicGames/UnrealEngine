// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/Threads.h"
#include "Model/ThreadsPrivate.h"

#include "AnalysisServicePrivate.h"
#include "Common/StringStore.h"
#include "Misc/ScopeLock.h"

namespace TraceServices
{

FThreadProvider::FThreadProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
}

FThreadProvider::~FThreadProvider()
{
	for (FThreadInfoInternal* Thread : SortedThreads)
	{
		delete Thread;
	}
}

void FThreadProvider::AddGameThread(uint32 Id)
{
	const FString Name = FName(NAME_GameThread).GetPlainNameString();
	AddThread(Id, *Name, EThreadPriority(-2));
}

void FThreadProvider::AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority)
{
	Session.WriteAccessCheck();

	FThreadInfoInternal* ThreadInfo;
	if (!ThreadMap.Contains(Id))
	{
		ThreadInfo = new FThreadInfoInternal();
		ThreadInfo->Id = Id;
		ThreadInfo->FallbackSortOrder = SortedThreads.Num();
		SortedThreads.Add(ThreadInfo);
		ThreadMap.Add(Id, ThreadInfo);
	}
	else
	{
		ThreadInfo = ThreadMap[Id];
	}
	if (Name != nullptr)
	{
		if (Name[0] != 0)
		{
			ThreadInfo->Name = Session.StoreString(Name);
		}

		if (!FCString::Strcmp(Name, TEXT("RHIThread")))
		{
			const TCHAR* GroupName = Session.StoreString(TEXT("Render"));
			SetThreadGroup(Id, GroupName);
		}

		ThreadInfo->PrioritySortOrder = GetPrioritySortOrder(Priority);
	}
	SortThreads();
	++ModCount;
}

void FThreadProvider::SetThreadPriority(uint32 Id, EThreadPriority Priority)
{
	Session.WriteAccessCheck();

	check(ThreadMap.Contains(Id));
	FThreadInfoInternal* ThreadInfo = ThreadMap[Id];
	ensure(ThreadInfo->PrioritySortOrder != -2);
	ThreadInfo->PrioritySortOrder = GetPrioritySortOrder(Priority);
	SortThreads();
	++ModCount;
}

void FThreadProvider::SetThreadGroup(uint32 Id, const TCHAR* GroupName)
{
	Session.WriteAccessCheck();

	check(ThreadMap.Contains(Id));
	FThreadInfoInternal* ThreadInfo = ThreadMap[Id];
	ThreadInfo->GroupName = GroupName;
	ThreadInfo->GroupSortOrder = GetGroupSortOrder(GroupName);
	SortThreads();
	++ModCount;
}

void FThreadProvider::EnumerateThreads(TFunctionRef<void(const FThreadInfo &)> Callback) const
{
	Session.ReadAccessCheck();

	for (const FThreadInfoInternal* Thread : SortedThreads)
	{
		FThreadInfo ThreadInfo;
		ThreadInfo.Id = Thread->Id;
		ThreadInfo.Name = Thread->Name;
		ThreadInfo.GroupName = Thread->GroupName;
		Callback(ThreadInfo);
	}
}

const TCHAR* FThreadProvider::GetThreadName(uint32 ThreadId) const
{
	Session.ReadAccessCheck();

	const FThreadInfoInternal* const* FindIt = ThreadMap.Find(ThreadId);
	if (!FindIt)
	{
		return TEXT("");
	}
	return (*FindIt)->Name;
}

void FThreadProvider::SortThreads()
{
	Algo::SortBy(SortedThreads, [](const FThreadInfoInternal* In) -> const FThreadInfoInternal&
	{
		return *In;
	});
}

uint32 FThreadProvider::GetGroupSortOrder(const TCHAR* GroupName)
{
	if (!GroupName)
	{
		return uint32(-1);
	}
	else if (!FCString::Strcmp(GroupName, TEXT("Render")))
	{
		return 0;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("AsyncLoading")))
	{
		return 1;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("TaskGraphHigh")))
	{
		return 2;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("TaskGraphNormal")))
	{
		return 3;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("TaskGraphLow")))
	{
		return 4;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("LargeThreadPool")))
	{
		return 5;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("ThreadPool")))
	{
		return 6;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("BackgroundThreadPool")))
	{
		return 7;
	}
	else if (!FCString::Strcmp(GroupName, TEXT("IOThreadPool")))
	{
		return 8;
	}
	else
	{
		return FCrc::Strihash_DEPRECATED(GroupName);
	}
}

int32 FThreadProvider::GetPrioritySortOrder(EThreadPriority ThreadPriority)
{
	switch (ThreadPriority)
	{
	case TPri_TimeCritical:
		return 0;
	case TPri_Highest:
		return 1;
	case TPri_AboveNormal:
		return 2;
	case TPri_Normal:
		return 3;
	case TPri_SlightlyBelowNormal:
		return 4;
	case TPri_BelowNormal:
		return 5;
	case TPri_Lowest:
		return 6;
	default:
		return int32(ThreadPriority);
	}
}

bool FThreadProvider::FThreadInfoInternal::operator<(const FThreadInfoInternal& Other) const
{
	if (PrioritySortOrder < 0 || Other.PrioritySortOrder < 0)
	{
		return PrioritySortOrder < Other.PrioritySortOrder;
	}

	if (GroupSortOrder == Other.GroupSortOrder)
	{
		if (PrioritySortOrder == Other.PrioritySortOrder)
		{
			return FallbackSortOrder < Other.FallbackSortOrder;
		}
		return PrioritySortOrder < Other.PrioritySortOrder;
	}
	return GroupSortOrder < Other.GroupSortOrder;
}

FName GetThreadProviderName()
{
	static const FName Name("ThreadProvider");
	return Name;
}

const IThreadProvider& ReadThreadProvider(const IAnalysisSession& Session)
{
	return *Session.ReadProvider<IThreadProvider>(GetThreadProviderName());
}

IEditableThreadProvider& EditThreadProvider(IAnalysisSession& Session)
{
	return *Session.EditProvider<IEditableThreadProvider>(GetThreadProviderName());
}

} // namespace TraceServices
