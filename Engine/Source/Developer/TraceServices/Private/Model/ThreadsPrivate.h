// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/PlatformAffinity.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Threads.h"

namespace TraceServices
{

class FThreadProvider
	: public IThreadProvider
	, public IEditableThreadProvider
{
public:
	explicit FThreadProvider(IAnalysisSession& Session);
	virtual ~FThreadProvider();

	// IEditableThreadProvider
	virtual void AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority) override;
	virtual const IThreadProvider* GetReadProvider() const  override { return this; }

	void AddGameThread(uint32 Id);
	void SetThreadPriority(uint32 Id, EThreadPriority Priority);
	void SetThreadGroup(uint32 Id, const TCHAR* GroupName);

	// IThreadProvider
	virtual uint64 GetModCount() const override { return ModCount; }
	virtual void EnumerateThreads(TFunctionRef<void(const FThreadInfo&)> Callback) const override;
	virtual const TCHAR* GetThreadName(uint32 ThreadId) const override;

private:
	struct FThreadInfoInternal
	{
		uint32 Id = 0;
		const TCHAR* Name = TEXT("UnnamedThread");
		uint32 GroupSortOrder = ~0u;
		int32 PrioritySortOrder = INT_MAX;
		uint32 FallbackSortOrder = ~0u;
		const TCHAR* GroupName = nullptr;

		bool operator<(const FThreadInfoInternal& Other) const;
	};

	void SortThreads();
	static int32 GetPrioritySortOrder(EThreadPriority ThreadPriority);
	static uint32 GetGroupSortOrder(const TCHAR* GroupName);

	IAnalysisSession& Session;
	uint64 ModCount = 0;
	TMap<uint32, FThreadInfoInternal*> ThreadMap;
	TArray<FThreadInfoInternal*> SortedThreads;
};

} // namespace TraceServices
