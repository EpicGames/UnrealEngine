// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/Model/TimingProfiler.h"

#include "Templates/TypeHash.h"

// TraceServices
#include "AnalysisServicePrivate.h"
#include "Common/StringStore.h"
#include "Common/TimelineStatistics.h"
#include "Common/Utils.h"
#include "Model/TimingProfilerPrivate.h"
#include "TraceServices/Model/Frames.h"

#define TIMING_PROFILER_DETECT_TIMER_CHANGES 0
#define TIMING_PROFILER_LOG_INVALID_METADATA_1 //{ UE_LOGF(LogTraceServices, Warning, "[TimingProfiler][1] MetadataTimerId=%u >= 0", MetadataTimerId); UE_DEBUG_BREAK(); }
#define TIMING_PROFILER_LOG_INVALID_METADATA_2 //{ UE_LOGF(LogTraceServices, Warning, "[TimingProfiler][2] MetadataTimerId=%u >= Num=%u", MetadataTimerId, uint32(Metadatas.Num())); UE_DEBUG_BREAK(); }
#define TIMING_PROFILER_LOG_INVALID_METADATA_SPEC //{ UE_LOGF(LogTraceServices, Warning, "[TimingProfiler][3] MetadataSpecId=%u >= Num=%u", MetadataSpecId, uint32(MetadataSpecs.Num())); UE_DEBUG_BREAK(); }

namespace TraceServices
{

FTimingProfilerProvider::FTimingProfilerProvider(IAnalysisSession& InSession)
	: Session(InSession)
{
	for (uint32 Index = 0; Index < uint32(ETimingProfilerTimerType::Count); ++Index)
	{
		bHasData[Index] = false;
	}

	// Adds the old GPU timelines.
	Timelines.Add(MakeShared<TimelineInternal>(Session.GetLinearAllocator())); // GPU Index 0
	Timelines.Add(MakeShared<TimelineInternal>(Session.GetLinearAllocator())); // GPU Index 1

	// Adds the Verse Sampling timeline.
	Timelines.Add(MakeShared<TimelineInternal>(Session.GetLinearAllocator())); // Verse Sampling

	AggregatedStatsTableLayout.
		AddColumn<const TCHAR*>([](const FTimingProfilerAggregatedStats& Row)
			{
				return Row.Timer->Name;
			},
			TEXT("Name")).
		AddColumn(&FTimingProfilerAggregatedStats::InstanceCount, TEXT("Count")).
		AddColumn(&FTimingProfilerAggregatedStats::AverageInstanceCount, TEXT("C.Avg")).
		AddColumn(&FTimingProfilerAggregatedStats::TotalInclusiveTime, TEXT("Incl")).
		AddColumn(&FTimingProfilerAggregatedStats::MinInclusiveTime, TEXT("I.Min")).
		AddColumn(&FTimingProfilerAggregatedStats::MaxInclusiveTime, TEXT("I.Max")).
		AddColumn(&FTimingProfilerAggregatedStats::AverageInclusiveTime, TEXT("I.Avg")).
		AddColumn(&FTimingProfilerAggregatedStats::MedianInclusiveTime, TEXT("I.Med")).
		AddColumn(&FTimingProfilerAggregatedStats::TotalExclusiveTime, TEXT("Excl")).
		AddColumn(&FTimingProfilerAggregatedStats::MinExclusiveTime, TEXT("E.Min")).
		AddColumn(&FTimingProfilerAggregatedStats::MaxExclusiveTime, TEXT("E.Max")).
		AddColumn(&FTimingProfilerAggregatedStats::AverageExclusiveTime, TEXT("E.Avg")).
		AddColumn(&FTimingProfilerAggregatedStats::MedianExclusiveTime, TEXT("E.Med"));

	// Add a default timer.
	FTimingProfilerTimer& Timer = Timers.AddDefaulted_GetRef();
	Timer.Id = 0;
	Timer.Name = TEXT("<default>");
	Timer.Type = ETimingProfilerTimerType::CpuScope;
}

FTimingProfilerProvider::~FTimingProfilerProvider()
{
}

bool FTimingProfilerProvider::HasCpuTiming() const
{
	Session.ReadAccessCheck();

	return bHasData[uint32(ETimingProfilerTimerType::CpuScope)];
}

bool FTimingProfilerProvider::HasGpuTiming() const
{
	Session.ReadAccessCheck();

	return bHasData[uint32(ETimingProfilerTimerType::GpuScope)] ||
		Timelines[GpuTimelineIndex]->GetEventCount() > 0 ||
		Timelines[Gpu2TimelineIndex]->GetEventCount() > 0;
}

bool FTimingProfilerProvider::HasVerseTiming() const
{
	Session.ReadAccessCheck();

	return bHasData[uint32(ETimingProfilerTimerType::VerseSampling)] ||
		Timelines[VerseTimelineIndex]->GetEventCount() > 0;
}

void FTimingProfilerProvider::ReadTimers(TFunctionRef<void(const ITimingProfilerTimerReader&)> Callback) const
{
	Session.ReadAccessCheck();
	Callback(*this);
}

uint32 FTimingProfilerProvider::GetTimerCount() const
{
	return Timers.Num();
}

const FTimingProfilerTimer* FTimingProfilerProvider::GetTimer(uint32 TimerId) const
{
	if (int32(TimerId) < 0)
	{
		TimerId = ~TimerId;
		TimerId = Metadatas[TimerId].TimerId;
	}
	return (TimerId < uint32(Timers.Num())) ? Timers.GetData() + TimerId : nullptr;
}

uint32 FTimingProfilerProvider::AddTimer(ETimingProfilerTimerType TimerType)
{
	Session.WriteAccessCheck();

	check(uint32(TimerType) < uint32(ETimingProfilerTimerType::Count));
	bHasData[uint32(TimerType)] = true;

	FTimingProfilerTimer& Timer = Timers.AddDefaulted_GetRef();
	Timer.Id = Timers.Num() - 1;
	Timer.Type = TimerType;
	return Timer.Id;
}

uint32 FTimingProfilerProvider::AddTimer(ETimingProfilerTimerType TimerType, FStringView TimerName)
{
	Session.WriteAccessCheck();

	check(uint32(TimerType) < uint32(ETimingProfilerTimerType::Count));
	bHasData[uint32(TimerType)] = true;

	FTimingProfilerTimer& Timer = Timers.AddDefaulted_GetRef();
	Timer.Id = Timers.Num() - 1;
	Timer.Name = Session.StoreString(TimerName);
	Timer.Type = TimerType;
	return Timer.Id;
}

uint32 FTimingProfilerProvider::AddTimer(ETimingProfilerTimerType TimerType, FStringView TimerName, const TCHAR* File, uint32 Line)
{
	Session.WriteAccessCheck();

	check(uint32(TimerType) < uint32(ETimingProfilerTimerType::Count));
	bHasData[uint32(TimerType)] = true;

	FTimingProfilerTimer& Timer = Timers.AddDefaulted_GetRef();
	Timer.Id = Timers.Num() - 1;
	Timer.Name = Session.StoreString(TimerName);
	Timer.File = File ? Session.StoreString(File) : nullptr;
	Timer.Line = Line;
	Timer.Type = TimerType;
	return Timer.Id;
}

void FTimingProfilerProvider::SetTimerName(uint32 TimerId, FStringView Name)
{
	Session.WriteAccessCheck();

	if (ensure(TimerId < uint32(Timers.Num())))
	{
		FTimingProfilerTimer& Timer = Timers[TimerId];

		const TCHAR* NewName = Session.StoreString(Name);
#if TIMING_PROFILER_DETECT_TIMER_CHANGES
		if (!ensure(Timer.Name == nullptr || Timer.Name == NewName || Timer.Name[0] == TEXT('<')))
		{
			UE_LOGF(LogTraceServices, Warning, "[TimingProfiler] Name changed for timer %u: from \"%ls\" to \"%ls\" !!!",
				TimerId, Timer.Name, NewName);
		}
#endif
		Timer.Name = NewName;
	}
}

void FTimingProfilerProvider::SetTimerNameAndLocation(uint32 TimerId, FStringView Name, const TCHAR* File, uint32 Line)
{
	Session.WriteAccessCheck();

	if (ensure(TimerId < uint32(Timers.Num())))
	{
		FTimingProfilerTimer& Timer = Timers[TimerId];

		const TCHAR* NewName = Session.StoreString(Name);
#if TIMING_PROFILER_DETECT_TIMER_CHANGES
		if (!ensure(Timer.Name == nullptr || Timer.Name == NewName || Timer.Name[0] == TEXT('<')))
		{
			UE_LOGF(LogTraceServices, Warning, "[TimingProfiler] Name changed for timer %u: from \"%ls\" to \"%ls\" !!!",
				TimerId, Timer.Name, NewName);
		}
#endif
		Timer.Name = NewName;

		const TCHAR* NewFile = File ? Session.StoreString(File) : nullptr;
#if TIMING_PROFILER_DETECT_TIMER_CHANGES
		if (!ensure(Timer.File == nullptr || Timer.File == NewFile))
		{
			UE_LOGF(LogTraceServices, Warning, "[TimingProfiler] Location changed for timer %u (%ls): from \"%ls\" (%u) to \"%ls\" (%u) !!!",
				TimerId, Timer.Name, Timer.File, Timer.Line, NewFile, Line);
		}
#endif
		Timer.File = NewFile;

		Timer.Line = Line;
	}
}

void FTimingProfilerProvider::SetTimerLocation(uint32 TimerId, const TCHAR* File, uint32 Line)
{
	Session.WriteAccessCheck();

	if (ensure(TimerId < uint32(Timers.Num())))
	{
		FTimingProfilerTimer& Timer = Timers[TimerId];

		const TCHAR* NewFile = File ? Session.StoreString(File) : nullptr;
#if TIMING_PROFILER_DETECT_TIMER_CHANGES
		if (!ensure(Timer.File == nullptr || Timer.File == NewFile))
		{
			UE_LOGF(LogTraceServices, Warning, "[TimingProfiler] Location changed for timer %u (%ls): from \"%ls\" (%u) to \"%ls\" (%u) !!!",
				TimerId, Timer.Name, Timer.File, Timer.Line, NewFile, Line);
		}
#endif
		Timer.File = NewFile;

		Timer.Line = Line;
	}
}

void FTimingProfilerProvider::SetMetadataSpec(uint32 TimerId, uint32 MetadataSpecId)
{
	Session.WriteAccessCheck();

	if (ensure(TimerId < uint32(Timers.Num())))
	{
		FTimingProfilerTimer& Timer = Timers[TimerId];
#if TIMING_PROFILER_DETECT_TIMER_CHANGES
		if (!ensure(Timer.MetadataSpecId == FMetadataSpec::InvalidMetadataSpecId ||
					Timer.MetadataSpecId == MetadataSpecId))
		{
			UE_LOGF(LogTraceServices, Warning, "Metadata changed for timer %u (%ls): from \"%ls\" (%d fields) to \"%ls\" (%d fields) !!!",
				TimerId, Timer.Name,
				MetadataSpecs[Timer.MetadataSpecId].Format, MetadataSpecs[Timer.MetadataSpecId].FieldNames.Num(),
				MetadataSpecs[MetadataSpecId].Format, MetadataSpecs[MetadataSpecId].FieldNames.Num());
		}
#endif
		Timer.MetadataSpecId = MetadataSpecId;
	}
}

uint32 FTimingProfilerProvider::AddMetadata(uint32 OriginalTimerId, TArray<uint8>&& Metadata)
{
	Session.WriteAccessCheck();

	uint32 MetadataId = Metadatas.Num();
	Metadatas.Add({ MoveTemp(Metadata), OriginalTimerId });

	return ~MetadataId;
}

uint32 FTimingProfilerProvider::GetOriginalTimerIdFromMetadata(uint32 MetadataTimerId) const
{
	Session.ReadAccessCheck();

	if (int32(MetadataTimerId) >= 0)
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_1;
		return MetadataTimerId;
	}

	MetadataTimerId = ~MetadataTimerId;
	if (MetadataTimerId >= uint32(Metadatas.Num()))
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_2;
		return 0;
	}

	const FMetadata& Metadata = Metadatas[MetadataTimerId];
	return Metadata.TimerId;
}

TArrayView<const uint8> FTimingProfilerProvider::GetMetadata(uint32 MetadataTimerId) const
{
	Session.ReadAccessCheck();

	if (int32(MetadataTimerId) >= 0)
	{
		//TIMING_PROFILER_LOG_INVALID_METADATA_1; // expected in the normal use case
		return TArrayView<const uint8>();
	}

	MetadataTimerId = ~MetadataTimerId;
	if (MetadataTimerId >= uint32(Metadatas.Num()))
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_2;
		return TArrayView<const uint8>();
	}

	const FMetadata& Metadata = Metadatas[MetadataTimerId];
	return Metadata.Payload;
}

void FTimingProfilerProvider::SetMetadata(uint32 MetadataTimerId, TArray<uint8>&& Metadata)
{
	Session.WriteAccessCheck();

	if (int32(MetadataTimerId) >= 0)
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_1;
		return;
	}

	MetadataTimerId = ~MetadataTimerId;
	if (MetadataTimerId >= uint32(Metadatas.Num()))
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_2;
		return;
	}

	FMetadata& MetadataToReplace = Metadatas[MetadataTimerId];
	MetadataToReplace.Payload = Metadata;
}

void FTimingProfilerProvider::SetMetadata(uint32 MetadataTimerId, TArray<uint8>&& Metadata, uint32 NewTimerId)
{
	Session.WriteAccessCheck();

	if (int32(MetadataTimerId) >= 0)
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_1;
		return;
	}

	MetadataTimerId = ~MetadataTimerId;
	if (MetadataTimerId >= uint32(Metadatas.Num()))
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_2;
		return;
	}

	FMetadata& MetadataToReplace = Metadatas[MetadataTimerId];
	MetadataToReplace.TimerId = NewTimerId;
	MetadataToReplace.Payload = Metadata;
}

TArrayView<uint8> FTimingProfilerProvider::GetEditableMetadata(uint32 MetadataTimerId)
{
	Session.WriteAccessCheck();

	if (int32(MetadataTimerId) >= 0)
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_1;
		return TArrayView<uint8>();
	}

	MetadataTimerId = ~MetadataTimerId;
	if (MetadataTimerId >= uint32(Metadatas.Num()))
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_2;
		return TArrayView<uint8>();
	}

	FMetadata& Metadata = Metadatas[MetadataTimerId];
	return Metadata.Payload;
}

uint32 FTimingProfilerProvider::AddMetadataSpec(FMetadataSpec&& Metadata)
{
	Session.WriteAccessCheck();

	uint32 Hash = PointerHash(Metadata.Format);
	for (const TCHAR* Field : Metadata.FieldNames)
	{
		Hash ^= PointerHash(Field);
	}

	TArray<uint32> FoundMetadataSpecIds;
	MetadataSpecMap.MultiFind(Hash, FoundMetadataSpecIds);
	if (FoundMetadataSpecIds.Num() > 0)
	{
		for (uint32 FoundMetadataSpecId : FoundMetadataSpecIds)
		{
			check(FoundMetadataSpecId < uint32(MetadataSpecs.Num()));
			const FMetadataSpec& FoundMetadataSpec = MetadataSpecs[FoundMetadataSpecId];
			if (Metadata.Format == FoundMetadataSpec.Format)
				//|| FCString::Strcmp(Metadata.Format, FoundMetadataSpec.Format) == 0)
			{
				const int32 NumFields = Metadata.FieldNames.Num();
				if (NumFields == FoundMetadataSpec.FieldNames.Num())
				{
					bool bSame = true;
					for (int32 FieldIndex = 0; FieldIndex < NumFields; ++FieldIndex)
					{
						if (Metadata.FieldNames[FieldIndex] == FoundMetadataSpec.FieldNames[FieldIndex])
							//|| FCString::Strcmp(Metadata.FieldNames[FieldIndex], FoundMetadataSpec.FieldNames[FieldIndex]) == 0)
						{
							continue;
						}
						bSame = false;
						break;
					}
					if (bSame)
					{
						return FoundMetadataSpecId;
					}
				}
			}
		}
	}

	uint32 MetadataSpecId = MetadataSpecs.Num();
	MetadataSpecs.Add(MoveTemp(Metadata));
	MetadataSpecMap.Add(Hash, MetadataSpecId);

	return MetadataSpecId;
}

const FMetadataSpec* FTimingProfilerProvider::GetMetadataSpec(uint32 MetadataSpecId) const
{
	Session.ReadAccessCheck();

	if (MetadataSpecId >= (uint32)MetadataSpecs.Num())
	{
		TIMING_PROFILER_LOG_INVALID_METADATA_SPEC;
		return nullptr;
	}

	return &MetadataSpecs[MetadataSpecId];
}

void FTimingProfilerProvider::MakeTimerDisplayName(const FTimingProfilerTimer& InTimer, TStringBuilder<1024>& InOutDisplayName) const
{
	Session.ReadAccessCheck();

	if (InTimer.Name)
	{
		InOutDisplayName.Append(InTimer.Name);
	}

	if (InTimer.HasValidMetadataSpecId())
	{
		const TraceServices::FMetadataSpec* Spec = GetMetadataSpec(InTimer.MetadataSpecId);
		if (Spec && Spec->Format)
		{
			TStringView Format = Spec->Format;
			Format.TrimStartInline();
			if (!Format.StartsWith(TEXT("- ")) && !Format.StartsWith(TEXT("= ")))
			{
				InOutDisplayName.Append(TEXT(" - "));
			}
			InOutDisplayName.Append(Format);
		}
	}
}

IEditableTimeline<FTimingProfilerEvent>& FTimingProfilerProvider::GetCpuThreadEditableTimeline(uint32 ThreadId)
{
	Session.WriteAccessCheck();

	const uint32* FoundTimelineIndex = CpuThreadTimelineIndexMap.Find(ThreadId);
	if (!FoundTimelineIndex)
	{
		TSharedRef<TimelineInternal> Timeline = MakeShared<TimelineInternal>(Session.GetLinearAllocator());
		uint32 TimelineIndex = Timelines.Num();
		CpuThreadTimelineIndexMap.Add(ThreadId, TimelineIndex);
		Timelines.Add(Timeline);
		bHasData[uint32(ETimingProfilerTimerType::CpuScope)] = true;
		return Timeline.Get();
	}
	else
	{
		return Timelines[*FoundTimelineIndex].Get();
	}
}

FTimingProfilerProvider::TimelineInternal& FTimingProfilerProvider::EditGpuTimeline()
{
	Session.WriteAccessCheck();

	bHasData[uint32(ETimingProfilerTimerType::GpuScope)] = true;
	return Timelines[GpuTimelineIndex].Get();
}

FTimingProfilerProvider::TimelineInternal& FTimingProfilerProvider::EditGpu2Timeline()
{
	Session.WriteAccessCheck();

	bHasData[uint32(ETimingProfilerTimerType::GpuScope)] = true;
	return Timelines[Gpu2TimelineIndex].Get();
}

void FTimingProfilerProvider::AddGpuQueue(uint32 QueueId, uint8 GPU, uint8 Index, uint8 Type, const TCHAR* Name)
{
	Session.WriteAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (!FoundQueueIndex)
	{
		GpuQueueIdToQueueIndexMap.Add(QueueId, (uint32)GpuQueues.Num());

		GpuQueueData.AddDefaulted();
		FGpuQueueInfo& GpuQueue = GpuQueues.AddDefaulted_GetRef();

		GpuQueue.Id = QueueId;
		GpuQueue.GPU = GPU;
		GpuQueue.Index = Index;
		GpuQueue.Type = Type;
		GpuQueue.Name = Name;

		GpuQueue.TimelineIndex = Timelines.Num();
		TSharedRef<TimelineInternal> Timeline = MakeShared<TimelineInternal>(Session.GetLinearAllocator());
		Timelines.Add(Timeline);

		GpuQueue.WorkTimelineIndex = Timelines.Num();
		TSharedRef<TimelineInternal> WorkTimeline = MakeShared<TimelineInternal>(Session.GetLinearAllocator());
		Timelines.Add(WorkTimeline);

		bHasData[uint32(ETimingProfilerTimerType::GpuScope)] = true;
	}
}

void FTimingProfilerProvider::AddGpuSignalFence(uint32 QueueId, const FGpuSignalFence& Fence)
{
	Session.WriteAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex)
	{
		check(*FoundQueueIndex < (uint32)GpuQueueData.Num());

		FGpuQueueData& Data = GpuQueueData[*FoundQueueIndex];

		if (Data.SignalFenceArray.Num())
		{
			ensure(Fence.Timestamp >= Data.SignalFenceArray.Last().Timestamp);
		}

		Data.SignalFenceArray.Add(Fence);
	}
}

void FTimingProfilerProvider::AddGpuWaitFence(uint32 QueueId, const FGpuWaitFence& Fence)
{
	Session.WriteAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex)
	{
		check(*FoundQueueIndex < (uint32)GpuQueueData.Num());

		FGpuQueueData& Data = GpuQueueData[*FoundQueueIndex];

		if (Data.WaitFenceArray.Num())
		{
			ensure(Fence.Timestamp >= Data.WaitFenceArray.Last().Timestamp);
		}

		Data.WaitFenceArray.Add(Fence);
	}
}

void FTimingProfilerProvider::EnumerateGpuSignalFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuSignalFencesCallback Callback) const
{
	Session.ReadAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex == nullptr)
	{
		return;
	}

	const FGpuQueueData& Data = GpuQueueData[*FoundQueueIndex];
	const TArray<FGpuSignalFence>& Array = Data.SignalFenceArray;

	int32 StartIndex = Algo::LowerBoundBy(Array, StartTime,
		[](const FGpuSignalFence& Fence) { return Fence.Timestamp; });

	for (int32 Index = StartIndex; Index < Array.Num() && Array[Index].Timestamp <= EndTime; ++Index)
	{
		if (Callback(Array[Index]) == EEnumerateResult::Stop)
		{
			break;
		}
	}
}

void FTimingProfilerProvider::EnumerateGpuWaitFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuWaitFencesCallback Callback) const
{
	Session.ReadAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex == nullptr)
	{
		return;
	}

	const FGpuQueueData& Data = GpuQueueData[*FoundQueueIndex];
	const TArray<FGpuWaitFence>& Array = Data.WaitFenceArray;

	int32 StartIndex = Algo::LowerBoundBy(Array, StartTime,
		[](const FGpuWaitFence& Fence) { return Fence.Timestamp; });

	for (int32 Index = StartIndex; Index < Array.Num() && Array[Index].Timestamp <= EndTime; ++Index)
	{
		if (Callback(Array[Index]) == EEnumerateResult::Stop)
		{
			break;
		}
	}
}

void FTimingProfilerProvider::EnumerateGpuFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuFencesCallback Callback) const
{
	Session.ReadAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex == nullptr)
	{
		return;
	}

	const FGpuQueueData& Data = GpuQueueData[*FoundQueueIndex];
	const TArray<FGpuSignalFence>& SignalArray = Data.SignalFenceArray;
	const TArray<FGpuWaitFence>& WaitArray = Data.WaitFenceArray;

	int32 SignalFenceIndex = Algo::LowerBoundBy(SignalArray, StartTime,
		[](const FGpuSignalFence& Fence) { return Fence.Timestamp; });

	int32 WaitFenceIndex = Algo::LowerBoundBy(WaitArray, StartTime,
		[](const FGpuWaitFence& Fence) { return Fence.Timestamp; });

	FGpuFenceWrapper Wrapper;

	while (true)
	{
		const FGpuSignalFence* SignalFence = nullptr;
		const FGpuWaitFence* WaitFence = nullptr;

		if (SignalFenceIndex < SignalArray.Num() && SignalArray[SignalFenceIndex].Timestamp <= EndTime)
		{
			SignalFence = &SignalArray[SignalFenceIndex];
		}

		if (WaitFenceIndex < WaitArray.Num() && WaitArray[WaitFenceIndex].Timestamp <= EndTime)
		{
			WaitFence = &WaitArray[WaitFenceIndex];
		}

		if (SignalFence && WaitFence)
		{
			if (SignalFence->Timestamp <= WaitFence->Timestamp)
			{
				Wrapper.FenceType = EGpuFenceType::SignalFence;
				Wrapper.Fence.Set<const FGpuSignalFence*>(SignalFence);
				++SignalFenceIndex;
			}
			else
			{
				Wrapper.FenceType = EGpuFenceType::WaitFence;
				Wrapper.Fence.Set<const FGpuWaitFence*>(WaitFence);
				++WaitFenceIndex;
			}
		}
		else if (SignalFence)
		{
			Wrapper.FenceType = EGpuFenceType::SignalFence;
			Wrapper.Fence.Set<const FGpuSignalFence*>(SignalFence);
			++SignalFenceIndex;
		}
		else if (WaitFence)
		{
			Wrapper.FenceType = EGpuFenceType::WaitFence;
			Wrapper.Fence.Set<const FGpuWaitFence*>(WaitFence);
			++WaitFenceIndex;
		}
		else
		{
			break;
		}

		Callback(Wrapper);
	}
}

void FTimingProfilerProvider::EnumerateResolvedGpuFences(uint32 QueueId, double StartTime, double EndTime, EnumerateResolvedGpuFencesCallback Callback) const
{
	Session.ReadAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex == nullptr)
	{
		return;
	}

	const FGpuQueueData& QueueData = GpuQueueData[*FoundQueueIndex];
	const TArray<FGpuSignalFence>& SignalFenceArray = QueueData.SignalFenceArray;

	int32 SignalFenceIndex = Algo::LowerBoundBy(SignalFenceArray, StartTime,
		[](const FGpuSignalFence& Fence) { return Fence.Timestamp; });

	while (SignalFenceIndex < SignalFenceArray.Num() && SignalFenceArray[SignalFenceIndex].Timestamp <= EndTime)
	{
		const FGpuSignalFence& SignalFence = SignalFenceArray[SignalFenceIndex];
		uint64 PrevSignalFenceValue = 0;
		if (SignalFenceIndex > 0)
		{
			PrevSignalFenceValue = SignalFenceArray[SignalFenceIndex - 1].Value;
		}

		for (uint32 QueueIndex = 0; QueueIndex < (uint32) GpuQueueData.Num(); ++QueueIndex)
		{
			if (QueueIndex == *FoundQueueIndex)
			{
				continue;
			}

			const FGpuQueueData& TargetQueueData = GpuQueueData[QueueIndex];
			const TArray<FGpuWaitFence>& TargetQueueArray = TargetQueueData.WaitFenceArray;

			int32 TargetIndex = Algo::LowerBoundBy(TargetQueueArray, PrevSignalFenceValue,
				[](const FGpuWaitFence& Fence) { return Fence.Value; });

			while (TargetIndex < TargetQueueArray.Num() &&
				   TargetQueueArray[TargetIndex].QueueToWaitForId == QueueId &&
				   TargetQueueArray[TargetIndex].Value <= SignalFence.Value)
			{
				if (((TargetQueueArray[TargetIndex].Value > PrevSignalFenceValue) || SignalFence.Value == 0))
				{
					uint32 WaitFenceQueueId = GpuQueues[QueueIndex].Id;
					Callback(QueueId, SignalFence, WaitFenceQueueId, TargetQueueArray[TargetIndex]);
				}
				++TargetIndex;
			}
		}
		++SignalFenceIndex;
	}

	const TArray<FGpuWaitFence>& WaitFenceArray = QueueData.WaitFenceArray;

	int32 WaitFenceIndex = Algo::LowerBoundBy(WaitFenceArray, StartTime,
		[](const FGpuWaitFence& Fence) { return Fence.Timestamp; });

	while (WaitFenceIndex < WaitFenceArray.Num() && WaitFenceArray[WaitFenceIndex].Timestamp <= EndTime)
	{
		const FGpuWaitFence& WaitFence = WaitFenceArray[WaitFenceIndex];

		const uint32* SignalQueueIndex = GpuQueueIdToQueueIndexMap.Find(WaitFence.QueueToWaitForId);
		if (SignalQueueIndex == nullptr)
		{
			continue;
		}

		const FGpuQueueData& SignalQueueData = GpuQueueData[*SignalQueueIndex];

		const TArray<FGpuSignalFence>& TargetQueueArray = SignalQueueData.SignalFenceArray;

		int32 TargetIndex = Algo::LowerBoundBy(TargetQueueArray, WaitFence.Value,
			[](const FGpuSignalFence& Fence) { return Fence.Value; });

		if (TargetIndex < TargetQueueArray.Num())
		{
			Callback(WaitFence.QueueToWaitForId, TargetQueueArray[TargetIndex], QueueId, WaitFence);
		}

		++WaitFenceIndex;
	}
}

IEditableTimeline<FTimingProfilerEvent>* FTimingProfilerProvider::GetGpuQueueEditableTimeline(uint32 QueueId)
{
	Session.WriteAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex)
	{
		check(*FoundQueueIndex < (uint32)GpuQueues.Num());
		uint32 TimelineIndex = GpuQueues[*FoundQueueIndex].TimelineIndex;
		check(TimelineIndex < (uint32)Timelines.Num());
		return &Timelines[TimelineIndex].Get();
	}

	return nullptr;
}

IEditableTimeline<FTimingProfilerEvent>* FTimingProfilerProvider::GetGpuQueueWorkEditableTimeline(uint32 QueueId)
{
	Session.WriteAccessCheck();

	const uint32* FoundQueueIndex = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (FoundQueueIndex)
	{
		check(*FoundQueueIndex < (uint32)GpuQueues.Num());
		uint32 TimelineIndex = GpuQueues[*FoundQueueIndex].WorkTimelineIndex;
		check(TimelineIndex < (uint32)Timelines.Num());
		return &Timelines[TimelineIndex].Get();
	}

	return nullptr;
}

bool FTimingProfilerProvider::GetCpuThreadTimelineIndex(uint32 ThreadId, uint32& OutTimelineIndex) const
{
	Session.ReadAccessCheck();

	const uint32* FoundTimelineIndex = CpuThreadTimelineIndexMap.Find(ThreadId);
	if (FoundTimelineIndex)
	{
		OutTimelineIndex = *FoundTimelineIndex;
		return true;
	}
	return false;
}

bool FTimingProfilerProvider::GetGpuTimelineIndex(uint32& OutTimelineIndex) const
{
	Session.ReadAccessCheck();

	OutTimelineIndex = GpuTimelineIndex;
	return true;
}

bool FTimingProfilerProvider::GetGpu2TimelineIndex(uint32& OutTimelineIndex) const
{
	Session.ReadAccessCheck();

	OutTimelineIndex = Gpu2TimelineIndex;
	return true;
}

void FTimingProfilerProvider::EnumerateGpuQueues(TFunctionRef<void(const FGpuQueueInfo&)> Callback) const
{
	Session.ReadAccessCheck();

	for (const FGpuQueueInfo& GpuQueue : GpuQueues)
	{
		Callback(GpuQueue);
	}
}

bool FTimingProfilerProvider::GetGpuQueueTimelineIndex(uint32 QueueId, uint32& OutTimelineIndex) const
{
	Session.ReadAccessCheck();

	const uint32* Index = GpuQueueIdToQueueIndexMap.Find(QueueId);
	if (Index)
	{
		OutTimelineIndex = GpuQueues[*Index].TimelineIndex;
		return true;
	}
	return false;
}

bool FTimingProfilerProvider::GetVerseTimelineIndex(uint32& OutTimelineIndex) const
{
	Session.ReadAccessCheck();

	OutTimelineIndex = VerseTimelineIndex;
	return true;
}

IEditableTimeline<FTimingProfilerEvent>* FTimingProfilerProvider::GetVerseEditableTimeline()
{
	Session.WriteAccessCheck();

	return &Timelines[VerseTimelineIndex].Get();
}

bool FTimingProfilerProvider::ReadTimeline(uint32 Index, TFunctionRef<void(const Timeline &)> Callback) const
{
	Session.ReadAccessCheck();

	if (Index < uint32(Timelines.Num()))
	{
		Callback(*Timelines[Index]);
		return true;
	}
	else
	{
		return false;
	}
}

void FTimingProfilerProvider::EnumerateTimelines(TFunctionRef<void(const Timeline&)> Callback) const
{
	Session.ReadAccessCheck();

	for (const auto& Timeline : Timelines)
	{
		Callback(*Timeline);
	}
}

ITable<FTimingProfilerAggregatedStats>* FTimingProfilerProvider::CreateAggregation(const FCreateAggregationParams& Params) const
{
	Session.ReadAccessCheck();

	TArray<const ITimingProfilerTimeline*> IncludedTimelines;

	//////////////////////////////////////////////////
	// GPU

	if (Params.GpuQueueFilter)
	{
		for (const FGpuQueueInfo& GpuQueue : GpuQueues)
		{
			if (Params.GpuQueueFilter(GpuQueue.Id))
			{
				IncludedTimelines.Add(&Timelines[GpuQueue.TimelineIndex].Get());
			}
		}
	}
	if (Params.bIncludeOldGpu1)
	{
		IncludedTimelines.Add(&Timelines[GpuTimelineIndex].Get());
	}
	if (Params.bIncludeOldGpu2)
	{
		IncludedTimelines.Add(&Timelines[Gpu2TimelineIndex].Get());
	}

	//////////////////////////////////////////////////
	// Verse

	if (Params.bIncludeVerseSampling)
	{
		IncludedTimelines.Add(&Timelines[VerseTimelineIndex].Get());
	}

	//////////////////////////////////////////////////
	// CPU

	if (Params.CpuThreadFilter)
	{
		for (const auto& KV : CpuThreadTimelineIndexMap)
		{
			if (Params.CpuThreadFilter(KV.Key))
			{
				IncludedTimelines.Add(&Timelines[KV.Value].Get());
			}
		}
	}

	//////////////////////////////////////////////////
	// Custom Timelines

	for (const ITimingProfilerTimeline* Timeline : Params.CustomTimelines)
	{
		IncludedTimelines.Add(Timeline);
	}

	//////////////////////////////////////////////////

	return CreateAggregationInternal(IncludedTimelines, Params);
}

ITable<FTimingProfilerAggregatedStats>* FTimingProfilerProvider::CreateAggregationInternal(TArray<const ITimingProfilerTimeline*>& IncludedTimelines, const FCreateAggregationParams& Params) const
{
	Session.ReadAccessCheck();

	auto BucketMappingFunc = [this](const TimelineInternal::EventType& Event) -> const FTimingProfilerTimer*
	{
		return GetTimer(Event.TimerIndex);
	};

	FTimelineStatistics::TAggregationResult<const FTimingProfilerTimer*, FAggregatedTimingStats> Aggregation;

	if (Params.FrameType == ETraceFrameType::TraceFrameType_Count)
	{
		if (Params.IntervalStart <= Session.GetDurationSeconds())
		{
			// Do not allow inf for the end time.
			double EndTime = FMath::Min(Params.IntervalEnd, Session.GetDurationSeconds());
			FTimelineStatistics::CreateAggregation(IncludedTimelines, BucketMappingFunc, Params.IntervalStart, EndTime, Params.CancellationToken, Aggregation);
		}
	}
	else
	{
		TArray<FFrameData> Frames;
		const IFrameProvider& FrameProvider = ReadFrameProvider(Session);
		FrameProvider.EnumerateFrames(Params.FrameType, Params.IntervalStart, Params.IntervalEnd,
			[&Frames]
			(const FFrame& Frame)
			{
				FFrameData NewFrameData;
				NewFrameData.StartTime = Frame.StartTime;
				NewFrameData.EndTime = Frame.EndTime;

				Frames.Add(NewFrameData);
			});

		if (Frames.Num() > 0)
		{
			// Do not allow inf for the last frame end time.
			Frames[Frames.Num() - 1].EndTime = FMath::Min(Session.GetDurationSeconds(), Frames[Frames.Num() - 1].EndTime);
			FTimelineStatistics::CreateFrameStatsAggregation(IncludedTimelines, BucketMappingFunc, Frames, Params.CancellationToken, Aggregation, RatioOfThreadsToUse);
		}
	}

	TTable<FTimingProfilerAggregatedStats>* Table = new TTable<FTimingProfilerAggregatedStats>(AggregatedStatsTableLayout);

	if (Params.CancellationToken.IsValid() && Params.CancellationToken->ShouldCancel())
	{
		return Table;
	}

	if (Params.SortBy == FCreateAggregationParams::ESortBy::TotalInclusiveTime)
	{
		auto SortLambda = (Params.SortOrder == FCreateAggregationParams::ESortOrder::Descending) ?
			[](const FAggregatedTimingStats& First, const FAggregatedTimingStats& Second) -> bool
			{
				return First.TotalInclusiveTime > Second.TotalInclusiveTime;
			}
			:
			[](const FAggregatedTimingStats& First, const FAggregatedTimingStats& Second) -> bool
			{
				return First.TotalInclusiveTime < Second.TotalInclusiveTime;
			};

		Aggregation.GetMap().ValueStableSort(SortLambda);
	}

	int EntryLimit = Params.TableEntryLimit;
	for (const auto& KV : Aggregation.GetMap())
	{
		FTimingProfilerAggregatedStats& Row = Table->AddRow();
		Row.Timer = KV.Key;
		const FAggregatedTimingStats& Stats = KV.Value;
		Row.InstanceCount = Stats.InstanceCount;
		Row.AverageInstanceCount = Stats.AverageInstanceCount;
		Row.TotalInclusiveTime = Stats.TotalInclusiveTime;
		Row.MinInclusiveTime = Stats.MinInclusiveTime;
		Row.MaxInclusiveTime = Stats.MaxInclusiveTime;
		Row.AverageInclusiveTime = Stats.AverageInclusiveTime;
		Row.MedianInclusiveTime = Stats.MedianInclusiveTime;
		Row.TotalExclusiveTime = Stats.TotalExclusiveTime;
		Row.MinExclusiveTime = Stats.MinExclusiveTime;
		Row.MaxExclusiveTime = Stats.MaxExclusiveTime;
		Row.AverageExclusiveTime = Stats.AverageExclusiveTime;
		Row.MedianExclusiveTime = Stats.MedianExclusiveTime;

		if (UNLIKELY(EntryLimit > 0))
		{
			if (UNLIKELY(--EntryLimit == 0))
			{
				break;
			}
		}
	}
	return Table;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingProfilerButterfly
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingProfilerButterfly
	: public ITimingProfilerButterfly
{
public:
	FTimingProfilerButterfly();
	virtual ~FTimingProfilerButterfly() = default;
	virtual const FTimingProfilerButterflyNode& GenerateCallersTree(uint32 TimerId) override;
	virtual const FTimingProfilerButterflyNode& GenerateCalleesTree(uint32 TimerId) override;

private:
	void CreateCallersTreeRecursive(const FTimingProfilerButterflyNode* TimerNode, const FTimingProfilerButterflyNode* RootNode, FTimingProfilerButterflyNode* OutputParent);
	void CreateCalleesTreeRecursive(const FTimingProfilerButterflyNode* TimerNode, FTimingProfilerButterflyNode* OutputParent);

	FSlabAllocator Allocator;
	TPagedArray<FTimingProfilerButterflyNode> Nodes;
	TArray<TArray<FTimingProfilerButterflyNode*>> TimerCallstacksMap;
	TMap<uint32, FTimingProfilerButterflyNode*> CachedCallerTrees;
	TMap<uint32, FTimingProfilerButterflyNode*> CachedCalleeTrees;

	friend class FTimingProfilerProvider;
};

FTimingProfilerButterfly::FTimingProfilerButterfly()
	: Allocator(2 << 20)
	, Nodes(Allocator, 1024)
{
}

void FTimingProfilerButterfly::CreateCallersTreeRecursive(const FTimingProfilerButterflyNode* TimerNode, const FTimingProfilerButterflyNode* RootNode, FTimingProfilerButterflyNode* OutputParent)
{
	if (!TimerNode)
	{
		return;
	}
	FTimingProfilerButterflyNode* AggregatedChildNode = nullptr;
	for (FTimingProfilerButterflyNode* Candidate : OutputParent->Children)
	{
		if (Candidate->Timer == TimerNode->Timer)
		{
			AggregatedChildNode = Candidate;
			break;
		}
	}
	if (!AggregatedChildNode)
	{
		AggregatedChildNode = &Nodes.PushBack();
		AggregatedChildNode->Timer = TimerNode->Timer;
		OutputParent->Children.Add(AggregatedChildNode);
		AggregatedChildNode->Parent = OutputParent;
	}

	AggregatedChildNode->InclusiveTime += RootNode->InclusiveTime;
	AggregatedChildNode->ExclusiveTime += RootNode->ExclusiveTime;
	AggregatedChildNode->Count += RootNode->Count;

	CreateCallersTreeRecursive(TimerNode->Parent, RootNode, AggregatedChildNode);
}

const FTimingProfilerButterflyNode& FTimingProfilerButterfly::GenerateCallersTree(uint32 TimerId)
{
	FTimingProfilerButterflyNode** Cached = CachedCallerTrees.Find(TimerId);
	if (Cached)
	{
		return **Cached;
	}

	FTimingProfilerButterflyNode* Root = &Nodes.PushBack();
	for (FTimingProfilerButterflyNode* TimerNode : TimerCallstacksMap[TimerId])
	{
		Root->Timer = TimerNode->Timer;
		Root->InclusiveTime += TimerNode->InclusiveTime;
		Root->ExclusiveTime += TimerNode->ExclusiveTime;
		Root->Count += TimerNode->Count;

		CreateCallersTreeRecursive(TimerNode->Parent, TimerNode, Root);
	}
	CachedCallerTrees.Add(TimerId, Root);
	return *Root;
}

void FTimingProfilerButterfly::CreateCalleesTreeRecursive(const FTimingProfilerButterflyNode* TimerNode, FTimingProfilerButterflyNode* OutputParent)
{
	for (const FTimingProfilerButterflyNode* ChildNode : TimerNode->Children)
	{
		FTimingProfilerButterflyNode* AggregatedChildNode = nullptr;
		for (FTimingProfilerButterflyNode* Candidate : OutputParent->Children)
		{
			if (Candidate->Timer == ChildNode->Timer)
			{
				AggregatedChildNode = Candidate;
				break;
			}
		}
		if (!AggregatedChildNode)
		{
			AggregatedChildNode = &Nodes.PushBack();
			AggregatedChildNode->Timer = ChildNode->Timer;
			OutputParent->Children.Add(AggregatedChildNode);
			AggregatedChildNode->Parent = OutputParent;
		}
		AggregatedChildNode->InclusiveTime += ChildNode->InclusiveTime;
		AggregatedChildNode->ExclusiveTime += ChildNode->ExclusiveTime;
		AggregatedChildNode->Count += ChildNode->Count;

		CreateCalleesTreeRecursive(ChildNode, AggregatedChildNode);
	}
}

const FTimingProfilerButterflyNode& FTimingProfilerButterfly::GenerateCalleesTree(uint32 TimerId)
{
	FTimingProfilerButterflyNode** Cached = CachedCalleeTrees.Find(TimerId);
	if (Cached)
	{
		return **Cached;
	}

	FTimingProfilerButterflyNode* Root = &Nodes.PushBack();
	for (FTimingProfilerButterflyNode* TimerNode : TimerCallstacksMap[TimerId])
	{
		Root->Timer = TimerNode->Timer;
		Root->InclusiveTime += TimerNode->InclusiveTime;
		Root->ExclusiveTime += TimerNode->ExclusiveTime;
		Root->Count += TimerNode->Count;

		CreateCalleesTreeRecursive(TimerNode, Root);
	}
	CachedCalleeTrees.Add(TimerId, Root);
	return *Root;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimingProfilerCallstackKey
{
	bool operator==(const FTimingProfilerCallstackKey& Other) const
	{
		return Other.TimerStack == TimerStack;
	}

	friend uint32 GetTypeHash(const FTimingProfilerCallstackKey& Key)
	{
		return Key.Hash;
	}

	TArray<uint32> TimerStack;
	uint32 Hash;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

ITimingProfilerButterfly* FTimingProfilerProvider::CreateButterfly(const FCreateButterflyParams& Params) const
{
	Session.ReadAccessCheck();

	TArray<const ITimingProfilerTimeline*> IncludedTimelines;

	//////////////////////////////////////////////////
	// GPU

	if (Params.GpuQueueFilter)
	{
		for (const FGpuQueueInfo& GpuQueue : GpuQueues)
		{
			if (Params.GpuQueueFilter(GpuQueue.Id))
			{
				IncludedTimelines.Add(&Timelines[GpuQueue.TimelineIndex].Get());
			}
		}
	}
	if (Params.bIncludeOldGpu1)
	{
		IncludedTimelines.Add(&Timelines[GpuTimelineIndex].Get());
	}
	if (Params.bIncludeOldGpu2)
	{
		IncludedTimelines.Add(&Timelines[Gpu2TimelineIndex].Get());
	}

	//////////////////////////////////////////////////
	// Verse

	if (Params.bIncludeVerseSampling)
	{
		IncludedTimelines.Add(&Timelines[VerseTimelineIndex].Get());
	}

	//////////////////////////////////////////////////
	// CPU

	if (Params.CpuThreadFilter)
	{
		for (const auto& KV : CpuThreadTimelineIndexMap)
		{
			if (Params.CpuThreadFilter(KV.Key))
			{
				IncludedTimelines.Add(&Timelines[KV.Value].Get());
			}
		}
	}

	//////////////////////////////////////////////////
	// Custom Timelines

	for (const ITimingProfilerTimeline* Timeline : Params.CustomTimelines)
	{
		IncludedTimelines.Add(Timeline);
	}

	//////////////////////////////////////////////////

	return CreateButterflyInternal(IncludedTimelines, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ITimingProfilerButterfly* FTimingProfilerProvider::CreateButterflyInternal(TArray<const ITimingProfilerTimeline*>& IncludedTimelines, const FCreateButterflyParams& Params) const
{
	Session.ReadAccessCheck();

	FTimingProfilerButterfly* Butterfly = new FTimingProfilerButterfly();
	Butterfly->TimerCallstacksMap.AddDefaulted(Timers.Num());

	FTimingProfilerCallstackKey CurrentCallstackKey;
	CurrentCallstackKey.TimerStack.Reserve(1024);

	struct FLocalStackEntry
	{
		FTimingProfilerButterflyNode* Node = nullptr;
		double StartTime = 0.0;
		double ExclusiveTime = 0.0;
		uint32 CurrentCallstackHash = 0;
		bool bIsRecursive = false;
	};

	TArray<FLocalStackEntry> CurrentCallstack;
	CurrentCallstack.Reserve(1024);

	TMap<FTimingProfilerCallstackKey, TTuple<FTimingProfilerButterflyNode*, bool>> CallstackNodeMap;

	double LastTime = 0.0;
	for (const ITimingProfilerTimeline* Timeline : IncludedTimelines)
	{
		Timeline->EnumerateEvents(Params.IntervalStart, Params.IntervalEnd,
			[this, &Params, &CurrentCallstackKey, &CurrentCallstack, &CallstackNodeMap, &LastTime, Butterfly]
			(bool IsEnter, double Time, const FTimingProfilerEvent& Event)
			{
				Time = FMath::Clamp(Time, Params.IntervalStart, Params.IntervalEnd);
				FTimingProfilerButterflyNode* ParentNode = nullptr;
				uint32 ParentCallstackHash = 17;
				if (CurrentCallstack.Num())
				{
					FLocalStackEntry& StackEntry = CurrentCallstack.Top();
					ParentNode = StackEntry.Node;
					ParentCallstackHash = StackEntry.CurrentCallstackHash;
					StackEntry.ExclusiveTime += Time - LastTime;
				}
				LastTime = Time;
				if (IsEnter)
				{
					const FTimingProfilerTimer* Timer = GetTimer(Event.TimerIndex);
					check(Timer != nullptr);

					FLocalStackEntry& StackEntry = CurrentCallstack.AddDefaulted_GetRef();
					StackEntry.StartTime = Time;
					StackEntry.CurrentCallstackHash = ParentCallstackHash * 17 + Timer->Id;

					CurrentCallstackKey.TimerStack.Push(Timer->Id);
					CurrentCallstackKey.Hash = StackEntry.CurrentCallstackHash;

					TTuple<FTimingProfilerButterflyNode*, bool>* FindIt = CallstackNodeMap.Find(CurrentCallstackKey);
					if (FindIt)
					{
						StackEntry.Node = FindIt->Get<0>();
						StackEntry.bIsRecursive = FindIt->Get<1>();
					}
					else
					{
						for (int32 StackIndex = 0, StackEnd = CurrentCallstack.Num() - 1; StackIndex < StackEnd; ++StackIndex)
						{
							if (CurrentCallstack[StackIndex].Node->Timer == Timer)
							{
								StackEntry.Node = CurrentCallstack[StackIndex].Node;
								StackEntry.bIsRecursive = true;
								break;
							}
						}

						if (!StackEntry.Node)
						{
							StackEntry.Node = &Butterfly->Nodes.PushBack();
							StackEntry.Node->InclusiveTime = 0.0;
							StackEntry.Node->ExclusiveTime = 0.0;
							StackEntry.Node->Count = 0;
							StackEntry.Node->Timer = Timer;
							Butterfly->TimerCallstacksMap[Timer->Id].Add(StackEntry.Node);

							StackEntry.Node->Parent = ParentNode;
							if (ParentNode)
							{
								ParentNode->Children.Add(StackEntry.Node);
							}
						}
						CallstackNodeMap.Add(CurrentCallstackKey, MakeTuple(StackEntry.Node, StackEntry.bIsRecursive));
					}
				}
				else
				{
					FLocalStackEntry& StackEntry = CurrentCallstack.Top();
					double InclusiveTime = Time - StackEntry.StartTime;
					check(InclusiveTime >= 0.0);
					check(StackEntry.ExclusiveTime >= 0.0 && StackEntry.ExclusiveTime <= InclusiveTime);
					if (!StackEntry.bIsRecursive)
					{
						StackEntry.Node->InclusiveTime += InclusiveTime;
					}
					StackEntry.Node->ExclusiveTime += StackEntry.ExclusiveTime;
					++StackEntry.Node->Count;

					CurrentCallstack.Pop(EAllowShrinking::No);
					CurrentCallstackKey.TimerStack.Pop(EAllowShrinking::No);
				}

				return EEventEnumerate::Continue;
			});
	}
	return Butterfly;
}

} // namespace TraceServices

#undef TIMING_PROFILER_LOG_INVALID_METADATA_SPEC
#undef TIMING_PROFILER_LOG_INVALID_METADATA_2
#undef TIMING_PROFILER_LOG_INVALID_METADATA_1
#undef TIMING_PROFILER_DETECT_TIMER_CHANGES
