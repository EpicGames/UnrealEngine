// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/TimingProfiler.h"
#include "TraceServices/Containers/SlabAllocator.h"
#include "Model/MonotonicTimeline.h"
#include "Model/Tables.h"

namespace TraceServices
{

class FAnalysisSessionLock;
class FStringStore;

struct FGpuQueueData
{
	TArray<FGpuSignalFence> SignalFenceArray;
	TArray<FGpuWaitFence> WaitFenceArray;
};

class FTimingProfilerProvider
	: public ITimingProfilerProvider
	, public ITimingProfilerTimerReader
	, public IEditableTimingProfilerProvider
{
public:
	typedef TMonotonicTimeline<FTimingProfilerEvent> TimelineInternal;

	explicit FTimingProfilerProvider(IAnalysisSession& InSession);
	virtual ~FTimingProfilerProvider();

	//////////////////////////////////////////////////
	// ITimingProfilerProvider

	// Timelines

	virtual bool ReadTimeline(uint32 Index, TFunctionRef<void(const Timeline&)> Callback) const override;
	virtual uint32 GetTimelineCount() const override { return Timelines.Num(); }
	virtual void EnumerateTimelines(TFunctionRef<void(const Timeline&)> Callback) const override;

	// Timers

	virtual void ReadTimers(TFunctionRef<void(const ITimingProfilerTimerReader&)> Callback) const override;
	virtual const ITimingProfilerTimerReader& GetTimerReader() const override { return *this; }

	// Metadata

	virtual uint32 GetOriginalTimerIdFromMetadata(uint32 MetadataTimerId) const override; // also in ITimingProfilerTimerReader
	virtual TArrayView<const uint8> GetMetadata(uint32 MetadataTimerId) const override; // also in ITimingProfilerTimerReader
	virtual const FMetadataSpec* GetMetadataSpec(uint32 MetadataSpecId) const override;
	virtual void MakeTimerDisplayName(const FTimingProfilerTimer& InTimer, TStringBuilder<1024>& InOutDisplayName) const override;

	// GPU

	virtual bool HasGpuTiming() const override;
	virtual bool GetGpuTimelineIndex(uint32& OutTimelineIndex) const override; // backward compatibility
	virtual bool GetGpu2TimelineIndex(uint32& OutTimelineIndex) const override; // backward compatibility
	virtual void EnumerateGpuQueues(TFunctionRef<void(const FGpuQueueInfo&)> Callback) const override;
	virtual bool GetGpuQueueTimelineIndex(uint32 QueueId, uint32& OutTimelineIndex) const override;
	virtual void EnumerateGpuSignalFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuSignalFencesCallback Callback) const override;
	virtual void EnumerateGpuWaitFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuWaitFencesCallback Callback) const override;
	virtual void EnumerateGpuFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuFencesCallback Callback) const override;
	virtual void EnumerateResolvedGpuFences(uint32 QueueId, double StartTime, double EndTime, EnumerateResolvedGpuFencesCallback Callback) const override;

	// Verse

	virtual bool HasVerseTiming() const override;
	virtual bool GetVerseTimelineIndex(uint32& OutTimelineIndex) const override;

	// CPU

	virtual bool HasCpuTiming() const override;
	virtual bool GetCpuThreadTimelineIndex(uint32 ThreadId, uint32& OutTimelineIndex) const override;

	// Aggregation

	virtual ITable<FTimingProfilerAggregatedStats>* CreateAggregation(const FCreateAggregationParams& Params) const override;
	virtual ITimingProfilerButterfly* CreateButterfly(const FCreateButterflyParams& Params) const override;

	// Misc

	virtual double GetRatioOfThreadsToUse() const override { Session.ReadAccessCheck(); return RatioOfThreadsToUse; }

	//////////////////////////////////////////////////
	// ITimingProfilerTimerReader

	virtual uint32 GetTimerCount() const override;

	// The returned FTimingProfilerTimer* is only valid under the current read lock.
	virtual const FTimingProfilerTimer* GetTimer(uint32 TimerId) const override;

	//virtual uint32 GetOriginalTimerIdFromMetadata(uint32 MetadataTimerId) const override; // also in ITimingProfilerProvider
	//virtual TArrayView<const uint8> GetMetadata(uint32 MetadataTimerId) const override; // also in ITimingProfilerProvider

	//////////////////////////////////////////////////
	// IEditableTimingProfilerProvider

	// Timers

	virtual uint32 AddTimer(ETimingProfilerTimerType Type) override;
	virtual uint32 AddTimer(ETimingProfilerTimerType Type, FStringView Name) override;
	virtual uint32 AddTimer(ETimingProfilerTimerType Type, FStringView Name, const TCHAR* File, uint32 Line) override;

	virtual void SetTimerName(uint32 TimerId, FStringView Name) override;
	virtual void SetTimerNameAndLocation(uint32 TimerId, FStringView Name, const TCHAR* File, uint32 Line) override;
	virtual void SetTimerLocation(uint32 TimerId, const TCHAR* File, uint32 Line) override;

	// Metadata

	virtual uint32 AddMetadata(uint32 OriginalTimerId, TArray<uint8>&& Metadata) override;
	virtual void SetMetadata(uint32 MetadataTimerId, TArray<uint8>&& Metadata) override;
	virtual void SetMetadata(uint32 MetadataTimerId, TArray<uint8>&& Metadata, uint32 NewTimerId) override;
	virtual TArrayView<uint8> GetEditableMetadata(uint32 MetadataTimerId) override;
	virtual uint32 AddMetadataSpec(FMetadataSpec&& Metadata) override;
	virtual void SetMetadataSpec(uint32 TimerId, uint32 MetadataSpecId) override;

	// GPU

	virtual void AddGpuQueue(uint32 QueueId, uint8 GPU, uint8 Index, uint8 Type, const TCHAR* Name) override;
	virtual void AddGpuSignalFence(uint32 QueueId, const FGpuSignalFence& SignalFence) override;
	virtual void AddGpuWaitFence(uint32 QueueId, const FGpuWaitFence& WaitFence) override;
	virtual IEditableTimeline<FTimingProfilerEvent>* GetGpuQueueEditableTimeline(uint32 QueueId) override;
	virtual IEditableTimeline<FTimingProfilerEvent>* GetGpuQueueWorkEditableTimeline(uint32 QueueId) override;

	// Verse

	virtual IEditableTimeline<FTimingProfilerEvent>* GetVerseEditableTimeline() override;

	// CPU

	virtual IEditableTimeline<FTimingProfilerEvent>& GetCpuThreadEditableTimeline(uint32 ThreadId) override;

	// Misc

	virtual void SetRatioOfThreadsToUse(double InRatioOfThreadsToUse) override { Session.WriteAccessCheck(); RatioOfThreadsToUse = InRatioOfThreadsToUse; }
	virtual const ITimingProfilerProvider* GetReadProvider() const override { return this; }

	//////////////////////////////////////////////////

	TimelineInternal& EditGpuTimeline();
	TimelineInternal& EditGpu2Timeline();

private:
	FTimingProfilerTimer& AddTimerInternal(ETimingProfilerTimerType TimerType, FStringView Name, const TCHAR* File, uint32 Line);
	ITable<FTimingProfilerAggregatedStats>* CreateAggregationInternal(TArray<const ITimingProfilerTimeline*>& IncludedTimelines, const FCreateAggregationParams& Params) const;
	ITimingProfilerButterfly* CreateButterflyInternal(TArray<const ITimingProfilerTimeline*>& IncludedTimelines, const FCreateButterflyParams& Params) const;

	struct FMetadata
	{
		TArray<uint8> Payload;
		uint32 TimerId;
	};

	IAnalysisSession& Session;
	TArray<FMetadata> Metadatas;
	TArray<FMetadataSpec> MetadataSpecs;
	TMultiMap<uint32, uint32> MetadataSpecMap; // hash(FMetadataSpec) --> MetadataSpecId (index)
	TArray<FTimingProfilerTimer> Timers;
	TArray<TSharedRef<TimelineInternal>> Timelines;
	TTableLayout<FTimingProfilerAggregatedStats> AggregatedStatsTableLayout;
	double RatioOfThreadsToUse = 0.75;

	bool bHasData[uint32(ETimingProfilerTimerType::Count)];

	// GPU

	static const uint32 GpuTimelineIndex = 0;
	static const uint32 Gpu2TimelineIndex = 1;
	TArray<FGpuQueueInfo> GpuQueues;
	TArray<FGpuQueueData> GpuQueueData;
	TMap<uint32, uint32> GpuQueueIdToQueueIndexMap; // GPU Queue Id --> queue index, in GpuQueues

	// Verse

	static const uint32 VerseTimelineIndex = 2;

	// CPU

	TMap<uint32, uint32> CpuThreadTimelineIndexMap; // CPU Thread Id --> timeline index, in Timelines
};

} // namespace TraceServices
