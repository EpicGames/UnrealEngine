// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/ModuleService.h"
#include "ModuleServicePrivate.h"

#include "Features/IModularFeatures.h"
#include "HAL/LowLevelMemTracker.h"
#include "UObject/NameTypes.h"

LLM_DECLARE_TAG(Insights_TraceServices);

namespace TraceServices
{

const FName ModuleFeatureName("TraceModuleFeature");

FModuleService::FModuleService()
{
}

void FModuleService::Initialize()
{
	if (bIsInitialized)
	{
		return;
	}

	TArray<IModule*> Modules = IModularFeatures::Get().GetModularFeatureImplementations<IModule>(ModuleFeatureName);
	for (IModule* Module : Modules)
	{
		FModuleInfo ModuleInfo;
		Module->GetModuleInfo(ModuleInfo);
		ModulesMap.Add(ModuleInfo.Name, Module);
		if (Module->ShouldBeEnabledByDefault())
		{
			EnabledModules.Add(Module);
		}
	}

	bIsInitialized = true;
}

void FModuleService::GetAvailableModules(TArray<FModuleInfo>& OutModules)
{
	FScopeLock Lock(&CriticalSection);
	Initialize();
	OutModules.Empty(ModulesMap.Num());
	for (const auto& KV : ModulesMap)
	{
		IModule* Module = KV.Value;
		FModuleInfo& ModuleInfo = OutModules.AddDefaulted_GetRef();
		Module->GetModuleInfo(ModuleInfo);
	}
}

void FModuleService::GetAvailableModulesEx(TArray<FModuleInfoEx>& OutModules)
{
	FScopeLock Lock(&CriticalSection);
	Initialize();
	OutModules.Empty(ModulesMap.Num());
	for (const auto& KV : ModulesMap)
	{
		IModule* Module = KV.Value;
		FModuleInfoEx& ModuleInfoEx = OutModules.AddDefaulted_GetRef();
		Module->GetModuleInfo(ModuleInfoEx.Info);
		ModuleInfoEx.bIsEnabled = EnabledModules.Contains(Module);
	}
}

void FModuleService::GetEnabledModules(TArray<FModuleInfo>& OutModules)
{
	FScopeLock Lock(&CriticalSection);
	Initialize();
	OutModules.Empty(ModulesMap.Num());
	for (IModule* Module : EnabledModules)
	{
		FModuleInfo& ModuleInfo = OutModules.AddDefaulted_GetRef();
		Module->GetModuleInfo(ModuleInfo);
	}
}

void FModuleService::SetModuleEnabled(const FName& ModuleName, bool bEnabled)
{
	FScopeLock Lock(&CriticalSection);
	Initialize();
	IModule** FindIt = ModulesMap.Find(ModuleName);
	if (!FindIt)
	{
		return;
	}
	bool bWasEnabled = !!EnabledModules.Find(*FindIt);
	if (bEnabled == bWasEnabled)
	{
		return;
	}
	if (bEnabled)
	{
		EnabledModules.Add(*FindIt);
	}
	else
	{
		EnabledModules.Remove(*FindIt);
	}
}

void FModuleService::OnAnalysisBegin(IAnalysisSession& Session)
{
	LLM_SCOPE_BYTAG(Insights_TraceServices);
	FScopeLock Lock(&CriticalSection);
	Initialize();
	for (IModule* Module : EnabledModules)
	{
		Module->OnAnalysisBegin(Session);
	}
}

TArray<const TCHAR*> FModuleService::GetModuleLoggers(const FName& ModuleName)
{
	TArray<const TCHAR*> Loggers;

	FScopeLock Lock(&CriticalSection);
	Initialize();
	IModule* FindIt = ModulesMap.FindRef(ModuleName);
	if (FindIt)
	{
		FindIt->GetLoggers(Loggers);
	}
	return Loggers;
}

TSet<FName> FModuleService::GetEnabledModulesFromCommandLine(const TCHAR* CommandLine)
{
	TSet<FName> EnabledModulesFromCommandLine;

	if (!CommandLine)
	{
		return EnabledModulesFromCommandLine;
	}

	FScopeLock Lock(&CriticalSection);
	Initialize();
	for (const auto& KV : ModulesMap)
	{
		IModule* Module = KV.Value;
		const TCHAR* ModuleCommandLineArgument = Module->GetCommandLineArgument();
		if (ModuleCommandLineArgument && FParse::Param(CommandLine, ModuleCommandLineArgument))
		{
			EnabledModulesFromCommandLine.Add(KV.Key);
		}
	}
	return EnabledModulesFromCommandLine;
}

void FModuleService::GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
	FScopeLock Lock(&CriticalSection);
	Initialize();
	for (IModule* Module : EnabledModules)
	{
		Module->GenerateReports(Session, CmdLine, OutputDirectory);
	}
}

} // namespace TraceServices
