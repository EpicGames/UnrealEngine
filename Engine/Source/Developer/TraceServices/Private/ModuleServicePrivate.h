// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Misc/ScopeLock.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

class IAnalysisSession;

class FModuleService
	: public IModuleService
{
public:
	FModuleService();

	virtual void GetAvailableModules(TArray<FModuleInfo>& OutModules) override;
	virtual void GetAvailableModulesEx(TArray<FModuleInfoEx>& OutModules) override;
	virtual void GetEnabledModules(TArray<FModuleInfo>& OutModules) override;
	virtual void SetModuleEnabled(const FName& ModuleName, bool bEnabled) override;
	virtual void GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;

	void OnAnalysisBegin(IAnalysisSession& Session);

	TArray<const TCHAR*> GetModuleLoggers(const FName& ModuleName);

private:
	void Initialize();
	TSet<FName> GetEnabledModulesFromCommandLine(const TCHAR* CommandLine);

	mutable FCriticalSection CriticalSection;
	bool bIsInitialized = false;
	TSet<IModule*> EnabledModules;
	TMap<FName, IModule*> ModulesMap;
};

} // namespace TraceServices
