// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookProfilerModule.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/CookAnalysis.h"
#include "Model/CookProfilerProviderPrivate.h"

namespace TraceServices
{

void FCookProfilerModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName CookProfilerModuleName("TraceModule_CookProfiler");

	OutModuleInfo.Name = CookProfilerModuleName;
	OutModuleInfo.DisplayName = TEXT("Cook Profiling");
}

void FCookProfilerModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	TSharedPtr<FCookProfilerProvider> CookProfilerProvider = MakeShared<FCookProfilerProvider>(Session);
	Session.AddProvider(GetCookProfilerProviderName(), CookProfilerProvider, CookProfilerProvider);
	Session.AddAnalyzer(new FCookAnalyzer(Session, *CookProfilerProvider));
}

void FCookProfilerModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("CookProfiler"));
}

FName GetCookProfilerProviderName()
{
	static const FName Name("CookProfilerProvider");
	return Name;
}

const ICookProfilerProvider* ReadCookProfilerProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<ICookProfilerProvider>(GetCookProfilerProviderName());
}

IEditableCookProfilerProvider* EditCookProfilerProvider(IAnalysisSession& Session)
{
	return Session.EditProvider<IEditableCookProfilerProvider>(GetCookProfilerProviderName());
}

} // namespace TraceServices
