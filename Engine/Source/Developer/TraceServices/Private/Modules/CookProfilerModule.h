// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"

namespace TraceServices
{

class FCookProfilerModule
	: public IModule
{
public:
	virtual void GetModuleInfo(FModuleInfo& OutModuleInfo) override;
#if WITH_EDITOR
	virtual bool ShouldBeEnabledByDefault() const override { return false; }
#endif
	virtual void GetLoggers(TArray<const TCHAR*>& OutLoggers) override;
	virtual void OnAnalysisBegin(IAnalysisSession& Session) override;
};

} // namespace TraceServices
