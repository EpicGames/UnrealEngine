// Copyright Epic Games, Inc. All Rights Reserved.

#include "CountersModule.h"

#include "Analyzers/CountersTraceAnalysis.h"
#include "TraceServices/ModuleService.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Counters.h"

namespace TraceServices
{

void FCountersModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName CountersModuleName("TraceModule_Counters");

	OutModuleInfo.Name = CountersModuleName;
	OutModuleInfo.DisplayName = TEXT("Counters");
}

void FCountersModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	IEditableCounterProvider& EditableCounterProvider = EditCounterProvider(Session);
	Session.AddAnalyzer(new FCountersAnalyzer(Session, EditableCounterProvider));
}

void FCountersModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("Counters"));
}

} // namespace TraceServices
