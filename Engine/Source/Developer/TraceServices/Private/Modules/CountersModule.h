// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"

namespace TraceServices
{

class FCountersModule
	: public IModule
{
public:
	virtual void GetModuleInfo(FModuleInfo& OutModuleInfo) override;
	virtual void GetLoggers(TArray<const TCHAR*>& OutLoggers) override;
	virtual void OnAnalysisBegin(IAnalysisSession& Session) override;
	virtual void GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override {}
};

} // namespace TraceServices
