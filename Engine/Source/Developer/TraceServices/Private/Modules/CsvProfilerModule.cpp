// Copyright Epic Games, Inc. All Rights Reserved.

#include "CsvProfilerModule.h"

#include "Analyzers/CsvProfilerTraceAnalysis.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/Threads.h"

namespace TraceServices
{

void FCsvProfilerModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName CsvProfilerModuleName("TraceModule_CsvProfiler");

	OutModuleInfo.Name = CsvProfilerModuleName;
	OutModuleInfo.DisplayName = TEXT("CsvProfiler");
}

void FCsvProfilerModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	const IFrameProvider& FrameProvider = ReadFrameProvider(Session);
	const IThreadProvider& ThreadProvider = ReadThreadProvider(Session);
	IEditableCounterProvider& EditableCounterProvider = EditCounterProvider(Session);
	TSharedPtr<FCsvProfilerProvider> CsvProfilerProvider = MakeShared<FCsvProfilerProvider>(Session);
	Session.AddProvider(GetCsvProfilerProviderName(), CsvProfilerProvider);
	Session.AddAnalyzer(new FCsvProfilerAnalyzer(Session, *CsvProfilerProvider, EditableCounterProvider, FrameProvider, ThreadProvider));
}

void FCsvProfilerModule::GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
	const ICsvProfilerProvider* CsvProfilerProvider = ReadCsvProfilerProvider(Session);
	if (CsvProfilerProvider)
	{
		CsvProfilerProvider->EnumerateCaptures([CsvProfilerProvider, &OutputDirectory](const FCaptureInfo& CaptureInfo)
		{
			Table2Csv(CsvProfilerProvider->GetTable(CaptureInfo.Id), *(FString(OutputDirectory) / TEXT("CsvProfiler") / CaptureInfo.Filename));
		});
	}
}

FName GetCsvProfilerProviderName()
{
	static const FName Name("CsvProfilerProvider");
	return Name;
}

const ICsvProfilerProvider* ReadCsvProfilerProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<ICsvProfilerProvider>(GetCsvProfilerProviderName());
}

} // namespace TraceServices
