// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiagnosticsModule.h"

#include "Analyzers/DiagnosticsAnalysis.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

void FDiagnosticsModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName DiagnosticsModuleName("TraceModule_Diagnostics");

	OutModuleInfo.Name = DiagnosticsModuleName;
	OutModuleInfo.DisplayName = TEXT("Diagnostics");
}

void FDiagnosticsModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	TSharedPtr<FDiagnosticsProvider> DiagnosticsProvider = MakeShared<FDiagnosticsProvider>(Session);
	Session.AddProvider(GetDiagnosticsProviderName(), DiagnosticsProvider);

	Session.AddAnalyzer(new FDiagnosticsAnalyzer(Session, DiagnosticsProvider.Get()));
}

void FDiagnosticsModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("Diagnostics"));
}

} // namespace TraceServices
