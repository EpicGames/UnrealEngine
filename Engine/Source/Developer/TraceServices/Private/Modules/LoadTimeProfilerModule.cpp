// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadTimeProfilerModule.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/LoadTimeTraceAnalysis.h"
#include "Analyzers/PlatformFileTraceAnalysis.h"
#include "HAL/FileManager.h"
#include "Model/FileActivity.h"
#include "TraceServices/Model/Bookmarks.h"

namespace TraceServices
{

void BookmarksToCsv(const IBookmarkProvider& BookmarkProvider, const TCHAR* Filename, double CaptureStartTime, double CaptureEndTime)
{
	TSharedPtr<FArchive> BookmarksOutputFile = MakeShareable(IFileManager::Get().CreateFileWriter(Filename));
	const char* Header = "Name, Timestamp\n";
	BookmarksOutputFile->Serialize((void*)Header, strlen(Header));
	BookmarkProvider.EnumerateBookmarks(CaptureStartTime, CaptureEndTime, [&BookmarksOutputFile](const FBookmark& Bookmark)
	{
		FString Line = FString::Printf(TEXT("%s,%.4f\n"), Bookmark.Text, Bookmark.Time);
		auto AnsiLine = StringCast<ANSICHAR>(*Line);
		BookmarksOutputFile->Serialize((void*)AnsiLine.Get(), AnsiLine.Length());
	});
}

void FLoadTimeProfilerModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName LoadTimeProfilerModuleName("TraceModule_LoadTimeProfiler");

	OutModuleInfo.Name = LoadTimeProfilerModuleName;
	OutModuleInfo.DisplayName = TEXT("Asset Loading");
}

void FLoadTimeProfilerModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	TSharedPtr<FLoadTimeProfilerProvider> LoadTimeProfilerProvider = MakeShared<FLoadTimeProfilerProvider>(Session, EditCounterProvider(Session));
	Session.AddProvider(GetLoadTimeProfilerProviderName(), LoadTimeProfilerProvider);
	Session.AddAnalyzer(new FAsyncLoadingTraceAnalyzer(Session, *LoadTimeProfilerProvider));
	TSharedPtr<FFileActivityProvider> FileActivityProvider = MakeShared<FFileActivityProvider>(Session);
	Session.AddProvider(GetFileActivityProviderName(), FileActivityProvider);
	Session.AddAnalyzer(new FPlatformFileTraceAnalyzer(Session, *FileActivityProvider));
}

void FLoadTimeProfilerModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("LoadTime"));
	OutLoggers.Add(TEXT("PlatformFile"));
}

void FLoadTimeProfilerModule::GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
	double CaptureStartTime = -DBL_MAX;
	double CaptureEndTime = DBL_MAX;
	const IBookmarkProvider& BookmarkProvider = ReadBookmarkProvider(Session);
	FString BeginCaptureBookmarkName;
	FParse::Value(CmdLine, TEXT("-BeginCaptureBookmark="), BeginCaptureBookmarkName);
	FString EndCaptureBookmarkName;
	FParse::Value(CmdLine, TEXT("-EndCaptureBookmark="), EndCaptureBookmarkName);


	BookmarkProvider.EnumerateBookmarks(0.0, DBL_MAX, [BeginCaptureBookmarkName, EndCaptureBookmarkName, &CaptureStartTime, &CaptureEndTime](const FBookmark& Bookmark)
	{
		if (CaptureStartTime == -DBL_MAX && !BeginCaptureBookmarkName.IsEmpty() && BeginCaptureBookmarkName == Bookmark.Text)
		{
			CaptureStartTime = Bookmark.Time;
		}
		if (CaptureEndTime == DBL_MAX && !EndCaptureBookmarkName.IsEmpty() && EndCaptureBookmarkName == Bookmark.Text)
		{
			CaptureEndTime = Bookmark.Time;
		}
	});
	if (CaptureStartTime == -DBL_MAX)
	{
		CaptureStartTime = 0.0;
	}
	if (CaptureEndTime == DBL_MAX)
	{
		CaptureEndTime = Session.GetDurationSeconds();
	}

	const ILoadTimeProfilerProvider* LoadTimeProfiler = ReadLoadTimeProfilerProvider(Session);
	FString ReportDirectory = FString(OutputDirectory) / TEXT("LoadTimeProfiler");
	if (LoadTimeProfiler)
	{
		TUniquePtr<ITable<FPackagesTableRow>> PackagesTable(LoadTimeProfiler->CreatePackageDetailsTable(CaptureStartTime, CaptureEndTime));
		Table2Csv(*PackagesTable.Get(), *(ReportDirectory / TEXT("Packages.csv")));
		TUniquePtr<ITable<FExportsTableRow>> ExportsTable(LoadTimeProfiler->CreateExportDetailsTable(CaptureStartTime, CaptureEndTime));
		Table2Csv(*ExportsTable.Get(), *(ReportDirectory / TEXT("Exports.csv")));
		TUniquePtr<ITable<FRequestsTableRow>> RequestsTable(LoadTimeProfiler->CreateRequestsTable(CaptureStartTime, CaptureEndTime));
		Table2Csv(*RequestsTable.Get(), *(ReportDirectory / TEXT("Requests.csv")));
	}
	const IFileActivityProvider* FileActivityProvider = ReadFileActivityProvider(Session);
	if (FileActivityProvider)
	{
		Table2Csv(FileActivityProvider->GetFileActivityTable(), *(FString(ReportDirectory) / TEXT("FileActivity.csv")));
	}
	if (CaptureStartTime > 0.0 || CaptureEndTime < Session.GetDurationSeconds())
	{
		TSharedPtr<FArchive> CaptureSummaryOutputFile = MakeShareable(IFileManager::Get().CreateFileWriter(*(ReportDirectory / TEXT("CaptureSummary.txt"))));
		FString SummaryLine = FString::Printf(TEXT("Capture start: %f\r\nCapture end: %f\r\nCapture duration: %f"), CaptureStartTime, CaptureEndTime, CaptureEndTime - CaptureStartTime);
		CaptureSummaryOutputFile->Serialize(TCHAR_TO_ANSI(*SummaryLine), SummaryLine.Len());
	}
	BookmarksToCsv(BookmarkProvider, *(FString(ReportDirectory) / TEXT("Bookmarks.csv")), CaptureStartTime, CaptureEndTime);
}

FName GetLoadTimeProfilerProviderName()
{
	static const FName Name("LoadTimeProfilerProvider");
	return Name;
}

const ILoadTimeProfilerProvider* ReadLoadTimeProfilerProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<ILoadTimeProfilerProvider>(GetLoadTimeProfilerProviderName());
}

FName GetFileActivityProviderName()
{
	static const FName Name("FileActivityProvider");
	return Name;
}

const IFileActivityProvider* ReadFileActivityProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IFileActivityProvider>(GetFileActivityProviderName());
}

} // namespace TraceServices
