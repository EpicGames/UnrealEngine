// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryModule.h"

#include "Analyzers/AllocationsAnalysis.h"
#include "Analyzers/CallstacksAnalysis.h"
#include "Analyzers/MemoryAnalysis.h"
#include "Analyzers/MetadataAnalysis.h"
#include "Analyzers/ModuleAnalysis.h"
#include "Model/AllocationsProvider.h"
#include "Model/CallstacksProvider.h"
#include "Model/MetadataProvider.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

void FMemoryModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName MemoryModuleName("TraceModule_Memory");

	OutModuleInfo.Name = MemoryModuleName;
	OutModuleInfo.DisplayName = TEXT("Memory");
}

void FMemoryModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	// LLM Tag Stats
	TSharedPtr<FMemoryProvider> MemoryProvider = MakeShared<FMemoryProvider>(Session);
	Session.AddProvider(GetMemoryProviderName(), MemoryProvider);
	Session.AddAnalyzer(new FMemoryAnalyzer(Session, MemoryProvider.Get()));

	// Module
	// The Module provider is created and registered by FModuleAnalyzer when the ModuleInit event is detected.
	Session.AddAnalyzer(new FModuleAnalyzer(Session));

	// Callstack
	TSharedPtr<FCallstacksProvider> CallstacksProvider = MakeShared<FCallstacksProvider>(Session);
	Session.AddProvider(GetCallstacksProviderName(), CallstacksProvider);
	Session.AddAnalyzer(new FCallstacksAnalyzer(Session, CallstacksProvider.Get()));

	// Metadata
	TSharedPtr<FMetadataProvider> MetadataProvider = MakeShared<FMetadataProvider>(Session);
	Session.AddProvider(GetMetadataProviderName(), MetadataProvider, MetadataProvider);
	Session.AddAnalyzer(new FMetadataAnalysis(Session, MetadataProvider.Get()));

	// Allocations
	TSharedPtr<FAllocationsProvider> AllocationsProvider = MakeShared<FAllocationsProvider>(Session, *MetadataProvider);
	Session.AddProvider(GetAllocationsProviderName(), AllocationsProvider, AllocationsProvider);
	Session.AddAnalyzer(new FAllocationsAnalyzer(Session, *AllocationsProvider, *MetadataProvider));
}

void FMemoryModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("Memory"));
}

} // namespace TraceServices
