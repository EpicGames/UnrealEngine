// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetProfilerModule.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/NetTraceAnalyzer.h"
#include "Model/NetProfilerProvider.h"

namespace TraceServices
{

void FNetProfilerModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName ModuleName("TraceModule_NetProfiler");

	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("NetProfiler");
}

void FNetProfilerModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	TSharedPtr<FNetProfilerProvider> NetProfilerProvider = MakeShared<FNetProfilerProvider>(Session);
	Session.AddProvider(GetNetProfilerProviderName(), NetProfilerProvider);

	Session.AddAnalyzer(new FNetTraceAnalyzer(Session, *NetProfilerProvider));
}

} // namespace TraceServices
