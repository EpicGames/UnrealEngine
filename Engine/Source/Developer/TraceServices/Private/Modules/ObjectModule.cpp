// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectModule.h"

#include "Analyzers/ObjectTraceAnalysis.h"
#include "Model/ObjectProviderPrivate.h"

namespace TraceServices
{

void FObjectModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName ObjectModuleName("TraceModule_Object");

	OutModuleInfo.Name = ObjectModuleName;
	OutModuleInfo.DisplayName = TEXT("Object");
}

void FObjectModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	TSharedPtr<FObjectProvider> ObjectProvider = MakeShared<FObjectProvider>(Session);
	Session.AddProvider(GetObjectProviderName(), ObjectProvider, ObjectProvider);
	IEditableObjectProvider* EditableObjectProvider = EditObjectProvider(Session);
	if (EditableObjectProvider)
	{
		Session.AddAnalyzer(new FObjectAnalyzer(Session, *EditableObjectProvider));
	}
}

FName GetObjectProviderName()
{
	static const FName Name("ObjectProvider");
	return Name;
}

const IObjectProvider* ReadObjectProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<IObjectProvider>(GetObjectProviderName());
}

IEditableObjectProvider* EditObjectProvider(IAnalysisSession& Session)
{
	return Session.EditProvider<IEditableObjectProvider>(GetObjectProviderName());
}

} // namespace TraceServices
