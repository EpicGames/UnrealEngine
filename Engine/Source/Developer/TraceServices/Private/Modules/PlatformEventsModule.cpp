// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformEventsModule.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/PlatformEventTraceAnalysis.h"
#include "Model/ContextSwitchesPrivate.h"
#include "Model/StackSamplesPrivate.h"

namespace TraceServices
{

void FPlatformEventsModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName PlatformEventsModuleName("TraceModule_PlatformEvents");

	OutModuleInfo.Name = PlatformEventsModuleName;
	OutModuleInfo.DisplayName = TEXT("PlatformEvents");
}

void FPlatformEventsModule::OnAnalysisBegin(IAnalysisSession& InSession)
{
	FAnalysisSession& Session = static_cast<FAnalysisSession&>(InSession);

	TSharedPtr<FContextSwitchesProvider> ContextSwitchesProvider = MakeShared<FContextSwitchesProvider>(Session);
	TSharedPtr<FStackSamplesProvider> StackSamplesProvider = MakeShared<FStackSamplesProvider>(Session);

	Session.AddProvider(GetContextSwitchesProviderName(), ContextSwitchesProvider);
	Session.AddProvider(GetStackSamplesProviderName(), StackSamplesProvider);
	Session.AddAnalyzer(new FPlatformEventTraceAnalyzer(Session, *ContextSwitchesProvider, *StackSamplesProvider));
}

} // namespace TraceServices
