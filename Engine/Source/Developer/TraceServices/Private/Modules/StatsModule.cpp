// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsModule.h"

#include "Analyzers/StatsTraceAnalysis.h"
#include "TraceServices/Model/Counters.h"

namespace TraceServices
{

void FStatsModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName StatsModuleName("TraceModule_Stats");

	OutModuleInfo.Name = StatsModuleName;
	OutModuleInfo.DisplayName = TEXT("Stats");
}

void FStatsModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	IEditableCounterProvider& EditableCounterProvider = EditCounterProvider(Session);
	Session.AddAnalyzer(new FStatsAnalyzer(Session, EditableCounterProvider));
}

} // namespace TraceServices
