// Copyright Epic Games, Inc. All Rights Reserved.

#include "TasksModule.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/TasksAnalysis.h"
#include "Model/TasksProfilerPrivate.h"
#include "TraceServices/ModuleService.h"

namespace TraceServices
{

void FTasksModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName TasksModuleName("TraceModule_TasksProfiler");

	OutModuleInfo.Name = TasksModuleName;
	OutModuleInfo.DisplayName = TEXT("TasksProfiler");
}

void FTasksModule::OnAnalysisBegin(IAnalysisSession& Session)
{
	TSharedPtr<FTasksProvider> TasksProvider = MakeShared<FTasksProvider>(Session);
	Session.AddProvider(GetTaskProviderName(), TasksProvider);
	Session.AddAnalyzer(new FTasksAnalyzer(Session, *TasksProvider));
}

FName GetTaskProviderName()
{
	static const FName Name("TasksProvider");
	return Name;
}

const ITasksProvider* ReadTasksProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<ITasksProvider>(GetTaskProviderName());
}

} // namespace TraceServices
