// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingProfilerModule.h"

#include "AnalysisServicePrivate.h"
#include "Analyzers/CpuProfilerTraceAnalysis.h"
#include "Analyzers/GpuProfilerTraceAnalysis.h"
#include "Analyzers/OldGpuProfilerTraceAnalysis.h"
#include "Analyzers/VerseTraceAnalysis.h"
#include "Model/CountersPrivate.h"
#include "Model/ThreadsPrivate.h"
#include "Model/TimingProfilerPrivate.h"

namespace TraceServices
{

void FTimingProfilerModule::GetModuleInfo(FModuleInfo& OutModuleInfo)
{
	static const FName TimingProfilerModuleName("TraceModule_TimingProfiler");

	OutModuleInfo.Name = TimingProfilerModuleName;
	OutModuleInfo.DisplayName = TEXT("Timing");
}

void FTimingProfilerModule::OnAnalysisBegin(IAnalysisSession& InSession)
{
	FAnalysisSession& Session = static_cast<FAnalysisSession&>(InSession);

	IEditableThreadProvider& EditableThreadProvider = EditThreadProvider(Session);

	// Add the TimingProfiler provider.
	TSharedPtr<FTimingProfilerProvider> TimingProfilerProvider = MakeShared<FTimingProfilerProvider>(Session);
	Session.AddProvider(GetTimingProfilerProviderName(), TimingProfilerProvider, TimingProfilerProvider);

	// Add the CPU Profiler analyzer.
	Session.AddAnalyzer(new FCpuProfilerAnalyzer(Session, *TimingProfilerProvider, EditableThreadProvider));

	// Add the old GPU Profiler analyzer (for backward compatibility with old traces).
	Session.AddAnalyzer(new FOldGpuProfilerAnalyzer(Session, *TimingProfilerProvider));

	// Add the new GPU Profiler analyzer.
	Session.AddAnalyzer(new FGpuProfilerAnalyzer(Session, *TimingProfilerProvider, EditCounterProvider(Session)));

	// Add the new Verse Sampling analyzer.
	Session.AddAnalyzer(new FVerseAnalyzer(Session, *TimingProfilerProvider));
}

void FTimingProfilerModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("CpuProfiler"));
	OutLoggers.Add(TEXT("GpuProfiler"));
}

FName GetTimingProfilerProviderName()
{
	static const FName Name("TimingProfilerProvider");
	return Name;
}

const ITimingProfilerProvider* ReadTimingProfilerProvider(const IAnalysisSession& Session)
{
	return Session.ReadProvider<ITimingProfilerProvider>(GetTimingProfilerProviderName());
}

IEditableTimingProfilerProvider* EditTimingProfilerProvider(IAnalysisSession& Session)
{
	return Session.EditProvider<IEditableTimingProfilerProvider>(GetTimingProfilerProviderName());
}

} // namespace TraceServices
