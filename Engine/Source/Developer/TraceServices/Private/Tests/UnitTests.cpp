// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "Common/PagedArray.h"
#include "Misc/AutomationTest.h"
#include "TraceServices/Containers/SlabAllocator.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPagedArrayFilteringTest, "System.Insights.Analysis.PagedArrayFiltering", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool operator!=(const FInt32Interval& lhs, const FInt32Interval& rhs)
{
	return lhs.Min != rhs.Min || lhs.Max != rhs.Max;
}

bool FPagedArrayFilteringTest::RunTest(const FString& Parameters)
{
	using namespace TraceServices;

	TArray<int32> Integers{ 0,1,2,3,4,5,6,7,8,9,10 };
	TArray<int32> Integer{ 1 };

	//first element >= Value
	TestEqual(TEXT("First index of element found in range"), Algo::LowerBound(Integers, 4), 4);
	TestEqual(TEXT("First index of element not in range"), Algo::LowerBound(Integers, 100), 11);
	//first element > Value
	TestEqual(TEXT("Upper bound value found in range"), Algo::UpperBound(Integers, 4), 5);
	TestEqual(TEXT("Upper bound value not found in range"), Algo::UpperBound(Integers, 100), 11);

	TestEqual(TEXT("Upper bound value found in single-item range"), Algo::UpperBound(Integer, 10), 1);

	FSlabAllocator alloc(32 << 20);
	int32 PageSize = 4;

	struct FTimeRegion
	{
		double BeginTime;
		double EndTime;
	};

	TPagedArray<FTimeRegion> EmptyLane(alloc, PageSize);

	TPagedArray<FTimeRegion> OneItemLane(alloc, PageSize);
	OneItemLane.EmplaceBack(FTimeRegion{ 1.0, 2.0 });

	TPagedArray<FTimeRegion> TestLane(alloc, PageSize);
	TestLane.EmplaceBack(FTimeRegion{ 1.0, 2.0 });
	TestLane.EmplaceBack(FTimeRegion{ 3.0, 4.0 });
	TestLane.EmplaceBack(FTimeRegion{ 5.0, 6.0 });
	TestLane.EmplaceBack(FTimeRegion{ 7.0, 8.0 });
	TestLane.EmplaceBack(FTimeRegion{ 9.0, 10.0 });
	TestLane.EmplaceBack(FTimeRegion{ 11.0, 12.0 });
	TestLane.EmplaceBack(FTimeRegion{ 13.0, 14.0 });

	FInt32Interval Result;
	auto BeginProj = [](const FTimeRegion& r) { return r.BeginTime; };
	auto EndProj = [](const FTimeRegion& r) { return r.EndTime; };

	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(EmptyLane, -10.0, 0.0, BeginProj, EndProj);
	TestEqual(TEXT("No overlaps for empty range"), Result, FInt32Interval{ -1,-1 });

	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(OneItemLane, -10.0, 10.0, BeginProj, EndProj);
	TestEqual(TEXT("Overlap is interval is larger than element-range"), Result, { 0,0 });

	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(TestLane, -10.0, 0.0, BeginProj, EndProj);
	TestEqual(TEXT("No overlaps for interval before element-range"), Result, { -1,-1 });

	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(TestLane, 100.0, 200.0, BeginProj, EndProj);
	TestEqual(TEXT("No overlaps for interval after element-range"), Result, { -1,-1 });

	//partially overlapping begin
	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(TestLane, 0, 5, BeginProj, EndProj);
	TestEqual(TEXT("Overlaps for interval earlier to inside element-range"), Result, { 0,2 });

	// full inside
	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(TestLane, 4, 8, BeginProj, EndProj);
	TestEqual(TEXT("Overlaps for interval inside element-range"), Result, { 2,3 });

	//partially overlapping end
	Result = GetElementRangeOverlappingGivenRange<FTimeRegion>(TestLane, 11.3, 100, BeginProj, EndProj);
	TestEqual(TEXT("Overlaps for interval inside to after element-range"), Result, { 5,6 });

	return !HasAnyErrors();
}