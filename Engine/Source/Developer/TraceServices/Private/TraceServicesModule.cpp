// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceServices/ITraceServicesModule.h"

#include "AnalysisServicePrivate.h"
#include "Features/IModularFeatures.h"
#include "HAL/LowLevelMemTracker.h"
#include "ModuleServicePrivate.h"
#include "Modules/CookProfilerModule.h"
#include "Modules/CountersModule.h"
#include "Modules/CsvProfilerModule.h"
#include "Modules/DiagnosticsModule.h"
#include "Modules/LoadTimeProfilerModule.h"
#include "Modules/MemoryModule.h"
#include "Modules/ModuleManager.h"
#include "Modules/NetProfilerModule.h"
#include "Modules/ObjectModule.h"
#include "Modules/PlatformEventsModule.h"
#include "Modules/StatsModule.h"
#include "Modules/TasksModule.h"
#include "Modules/TimingProfilerModule.h"
#include "Stats/StatsTrace.h"

LLM_DEFINE_TAG(Insights_TraceServices, NAME_None, TEXT("Insights"));

class FTraceServicesModule
	: public ITraceServicesModule
{
public:
	virtual TSharedPtr<TraceServices::IAnalysisService> GetAnalysisService() override;
	virtual TSharedPtr<TraceServices::IModuleService> GetModuleService() override;
	virtual TSharedPtr<TraceServices::IAnalysisService> CreateAnalysisService() override;
	virtual TSharedPtr<TraceServices::IModuleService> CreateModuleService() override;

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<TraceServices::FAnalysisService> AnalysisService;
	TSharedPtr<TraceServices::FModuleService> ModuleService;

	TraceServices::FTimingProfilerModule TimingProfilerModule;
	TraceServices::FLoadTimeProfilerModule LoadTimeProfilerModule;
	TraceServices::FStatsModule StatsModule;
	TraceServices::FCsvProfilerModule CsvProfilerModule;
	TraceServices::FCountersModule CountersModule;
	TraceServices::FNetProfilerModule NetProfilerModule;
	TraceServices::FMemoryModule MemoryModule;
	TraceServices::FDiagnosticsModule DiagnosticsModule;
	TraceServices::FPlatformEventsModule PlatformEventsModule;
	TraceServices::FTasksModule TasksModule;
	TraceServices::FCookProfilerModule CookProfilingModule;
	TraceServices::FObjectModule ObjectModule;
};

TSharedPtr<TraceServices::IAnalysisService> FTraceServicesModule::GetAnalysisService()
{
	if (!AnalysisService.IsValid())
	{
		GetModuleService();

		LLM_SCOPE_BYTAG(Insights_TraceServices);
		AnalysisService = MakeShared<TraceServices::FAnalysisService>(*ModuleService.Get());
	}
	return AnalysisService;
}

TSharedPtr<TraceServices::IModuleService> FTraceServicesModule::GetModuleService()
{
	if (!ModuleService.IsValid())
	{
		LLM_SCOPE_BYTAG(Insights_TraceServices);
		ModuleService = MakeShared<TraceServices::FModuleService>();
	}
	return ModuleService;
}

TSharedPtr<TraceServices::IAnalysisService> FTraceServicesModule::CreateAnalysisService()
{
	checkf(!AnalysisService.IsValid(), TEXT("A AnalysisService already exists."));
	GetModuleService();

	{
		LLM_SCOPE_BYTAG(Insights_TraceServices);
		AnalysisService = MakeShared<TraceServices::FAnalysisService>(*ModuleService.Get());
	}

	return AnalysisService;
}

TSharedPtr<TraceServices::IModuleService> FTraceServicesModule::CreateModuleService()
{
	checkf(!ModuleService.IsValid(), TEXT("A ModuleService already exists."));

	{
		LLM_SCOPE_BYTAG(Insights_TraceServices);
		ModuleService = MakeShared<TraceServices::FModuleService>();
	}

	return ModuleService;
}

void FTraceServicesModule::StartupModule()
{
	LLM_SCOPE_BYTAG(Insights_TraceServices);

	// Load the analysis module.
	IModuleInterface& TraceAnalysisModule = FModuleManager::LoadModuleChecked<IModuleInterface>("TraceAnalysis");

	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TimingProfilerModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &CsvProfilerModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &CountersModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &NetProfilerModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &DiagnosticsModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &PlatformEventsModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &StatsModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &LoadTimeProfilerModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &MemoryModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TasksModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &CookProfilingModule);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &ObjectModule);
}

void FTraceServicesModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(Insights_TraceServices);

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &ObjectModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &CookProfilingModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TasksModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &MemoryModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &LoadTimeProfilerModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &StatsModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &PlatformEventsModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &DiagnosticsModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &NetProfilerModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &CountersModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &CsvProfilerModule);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TimingProfilerModule);

	AnalysisService.Reset();
	ModuleService.Reset();
}

IMPLEMENT_MODULE(FTraceServicesModule, TraceServices)
