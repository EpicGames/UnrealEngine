// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisCache.h"
#include "Containers/Array.h"
#include "Memory/SharedBuffer.h"

namespace TraceServices
{
	/**
	 * A typed paged array allowing persistent storage of the data. User defines page size (in number of elements)
	 * which defines the size of the page. The page is then mapped to a number of cache blocks. The blocks are always
	 * loaded into a coherent buffer.
	 *   +-------------------------------------------------------------------+
	 *	 |  Page                                                             |
	 *	 +-------------------------------------------------------------------+
	 *	 |                   |                    |                  |
	 *	 |                   |                    |                  |
	 *	 v                   v                    v                  v
	 *	 +------------------------------------------------------------------------------+
	 *	 |     Block n       |     Block n+1      |     Block n+2    |     Block n+3    |
	 *	 +------------------------------------------------------------------------------+
	 *
	 * Depending on the page size there could be some blocks that are only partially used.
	 *
	 * The container supports unloading pages for extremely large datasets by defining the number of resident pages.
	 * Once the limit is reached pages are unloaded. Pages are lazily loaded on access.
	 * 
	 */
	template<typename InItemType, unsigned int PageSize>
	class TCachedPagedArray
	{
	public:
		typedef InItemType ItemType;

		TCachedPagedArray(const TCHAR* InCacheId, IAnalysisCache& InCache, uint32 InResidentPages = ~0)
			: Cache(InCache)
			, CacheId(0)
			, ResidentPageCount(InResidentPages)
			, ElementCount(nullptr)
		{
			CacheId = Cache.GetCacheId(InCacheId);
			const FMutableMemoryView UserData = Cache.GetUserData(CacheId);
			check(UserData.GetData() && UserData.GetSize() >= sizeof(uint64));
			ElementCount = (uint64*) UserData.GetData();
			
			Pages.InsertZeroed(0, NumPages());
			// Load all pages if requested
			if (ResidentPageCount == ~0)
			{
				const uint32 PageCount = NumPages();
				for(uint32 PageIndex = 0; PageIndex < PageCount; ++PageIndex)
				{
					GetPage(PageIndex); //todo: Load all pages into one buffer!
				}
			}
		}

		template <typename... ArgsType>
		ItemType& EmplaceBack(ArgsType&&... Args)
		{
			ItemType* Page = GetPageForNextItem();
			const uint32 IndexInPage = *ElementCount % PageSize;
			
			ItemType* ItemPtr = Page + IndexInPage;
			++(*ElementCount);
			new (ItemPtr) ItemType(Forward<ArgsType>(Args)...);
			return *ItemPtr;
		}

		ItemType& operator[](uint64 Index)
		{
			check(Index < *ElementCount);
			const uint32 PageIndex = static_cast<uint32>(Index / PageSize);
			const uint32 IndexInPage = static_cast<uint32>(Index % PageSize);
			ItemType* Page = Pages[PageIndex];
			if (!Page)
			{
				Page = GetPage(PageIndex);
			}
			return *(Page + IndexInPage);
		}

		uint32 NumPages() const
		{
			return static_cast<uint32>((*ElementCount + PageSize - 1) / PageSize);
		}

		uint64 Num() const
		{
			return *ElementCount;
		}
	
	private:
		constexpr static uint32 PageSizeBytes = sizeof(ItemType) * PageSize;
		constexpr static uint32 BlocksPerPage = (PageSizeBytes + IAnalysisCache::BlockSizeBytes - 1) / IAnalysisCache::BlockSizeBytes;
		constexpr static uint32 WastePerPage = (BlocksPerPage * IAnalysisCache::BlockSizeBytes) - PageSizeBytes;

		inline ItemType* GetPageForNextItem()
		{
			if (*ElementCount % PageSize == 0)
			{
				FSharedBuffer& NewPage = ResidentPages.Emplace_GetRef(Cache.CreateBlocks(CacheId, BlocksPerPage));
				Pages.Push((ItemType*)NewPage.GetData());
				EvictPages();
			}
			return Pages.Last();
		}

		void GetItemsFromPage(uint64 PageIndex, ItemType** OutFirstItem, ItemType** OutLastItem) const
		{
			ItemType* Page = GetPage(PageIndex);
			uint64 Count = PageIndex == Pages.Num() ? (*ElementCount % PageSize) : PageSize;
			*OutFirstItem = Page;
			*OutLastItem = Page + Count - 1;
		}
		
		ItemType* GetPage(uint32 InPageIndex)
		{
			ItemType* Page = Pages[InPageIndex];
			if (!Page)
			{
				const uint32 BlockIndex = InPageIndex * BlocksPerPage;
				const FSharedBuffer& Block = ResidentPages.Emplace_GetRef(Cache.GetBlocks(CacheId, BlockIndex, BlocksPerPage));
				Page = Pages[InPageIndex] = (ItemType*)Block.GetData();
				EvictPages();
			}
			return Page;
		}

		void EvictPages()
		{
			if (uint32(ResidentPages.Num()) > ResidentPageCount)
			{
				// todo: Find better metric of evicting pages
				// Remove the oldest page
				const int32 Index = Pages.Find((ItemType*)ResidentPages[0].GetData());
				check(Index!=INDEX_NONE);
				Pages[Index] = nullptr;
				ResidentPages.RemoveAt(0); 
			}
		}

		IAnalysisCache& Cache;
		TArray<ItemType*> Pages;
		TArray<FSharedBuffer> ResidentPages;
		uint32 CacheId;
		uint32 ResidentPageCount;
		uint64* ElementCount;
	};
}
