// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/UnrealMemory.h"
#include "Memory/SharedBuffer.h"
#include "Templates/Tuple.h"
#include "TraceServices/Model/AnalysisCache.h"

namespace TraceServices
{
	
/**
 * A deduplicating persistent string store. Allows users to store a string in a session and retrieve it at a given
 * offset in consecutive sessions. The store uses analysis cache to store the strings. The store is not thread safe.
 */
template<typename CharType>
class TCachedStringStore
{
public:
	typedef TStringView<CharType> StringViewType;

	/**
	 * Create a persistent string store.
	 * @param InCacheIdentifier A unique string identifying this store.
	 * @param InCache Cache to use
	 */
	TCachedStringStore(const TCHAR* InCacheIdentifier, IAnalysisCache& InCache)
		: Cache(InCache)
		, CacheIndex(0)
	{
		CacheIndex = Cache.GetCacheId(InCacheIdentifier, ECacheFlags_NoGlobalCaching);
		const FMutableMemoryView UserData = Cache.GetUserData(CacheIndex);
		check(UserData.GetData() && UserData.GetSize() >= sizeof(FPersistentState));
		State = (FPersistentState*)UserData.GetData();
		
		// If the block was just created it will be zeroed.
		if (State->BlockCount == 0)
		{
			check(State->BufferLeftBytes == 0);
			BufferCursor = nullptr; 
		}
		// otherwise load the saved blocks and build acceleration structures.
		else
		{
			check(State->BufferLeftBytes <= BlockSize);
			for (uint32 BlockIndex = 0; BlockIndex < State->BlockCount; ++BlockIndex)
			{
				FSharedBuffer& Block = Blocks.Add_GetRef(Cache.GetBlocks(CacheIndex, BlockIndex, 1));
				check(!Block.IsNull());
				CharType* Cursor = (CharType*)Block.GetData();
				CharType* BlockEnd = (CharType*) ((uint8*) Block.GetData() + BlockSize);
				
				while (Cursor < BlockEnd && TCString<CharType>::Strlen(Cursor) > 0)
				{
					const int32 Length = TCString<CharType>::Strlen(Cursor);
					if (Length <= 0)
					{
						continue;
					}
					const uint32 Hash = GetTypeHash(StringViewType(Cursor));
					const uint64 Offset = BlockIndex * BlockSize + ((const uint8*)Cursor - (const uint8*)Block.GetData());
					StoredStrings.Add(Hash, TTuple<uint64,const CharType*>(Offset, Cursor));
					Cursor += Length + 1;
				}
			}
			check(Blocks.Num() > 0); // Should always have at least one Block
			BufferCursor = (CharType*) ((uint8*)Blocks.Last().GetData() + (BlockSize - State->BufferLeftBytes));
		}

		
		check(Blocks.Num() == State->BlockCount);
	}

	~TCachedStringStore()
	{
		check(Blocks.Num() == State->BlockCount);
	}

	/**
	 * Resolve an offset to a c string.
	 * @param OffsetBytes Offset in bytes
	 * @return The string at the offset or null if the offset was invalid.
	 */
	const CharType* GetStringAtOffset(uint64 OffsetBytes) const
	{
		const int32 BlockIndex = static_cast<int32>(OffsetBytes / BlockSize);
		const uint32 OffsetInBlockBytes = static_cast<uint32>(OffsetBytes % BlockSize);
		if (BlockIndex >= Blocks.Num())
		{
			return nullptr;
		}
		const uint8* BlockStart = (uint8*) Blocks[BlockIndex].GetData();
		return (CharType*)(BlockStart + OffsetInBlockBytes);
	}

	/**
	 * Gets the offset of a string previously stored string.
	 * @param String A string that was stored
	 * @return Offset in bytes to the string, or ~0 if the string was not found in the store.
	 */
	uint64 GetOffsetOfString(const TCHAR* String) const
	{
		const uint32 Length = FCString::Strlen(String);
		const FMemoryView View = MakeMemoryView(String,Length);
		for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); ++BlockIndex)
		{
			const FSharedBuffer& Block = Blocks[BlockIndex];
			if (Block.GetView().Contains(View))
			{
				const uint32 OffsetInBlock = (UPTRINT)String - (UPTRINT)Block.GetData();
				return (BlockIndex * BlockSize) + OffsetInBlock;
			}
		}
		return ~0;
	}

	/**
	 * Store a string, get offset back.
	 * @param String String to store
	 * @return Offset in bytes
	 */
	uint64 Store_GetOffset(const CharType* String)
	{
		return StoreInternal(StringViewType(String)).template Get<0>();
	}

	/**
	 * Store a string, get offset back.
	 * @param String String to store
	 * @return Offset in bytes
	 */
	uint64 Store_GetOffset(const StringViewType String)
	{
		return StoreInternal(String).template Get<0>();
	}

	/**
	 * Store a string, get string back.
	 * @param String String to store
	 * @return Stored string
	 */
	const CharType* Store_GetString(const CharType* String)
	{
		return StoreInternal(StringViewType(String)).template Get<1>();
	}

	/**
	 * Store a string, get string back.
	 * @param String String to store
	 * @return Stored string
	 */
	const CharType* Store_GetString(const StringViewType String)
	{
		return StoreInternal(String).template Get<1>();
	}

	/**
	 * Get the number of unique stored string.
	 * @return Number of strings
	 */
	uint32 Num() const
	{
		return StoredStrings.Num();
	}

	/**
	 * Get the total number of bytes used. Note that this does ot account for unused block space.
	 * @return Number of bytes used to store the strings.
	 */
	uint64 AllocatedSize()
	{
		return (State->BlockCount * BlockSize) - State->BufferLeftBytes;
	}

private:

	const TTuple<uint64, const CharType*>& StoreInternal(const StringViewType& String)
	{
		const uint32 Hash = GetTypeHash(String);
		
		if (TTuple<uint64, const CharType*>* AlreadyStored = StoredStrings.Find(Hash))
		{
			return *AlreadyStored;
		}

		check(String.Len() > 0);
		uint32 StringLength = String.Len() + 1;
		uint32 StringLengthBytes = StringLength * sizeof(CharType);
		check(StringLengthBytes <= BlockSize);
		if (State->BufferLeftBytes < StringLengthBytes)
		{
			//Request new block
			FSharedBuffer& Buffer = Blocks.Add_GetRef(Cache.CreateBlocks(CacheIndex, 1));
			BufferCursor = (CharType*) Buffer.GetData();
			++State->BlockCount;
			State->BufferLeftBytes = BlockSize;
		}
		
		const CharType* StoreString = BufferCursor;
		const uint64 StoredOffset = ((Blocks.Num() - 1) * BlockSize) + (BlockSize - State->BufferLeftBytes);
		FMemory::Memcpy(BufferCursor, String.GetData(), (StringLength - 1) * sizeof(CharType));
		BufferCursor[StringLength - 1] = TEXT('\0');
		State->BufferLeftBytes -= StringLengthBytes;
		BufferCursor += StringLength;

		// Sanity check
		check(GetStringAtOffset(StoredOffset) == StoreString);
		
		return StoredStrings.Add(Hash, MakeTuple(StoredOffset, StoreString));
	}
	
	struct FPersistentState
	{
		uint32 BlockCount;
		uint32 BufferLeftBytes;
	};
	
	constexpr static uint32 BlockSize = IAnalysisCache::BlockSizeBytes;

	IAnalysisCache& Cache;
	uint32 CacheIndex;
	TMap<uint32, TTuple<uint64,const CharType*>> StoredStrings;
	TArray<FSharedBuffer> Blocks;
	CharType* BufferCursor;
	FPersistentState* State;
};

typedef TCachedStringStore<WIDECHAR> FCachedStringStore;
typedef TCachedStringStore<ANSICHAR> FCachedStringStoreAnsi;

}