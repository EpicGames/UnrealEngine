// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Containers/Allocators.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Math/Interval.h"
#include "Templates/Function.h"
#include "Templates/IsArithmetic.h"
#include "Templates/Less.h"
#include "Templates/Requires.h"

#ifndef TRACESERVICES_PAGED_ARRAY_ITERATOR_V2
#define TRACESERVICES_PAGED_ARRAY_ITERATOR_V2 0 // enables a simpler implementation of TPagedArrayIterator, for debug purposes
#endif

#ifndef TRACESERVICES_PAGED_ARRAY_ITERATOR_DEBUG_ENABLED
#define TRACESERVICES_PAGED_ARRAY_ITERATOR_DEBUG_ENABLED 0
#endif

namespace TraceServices {

template<typename ItemType>
struct TPagedArrayPage
{
	ItemType* Items = nullptr;
	uint64 Count = 0;
};

template<typename ItemType>
inline const ItemType* GetData(const TPagedArrayPage<ItemType>& Page)
{
	return Page.Items;
}

template<typename ItemType>
inline SIZE_T GetNum(const TPagedArrayPage<ItemType>& Page)
{
	return Page.Count;
}

template<typename ItemType>
inline const ItemType* GetFirstItem(const TPagedArrayPage<ItemType>& Page)
{
	return Page.Items;
}

template<typename ItemType>
inline const ItemType* GetLastItem(const TPagedArrayPage<ItemType>& Page)
{
	if (Page.Count)
	{
		return Page.Items + Page.Count - 1;
	}
	else
	{
		return nullptr;
	}
}

template<typename ItemType, typename PageType>
class TPagedArray;

#if !TRACESERVICES_PAGED_ARRAY_ITERATOR_V2

template<typename ItemType, typename PageType>
class TPagedArrayIterator
{
public:
	TPagedArrayIterator()
	{
	}

	TPagedArrayIterator(const TPagedArray<ItemType, PageType>& InOuter, uint64 InItemIndex)
		: Outer(&InOuter)
	{
#if TRACESERVICES_PAGED_ARRAY_ITERATOR_DEBUG_ENABLED
		TotalItemCount = Outer->Num();
		TotalPageCount = Outer->PagesArray.Num();
#endif

		SetPositionInternal(InItemIndex);
		DebugCheckState();
	}

	//////////////////////////////////////////////////
	// Page Iterator

	uint64 GetCurrentPageIndex() const
	{
		return CurrentPageIndex;
	}

	const PageType* GetCurrentPage() const
	{
		return Outer->FirstPage + CurrentPageIndex;
	}

	const PageType* SetCurrentPage(uint64 PageIndex)
	{
		DebugCheckState();
		uint64 ItemIndex = PageIndex * Outer->PageSize;
		if (ItemIndex >= Outer->Num()) // end()
		{
			ItemIndex = Outer->Num();
			CurrentPageIndex = ItemIndex / Outer->PageSize;
			CurrentPageFirstItem = nullptr;
			CurrentPageLastItem = nullptr;
			CurrentItemIndex = ItemIndex;
			CurrentItem = nullptr;
			DebugCheckState();
			return nullptr;
		}
		else
		{
			check(PageIndex < Outer->PagesArray.Num());
			CurrentPageIndex = PageIndex;
			OnCurrentPageChanged();
			CurrentItemIndex = ItemIndex;
			PageType* CurrentPage = Outer->FirstPage + CurrentPageIndex;
			check(CurrentPage->Count > 0);
			CurrentItem = CurrentPage->Items;
			DebugCheckState();
			return CurrentPage;
		}
	}

	const PageType* PrevPage()
	{
		DebugCheckState();
		if (CurrentPageIndex == 0)
		{
			CurrentItem = nullptr;
			CurrentPageFirstItem = nullptr;
			CurrentPageLastItem = nullptr;
			CurrentItemIndex = Outer->Num();
			DebugCheckState();
			return nullptr;
		}
		--CurrentPageIndex;
		OnCurrentPageChanged();
		CurrentItemIndex = CurrentPageIndex * Outer->PageSize + (CurrentPageLastItem - CurrentPageFirstItem);
		CurrentItem = CurrentPageLastItem;
		DebugCheckState();
		return GetCurrentPage();
	}

	const PageType* NextPage()
	{
		DebugCheckState();
		if (CurrentPageIndex == Outer->PagesArray.Num() - 1)
		{
			CurrentItem = nullptr;
			CurrentPageFirstItem = nullptr;
			CurrentPageLastItem = nullptr;
			CurrentItemIndex = Outer->Num();
			DebugCheckState();
			return nullptr;
		}
		++CurrentPageIndex;
		OnCurrentPageChanged();
		CurrentItemIndex = CurrentPageIndex * Outer->PageSize;
		CurrentItem = CurrentPageFirstItem;
		DebugCheckState();
		return GetCurrentPage();
	}

	//////////////////////////////////////////////////
	// Item Iterator

	uint64 GetCurrentItemIndex() const
	{
		return CurrentItemIndex;
	}

	const ItemType* GetCurrentItem() const
	{
		return CurrentItem;
	}

	const ItemType* SetPosition(uint64 InItemIndex)
	{
		DebugCheckState();
		SetPositionInternal(InItemIndex);
		DebugCheckState();
		return CurrentItem;
	}

	const ItemType* PrevItem()
	{
		if (CurrentItem == CurrentPageFirstItem)
		{
			if (!PrevPage())
			{
				return nullptr;
			}
			else
			{
				return CurrentItem;
			}
		}
		DebugCheckState();
		--CurrentItemIndex;
		--CurrentItem;
		DebugCheckState();
		return CurrentItem;
	}

	const ItemType* NextItem()
	{
		if (CurrentItem == CurrentPageLastItem)
		{
			if (!NextPage())
			{
				return nullptr;
			}
			else
			{
				return CurrentItem;
			}
		}
		DebugCheckState();
		++CurrentItemIndex;
		++CurrentItem;
		DebugCheckState();
		return CurrentItem;
	}

	//////////////////////////////////////////////////
	// operators

	const ItemType& operator*() const
	{
		return *CurrentItem;
	}

	const ItemType* operator->() const
	{
		return CurrentItem;
	}

	explicit operator bool() const
	{
		return CurrentItem != nullptr;
	}

	TPagedArrayIterator& operator++()
	{
		NextItem();
		return *this;
	}

	TPagedArrayIterator operator++(int)
	{
		TPagedArrayIterator Tmp(*this);
		NextItem();
		return Tmp;
	}

	TPagedArrayIterator& operator--()
	{
		PrevItem();
		return *this;
	}

	TPagedArrayIterator operator--(int)
	{
		TPagedArrayIterator Tmp(*this);
		PrevItem();
		return Tmp;
	}

private:
	void OnCurrentPageChanged()
	{
		PageType* CurrentPage = Outer->FirstPage + CurrentPageIndex;
		CurrentPageFirstItem = CurrentPage->Items;
		if (CurrentPage->Items)
		{
			CurrentPageLastItem = CurrentPage->Items + CurrentPage->Count - 1;
		}
		else
		{
			CurrentPageLastItem = nullptr;
		}
	}

	void SetPositionInternal(uint64 InItemIndex)
	{
		CurrentPageIndex = InItemIndex / Outer->PageSize;
		CurrentItemIndex = InItemIndex;

		if (InItemIndex == Outer->Num()) // end()
		{
			CurrentPageFirstItem = nullptr;
			CurrentPageLastItem = nullptr;
			CurrentItem = nullptr;
		}
		else
		{
			check(InItemIndex < Outer->Num());
			check(CurrentPageIndex < Outer->PagesArray.Num());
			OnCurrentPageChanged();

			PageType* CurrentPage = Outer->FirstPage + CurrentPageIndex;
			uint64 ItemIndexInPage = InItemIndex % Outer->PageSize;
			check(ItemIndexInPage < CurrentPage->Count);
			CurrentItem = CurrentPage->Items + ItemIndexInPage;
		}
	}

	FORCEINLINE friend bool operator!=(const TPagedArrayIterator& Lhs, const TPagedArrayIterator& Rhs)
	{
		checkSlow(Lhs.Outer == Rhs.Outer); // Needs to be iterators of the same array
		return Lhs.CurrentItemIndex != Rhs.CurrentItemIndex;
	}

#if TRACESERVICES_PAGED_ARRAY_ITERATOR_DEBUG_ENABLED
	void DebugCheckState()
	{
		if (Outer)
		{
			check(TotalPageCount == Outer->PagesArray.Num());
			check(TotalItemCount == Outer->Num());

			if (CurrentItemIndex == TotalItemCount) // end()
			{
				check(CurrentItem == nullptr);

				check(CurrentPageIndex <= TotalPageCount);
				check(CurrentPageFirstItem == nullptr);
				check(CurrentPageLastItem == nullptr);
			}
			else
			{
				check(CurrentItemIndex < TotalItemCount);
				check(CurrentItem != nullptr);

				check(CurrentPageIndex < TotalPageCount);
				PageType* CurrentPage = Outer->FirstPage + CurrentPageIndex;
				check(CurrentPage->Count > 0);

				uint64 ItemIndexInPage = CurrentItemIndex % Outer->PageSize;
				check(ItemIndexInPage < CurrentPage->Count);

				check(CurrentPageFirstItem == CurrentPage->Items);
				check(CurrentPageLastItem == CurrentPage->Items + CurrentPage->Count - 1);
				check(CurrentItem == CurrentPage->Items + ItemIndexInPage);
			}
		}
		else
		{
			check(TotalPageCount == 0);
			check(TotalItemCount == 0);

			check(CurrentPageIndex == 0);
			check(CurrentPageFirstItem == nullptr);
			check(CurrentPageLastItem == nullptr);
			check(CurrentItemIndex == 0);
			check(CurrentItem == nullptr);
		}
	}
#else
	void DebugCheckState()
	{
	}
#endif

private:
	const TPagedArray<ItemType, PageType>* Outer = nullptr;
	uint64 CurrentPageIndex = 0;
	const ItemType* CurrentPageFirstItem = nullptr;
	const ItemType* CurrentPageLastItem = nullptr;
	uint64 CurrentItemIndex = 0;
	const ItemType* CurrentItem = nullptr;
#if TRACESERVICES_PAGED_ARRAY_ITERATOR_DEBUG_ENABLED
	uint64 TotalPageCount = 0;
	uint64 TotalItemCount = 0;
#endif
};

#else // TRACESERVICES_PAGED_ARRAY_ITERATOR_V2

template<typename ItemType, typename PageType>
class TPagedArrayIterator
{
public:
	TPagedArrayIterator()
	{
	}

	TPagedArrayIterator(const TPagedArray<ItemType, PageType>& InOuter, uint64 InItemIndex)
		: Outer(&InOuter)
		, CurrentItemIndex(InItemIndex)
	{
	}

	//////////////////////////////////////////////////
	// Page Iterator

	uint64 GetCurrentPageIndex() const
	{
		return CurrentItemIndex / Outer->PageSize;
	}

	const PageType* GetCurrentPage() const
	{
		return Outer->FirstPage + CurrentItemIndex / Outer->PageSize;
	}

	const PageType* SetCurrentPage(uint64 PageIndex)
	{
		CurrentItemIndex = PageIndex * Outer->PageSize;
		return Outer->FirstPage + PageIndex;
	}

	const PageType* PrevPage()
	{
		uint64 PageIndex = CurrentItemIndex / Outer->PageSize;
		if (PageIndex > 0)
		{
			--PageIndex;
			CurrentItemIndex = PageIndex * Outer->PageSize;
			return Outer->FirstPage + PageIndex;
		}
		else
		{
			CurrentItemIndex = Outer->Num();
			return nullptr;
		}
	}

	const PageType* NextPage()
	{
		uint64 PageIndex = CurrentItemIndex / Outer->PageSize + 1;
		if (PageIndex < Outer->NumPages())
		{
			CurrentItemIndex = PageIndex * Outer->PageSize;
			return Outer->FirstPage + PageIndex;
		}
		else
		{
			CurrentItemIndex = Outer->Num();
			return nullptr;
		}
	}

	//////////////////////////////////////////////////
	// Item Iterator

	uint64 GetCurrentItemIndex() const
	{
		return CurrentItemIndex;
	}

	const ItemType* GetCurrentItem() const
	{
		return &(*Outer)[CurrentItemIndex];
	}

	const ItemType* SetPosition(uint64 Index)
	{
		CurrentItemIndex = Index;
		return GetCurrentItem();
	}

	const ItemType* PrevItem()
	{
		if (CurrentItemIndex > 0)
		{
			--CurrentItemIndex;
			return GetCurrentItem();
		}
		else
		{
			return nullptr;
		}
	}

	const ItemType* NextItem()
	{
		if (CurrentItemIndex + 1 < Outer->Num())
		{
			++CurrentItemIndex;
			return GetCurrentItem();
		}
		else
		{
			CurrentItemIndex = Outer->Num();
			return nullptr;
		}
	}

	//////////////////////////////////////////////////
	// operators

	const ItemType& operator*() const
	{
		return (*Outer)[CurrentItemIndex];
	}

	const ItemType* operator->() const
	{
		return &(*Outer)[CurrentItemIndex];
	}

	explicit operator bool() const
	{
		return CurrentItemIndex < Outer->Num();
	}

	TPagedArrayIterator& operator++()
	{
		++CurrentItemIndex;
		return *this;
	}

	TPagedArrayIterator operator++(int)
	{
		TPagedArrayIterator Tmp(*this);
		++CurrentItemIndex;
		return Tmp;
	}

	TPagedArrayIterator& operator--()
	{
		--CurrentItemIndex;
		return *this;
	}

	TPagedArrayIterator operator--(int)
	{
		TPagedArrayIterator Tmp(*this);
		--CurrentItemIndex;
		return Tmp;
	}

private:
	FORCEINLINE friend bool operator!=(const TPagedArrayIterator& Lhs, const TPagedArrayIterator& Rhs)
	{
		checkSlow(Lhs.Outer == Rhs.Outer); // Needs to be iterators of the same array
		return Lhs.CurrentItemIndex != Rhs.CurrentItemIndex;
	}

	const TPagedArray<ItemType, PageType>* Outer = nullptr;
	uint64 CurrentItemIndex = 0;
};

#endif // TRACESERVICES_PAGED_ARRAY_ITERATOR_V2

template<typename InItemType, typename InPageType = TPagedArrayPage<InItemType>>
class TPagedArray
{
public:
	typedef InItemType ItemType;
	typedef InPageType PageType;
	typedef TPagedArrayIterator<InItemType, InPageType> TIterator;

	TPagedArray(ILinearAllocator& InAllocator, uint64 InPageSize)
		: Allocator(InAllocator)
		, PageSize(InPageSize)
	{

	}

	~TPagedArray()
	{
		for (PageType& Page : PagesArray)
		{
			ItemType* PageEnd = Page.Items + Page.Count;
			for (ItemType* Item = Page.Items; Item != PageEnd; ++Item)
			{
				Item->~ItemType();
			}
		}
	}

	uint64 Num() const
	{
		return TotalItemCount;
	}

	uint64 GetPageSize() const
	{
		return PageSize;
	}

	uint64 NumPages() const
	{
		return PagesArray.Num();
	}

	ItemType& PushBack()
	{
		if (!LastPage || LastPage->Count == PageSize)
		{
			LastPage = &PagesArray.AddDefaulted_GetRef();
			FirstPage = PagesArray.GetData();
			LastPage->Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
		}
		++TotalItemCount;
		ItemType* ItemPtr = LastPage->Items + LastPage->Count;
		new (ItemPtr) ItemType();
		++LastPage->Count;
		return *ItemPtr;
	}

	template <typename... ArgsType>
	ItemType& EmplaceBack(ArgsType&&... Args)
	{
		if (!LastPage || LastPage->Count == PageSize)
		{
			LastPage = &PagesArray.AddDefaulted_GetRef();
			FirstPage = PagesArray.GetData();
			LastPage->Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
		}
		++TotalItemCount;
		ItemType* ItemPtr = LastPage->Items + LastPage->Count;
		new (ItemPtr) ItemType(Forward<ArgsType>(Args)...);
		++LastPage->Count;
		return *ItemPtr;
	}

	ItemType& Insert(uint64 Index)
	{
		if (Index >= TotalItemCount)
		{
			return PushBack();
		}
		PushBack();
		uint64 PageIndex = Index / PageSize;
		uint64 PageItemIndex = Index % PageSize;
		for (uint64 CurrentPageIndex = PagesArray.Num() - 1; CurrentPageIndex > PageIndex; --CurrentPageIndex)
		{
			PageType* CurrentPage = FirstPage + CurrentPageIndex;
			memmove(CurrentPage->Items + 1, CurrentPage->Items, sizeof(ItemType) * (CurrentPage->Count - 1));
			PageType* PrevPage = CurrentPage - 1;
			memcpy(CurrentPage->Items, PrevPage->Items + PrevPage->Count - 1, sizeof(ItemType));
		}
		PageType* Page = FirstPage + PageIndex;
		memmove(Page->Items + PageItemIndex + 1, Page->Items + PageItemIndex, sizeof(ItemType) * (Page->Count - PageItemIndex - 1));
		return Page->Items[PageItemIndex];
	}

	PageType* GetLastPage()
	{
		return LastPage;
	}

	const PageType* GetLastPage() const
	{
		return LastPage;
	}

	PageType* GetPage(uint64 PageIndex)
	{
		return FirstPage + PageIndex;
	}

	const PageType* GetPage(uint64 PageIndex) const
	{
		return FirstPage + PageIndex;
	}

	PageType* GetItemPage(uint64 ItemIndex)
	{
		uint64 PageIndex = ItemIndex / PageSize;
		return FirstPage + PageIndex;
	}

	const PageType* GetItemPage(uint64 ItemIndex) const
	{
		uint64 PageIndex = ItemIndex / PageSize;
		return FirstPage + PageIndex;
	}

	TIterator GetIterator() const
	{
		return TIterator(*this, 0);
	}

	TIterator GetIteratorFromPage(uint64 PageIndex) const
	{
		return TIterator(*this, PageIndex * PageSize);
	}

	TIterator GetIteratorFromItem(uint64 ItemIndex) const
	{
		return TIterator(*this, ItemIndex);
	}

	const PageType* GetPages() const
	{
		return FirstPage;
	}

	ItemType& operator[](uint64 Index)
	{
		uint64 PageIndex = Index / PageSize;
		uint64 IndexInPage = Index % PageSize;
		PageType* Page = FirstPage + PageIndex;
		ItemType* Item = Page->Items + IndexInPage;
		return *Item;
	}

	const ItemType& operator[](uint64 Index) const
	{
		return const_cast<TPagedArray&>(*this)[Index];
	}

	ItemType& First()
	{
		ItemType* Item = FirstPage->Items;
		return *Item;
	}

	const ItemType& First() const
	{
		const ItemType* Item = FirstPage->Items;
		return *Item;
	}

	ItemType& Last()
	{
		ItemType* Item = LastPage->Items + LastPage->Count - 1;
		return *Item;
	}

	const ItemType& Last() const
	{
		const ItemType* Item = LastPage->Items + LastPage->Count - 1;
		return *Item;
	}

	FORCEINLINE TIterator begin() { return TIterator(*this, 0); }
	FORCEINLINE TIterator begin() const { return TIterator(*this, 0); }
	FORCEINLINE TIterator end() { return TIterator(*this, TotalItemCount); }
	FORCEINLINE TIterator end() const { return TIterator(*this, TotalItemCount); }

private:
	template<typename ItemType, typename PageType>
	friend class TPagedArrayIterator;

	ILinearAllocator& Allocator;
	TArray<PageType> PagesArray;
	PageType* FirstPage = nullptr;
	PageType* LastPage = nullptr;
	uint64 PageSize;
	uint64 TotalItemCount = 0;
};

template<typename ItemType, typename PageType>
inline const PageType* GetData(const TPagedArray<ItemType, PageType>& PagedArray)
{
	return PagedArray.GetPages();
}

template<typename ItemType, typename PageType>
inline SIZE_T GetNum(const TPagedArray<ItemType, PageType>& PagedArray)
{
	return PagedArray.NumPages();
}

/**
 * Use binary search to find the first and last element inside a TPagedArray that overlaps a given input interval.
 * This requires the elements in the array to be sorted by the value returned from the Projection.
 * Example usage for Timeline events would require a projection that returns Item.StartTime and the resulting range.Min
 * will point to the last element where Item.End > StartTime and range.Max to the last element where
 * Item.StartTime < EndTime.
 */
template<typename ItemType, typename PageType>
FInt32Interval GetElementRangeOverlappingGivenRange(const TPagedArray<ItemType, PageType>& PagedArray,
	double StartTime, double EndTime,
	TFunctionRef<double(const ItemType&)> ItemStartProjection,
	TFunctionRef<double(const ItemType&)> ItemEndProjection
	)
{
	FInt32Interval Result = { -1, -1 };
	check(EndTime >= StartTime);

	const PageType* PageData = PagedArray.GetPages();
	if (!PageData)
	{
		return Result;
	}

	const int32 NumPoints = static_cast<int32>(PagedArray.Num());
	const int32 NumPages = static_cast<int32>(PagedArray.NumPages());
	const int32 PageSize = static_cast<int32>(PagedArray.GetPageSize());

	if (ItemStartProjection(PagedArray.First()) > EndTime || ItemEndProjection(PagedArray.Last()) <= StartTime)
	{
		return Result;
	}

	TArrayView<const PageType, int32> Pages = MakeArrayView(PageData, NumPages);

	// find the page before the first page that's already inside the range (thus -1)
	int32 StartPageIndex = Algo::UpperBoundBy(Pages, StartTime,
		[&ItemEndProjection](const PageType& Page) { return ItemEndProjection(Page.Items[0]); });
	StartPageIndex -= 1;

	if (StartPageIndex < 0)
	{
		Result.Min = 0;
		StartPageIndex = 0;
	}
	else
	{
		// if we went past the end with StartPageIndex we still have to search the last page
		const PageType& Page = PageData[StartPageIndex];
		TArrayView<ItemType, int32> PageValues(Page.Items, static_cast<int32>(Page.Count));
		const int32 Index = Algo::UpperBoundBy(PageValues, StartTime, ItemEndProjection);
		check(Index >= 0);
		// Index may point past the end of the page, but the first item in the next page could be valid
		// as long as we're not in the last page
		if (Index >= Page.Count && StartPageIndex + 1 == NumPages )
		{
			return Result;
		}

		Result.Min = StartPageIndex * PageSize + Index;
		check(Index <= NumPoints);
	}

	TArrayView<const PageType, int32> RemainingPages = MakeArrayView(&PageData[StartPageIndex], NumPages - StartPageIndex);

	int32 EndPageIndex = Algo::UpperBoundBy(RemainingPages, EndTime,
		[&ItemStartProjection](const PageType& Page) { return ItemStartProjection(Page.Items[0]); });
	// EndPageIndex needs to be remapped back to the full page range
	EndPageIndex += StartPageIndex;

	// past the end means we still have to search the last page, so no early out here
	check(EndPageIndex > 0)
	// find the page before the first page that outside the range (thus -1)
	EndPageIndex -= 1;

	{
		const PageType& Page = PageData[EndPageIndex];
		TArrayView<ItemType, int32> PageValues(Page.Items, static_cast<int32>(Page.Count));
		int32 Index = Algo::UpperBoundBy(PageValues, EndTime, ItemStartProjection);
		check(Index >= 0);
		// Index may point past the end of the page, that means include last element in result
		if (Index >= PageValues.Num())
		{
			Index = PageValues.Num() - 1;
		}
		else
		{
			Index -= 1;
		}
		Result.Max = EndPageIndex * PageSize + Index;
	}

	return Result;
}

namespace PagedArrayAlgoImpl
{

/**
 * Performs binary search, resulting in position of the first element with projected value >= Value.
 *
 * @param PagedArray The paged array to search through; must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param Projection The functor or data member pointer; called via Invoke to compare to Value.
 * @param SortPredicate The predicate for sort comparison; defaults to <.
 *
 * @returns The position of the first element with projected value >= Value; may be == Num.
 */
template<typename SizeType, typename ItemType, typename PageType, typename ValueType, typename ProjectionType, typename SortPredicateType = TLess<>()>
FORCEINLINE SizeType LowerBoundInternal(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate)
{
	TArrayView<const PageType, int32> Pages = MakeArrayView(PagedArray.GetPages(), (int32)PagedArray.NumPages());

	// Find the first page with projected value of the first item >= searched Value.
	int32 PageIndex = Algo::LowerBoundBy(Pages, Value, [&Projection](const PageType& Page) { return Invoke(Projection, Page.Items[0]); }, SortPredicate);
	if (PageIndex == 0)
	{
		return 0;
	}

	// Look backward to previous items (with projected value >= searched Value) to find the first one.
	SizeType ElementIndex = PageIndex * PagedArray.GetPageSize();
	if (ElementIndex > PagedArray.Num())
	{
		ElementIndex = PagedArray.Num();
	}
	check(ElementIndex > 0);
	for (auto It = PagedArray.GetIteratorFromItem(ElementIndex - 1); It; It.PrevItem())
	{
		auto&& CheckValue = Invoke(Projection, *(It.GetCurrentItem()));
		if (SortPredicate(CheckValue, Value))
		{
			break;
		}
		ElementIndex = It.GetCurrentItemIndex();
	}
	return ElementIndex;
}

/**
 * Performs binary search, resulting in position of the first element with projected value > Value.
 *
 * @param PagedArray The paged array to search through; must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param Projection The functor or data member pointer; called via Invoke to compare to Value.
 * @param SortPredicate The predicate for sort comparison; defaults to <.
 *
 * @returns The position of the first element with projected value > Value; may be == Num.
 */
template<typename SizeType, typename ItemType, typename PageType, typename ValueType, typename ProjectionType, typename SortPredicateType = TLess<>()>
FORCEINLINE SizeType UpperBoundInternal(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate)
{
	TArrayView<const PageType, int32> Pages = MakeArrayView(PagedArray.GetPages(), (int32)PagedArray.NumPages());

	// Find the first page with projected value of the first item > searched Value.
	int32 PageIndex = Algo::UpperBoundBy(Pages, Value, [&Projection](const PageType& Page) { return Invoke(Projection, Page.Items[0]); }, SortPredicate);
	if (PageIndex == 0)
	{
		return 0;
	}

	// Look backward to previous items (with projected value > searched Value) to find the first one.
	SizeType ElementIndex = PageIndex * PagedArray.GetPageSize();
	if (ElementIndex > PagedArray.Num())
	{
		ElementIndex = PagedArray.Num();
	}
	check(ElementIndex > 0);
	for (auto It = PagedArray.GetIteratorFromItem(ElementIndex - 1); It; It.PrevItem())
	{
		auto&& CheckValue = Invoke(Projection, *(It.GetCurrentItem()));
		if (!SortPredicate(Value, CheckValue))
		{
			break;
		}
		ElementIndex = It.GetCurrentItemIndex();
	}
	return ElementIndex;
}

/**
 * Performs binary search, resulting in position of the first element with projected value closest to Value.
 *
 * @param PagedArray The paged array to search through; must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param Projection The functor or data member pointer; called via Invoke to compare to Value.
 * @param SortPredicate The predicate for sort comparison; defaults to <.
 *
 * @returns The position of the first element with projected value closest to Value; == 0 if Num == 0, otherwise is a value in range [0 .. Num-1].
 */
template<typename SizeType, typename ItemType, typename PageType, typename ValueType, typename ProjectionType, typename SortPredicateType = TLess<>()
	UE_REQUIRES(TIsArithmetic<ValueType>::Value)>
FORCEINLINE SizeType BinarySearchClosestInternal(const TPagedArray<ItemType, PageType>& PagedArray, ValueType Value, ProjectionType Projection, SortPredicateType SortPredicate)
{
	if (PagedArray.Num() <= 1)
	{
		return 0;
	}

	// Find the first page with projected value of the first item >= searched Value.
	TArrayView<const PageType, int32> Pages = MakeArrayView(PagedArray.GetPages(), (int32)PagedArray.NumPages());
	int32 StartPageIndex = Algo::LowerBoundBy(Pages, Value, [&Projection](const PageType& Page) { return Invoke(Projection, Page.Items[0]); }, SortPredicate);
	if (StartPageIndex == 0)
	{
		return 0;
	}

	// Iterate backward starting from this item (first item of the found page).
	SizeType ClosestElementIndex = StartPageIndex * PagedArray.GetPageSize();
	if (ClosestElementIndex >= PagedArray.Num())
	{
		ClosestElementIndex = PagedArray.Num() - 1;
	}
	if (ClosestElementIndex > 0)
	{
		auto&& ClosestElementValue = Invoke(Projection, PagedArray[ClosestElementIndex]);
		ValueType ClosestTimeDelta = FMath::Abs(ClosestElementValue - Value);
		for (auto It = PagedArray.GetIteratorFromItem(ClosestElementIndex - 1); It; It.PrevItem())
		{
			auto&& CheckValue = Invoke(Projection, *(It.GetCurrentItem()));
			ValueType Delta = FMath::Abs(CheckValue - Value);
			if (Delta > ClosestTimeDelta)
			{
				break;
			}
			ClosestTimeDelta = Delta;
			ClosestElementIndex = It.GetCurrentItemIndex();
		}
	}
	return ClosestElementIndex;
}

} // namespace PagedArrayAlgoImpl

namespace PagedArrayAlgo
{

/**
 * Performs binary search, resulting in position of the first element >= Value.
 *
 * @param PagedArray The paged array to search through; must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param SortPredicate Predicate for sort comparison; defaults to <.
 *
 * @returns Position of the first element >= Value; may be == Num.
 */
template <typename ItemType, typename PageType, typename SortPredicateType>
FORCEINLINE auto LowerBound(const TPagedArray<ItemType, PageType>& PagedArray, const ItemType& Value, SortPredicateType SortPredicate) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::LowerBoundInternal<SizeType>(PagedArray, Value, FIdentityFunctor(), SortPredicate);
}
template <typename ItemType, typename PageType>
FORCEINLINE auto LowerBound(const TPagedArray<ItemType, PageType>& PagedArray, const ItemType& Value) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::LowerBoundInternal<SizeType>(PagedArray, Value, FIdentityFunctor(), TLess<>());
}

/**
 * Performs binary search, resulting in position of the first element with projected value >= Value.
 *
 * @param PagedArray The paged array to search through, must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param Projection Functor or data member pointer; called via Invoke to compare to Value.
 * @param SortPredicate Predicate for sort comparison; defaults to <.
 *
 * @returns Position of the first element with projected value >= Value; may be == Num.
 */
template <typename ItemType, typename PageType, typename ValueType, typename ProjectionType, typename SortPredicateType>
FORCEINLINE auto LowerBoundBy(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::LowerBoundInternal<SizeType>(PagedArray, Value, Projection, SortPredicate);
}
template <typename ItemType, typename PageType, typename ValueType, typename ProjectionType>
FORCEINLINE auto LowerBoundBy(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::LowerBoundInternal<SizeType>(PagedArray, Value, Projection, TLess<>());
}

/**
 * Performs binary search, resulting in position of the first element > Value.
 *
 * @param PagedArray The paged array to search through, must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param SortPredicate Predicate for sort comparison; defaults to <.
 *
 * @returns Position of the first element > Value; may be == Num.
 */
template <typename ItemType, typename PageType, typename SortPredicateType>
FORCEINLINE auto UpperBound(const TPagedArray<ItemType, PageType>& PagedArray, const ItemType& Value, SortPredicateType SortPredicate) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::UpperBoundInternal<SizeType>(PagedArray, Value, FIdentityFunctor(), SortPredicate);
}
template <typename ItemType, typename PageType>
FORCEINLINE auto UpperBound(const TPagedArray<ItemType, PageType>& PagedArray, const ItemType& Value) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::UpperBoundInternal<SizeType>(PagedArray, Value, FIdentityFunctor(), TLess<>());
}

/**
 * Performs binary search, resulting in position of the first element with projected value > Value.
 *
 * @param PagedArray The paged array to search through, must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param Projection Functor or data member pointer; called via Invoke to compare to Value.
 * @param SortPredicate Predicate for sort comparison; defaults to <.
 *
 * @returns Position of the first element with projected value > Value; may be == Num.
 */
template <typename ItemType, typename PageType, typename ValueType, typename ProjectionType, typename SortPredicateType>
FORCEINLINE auto UpperBoundBy(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::UpperBoundInternal<SizeType>(PagedArray, Value, Projection, SortPredicate);
}
template <typename ItemType, typename PageType, typename ValueType, typename ProjectionType>
FORCEINLINE auto UpperBoundBy(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::UpperBoundInternal<SizeType>(PagedArray, Value, Projection, TLess<>());
}

/**
 * Performs binary search, resulting in position of the first element closest to Value.
 *
 * @param PagedArray The paged array to search through; must be already sorted by SortPredicate.
 * @param Value The value to look for
  * @param SortPredicate The predicate for sort comparison; defaults to <.
 *
 * @returns The position of the first element closest to Value; == 0 if Num == 0, otherwise is a value in range [0 .. Num-1].
 */
template <typename ItemType, typename PageType, typename SortPredicateType>
FORCEINLINE auto BinarySearchClosest(const TPagedArray<ItemType, PageType>& PagedArray, const ItemType& Value, SortPredicateType SortPredicate) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::BinarySearchClosestInternal<SizeType>(PagedArray, Value, FIdentityFunctor(), SortPredicate);
}
template <typename ItemType, typename PageType>
FORCEINLINE auto BinarySearchClosest(const TPagedArray<ItemType, PageType>& PagedArray, const ItemType& Value) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::BinarySearchClosestInternal<SizeType>(PagedArray, Value, FIdentityFunctor(), TLess<>());
}

/**
 * Performs binary search, resulting in position of the first element with projected value closest to Value.
 *
 * @param PagedArray The paged array to search through; must be already sorted by SortPredicate.
 * @param Value The value to look for
 * @param Projection The functor or data member pointer; called via Invoke to compare to Value.
 * @param SortPredicate The predicate for sort comparison; defaults to <.
 *
 * @returns The position of the first element with projected value closest to Value; == 0 if Num == 0, otherwise is a value in range [0 .. Num-1].
 */
template <typename ItemType, typename PageType, typename ValueType, typename ProjectionType, typename SortPredicateType>
FORCEINLINE auto BinarySearchClosestBy(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection, SortPredicateType SortPredicate) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::BinarySearchClosestInternal<SizeType>(PagedArray, Value, Projection, SortPredicate);
}
template <typename ItemType, typename PageType, typename ValueType, typename ProjectionType>
FORCEINLINE auto BinarySearchClosestBy(const TPagedArray<ItemType, PageType>& PagedArray, const ValueType& Value, ProjectionType Projection) -> decltype(GetNum(PagedArray))
{
	using SizeType = decltype(GetNum(PagedArray));
	return PagedArrayAlgoImpl::BinarySearchClosestInternal<SizeType>(PagedArray, Value, Projection, TLess<>());
}

} // namespace PagedArrayAlgo
} // namespace TraceServices
