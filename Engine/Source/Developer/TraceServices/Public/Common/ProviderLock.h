// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

struct FProviderEditScopeLock
{
	FProviderEditScopeLock(const IEditableProvider& InProvider)
		: Provider(InProvider)
	{
		Provider.BeginEdit();
	}

	~FProviderEditScopeLock()
	{
		Provider.EndEdit();
	}

	const IEditableProvider& Provider;
};

struct FProviderReadScopeLock
{
	FProviderReadScopeLock(const IProvider& InProvider)
		: Provider(InProvider)
	{
		Provider.BeginRead();
	}

	~FProviderReadScopeLock()
	{
		Provider.EndRead();
	}

	const IProvider& Provider;
};

/**
 * Utility class to implement the read/write lock for a provider.
 * Example usage:
       extern thread_local FProviderLock::FThreadLocalState MyProviderLockState;
 *     virtual void EditAccessCheck() const override.{ Lock.WriteAccessCheck(MyProviderLockState); }
 *     FProviderLock Lock;
 */
class FProviderLock
{
public:
	struct FThreadLocalState
	{
		FProviderLock* Lock;
		int32 ReadLockCount;
		int32 WriteLockCount;
	};

public:
	TRACESERVICES_API void ReadAccessCheck(FThreadLocalState& State) const;
	TRACESERVICES_API void WriteAccessCheck(FThreadLocalState& State) const;

	TRACESERVICES_API void BeginRead(FThreadLocalState& State);
	TRACESERVICES_API void EndRead(FThreadLocalState& State);

	TRACESERVICES_API void BeginWrite(FThreadLocalState& State);
	TRACESERVICES_API void EndWrite(FThreadLocalState& State);

private:
	FRWLock RWLock;
};

} // namespace TraceServices
