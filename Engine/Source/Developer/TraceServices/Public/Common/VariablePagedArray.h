// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Containers/Allocators.h"
#include "Containers/Array.h"

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////////////////////////
//
// The TVariablePagedArrayPage allocates items in fixed size pages. Each page allocates space for
// PageSize items, but a page may not use all its allocated items.
//
// To optimize lookup by index, as O(log(n)), a PageGroups array was added to identify the lists of
// full pages. After initial lookup, an iterator is O(1).
//
// Example:
//    Page Size: 8 items
//        Pages:   A   B   C   D   E   F   G   H
//   Full Pages:   ?   *   ?   ?   *   *   *   ?    // pages that are known to be full
//   Item Count:  [8   8   7] [3   8   8   8   8]   // used items in each page
//  First Index:   0   8   16  23  26  34  42  50   // first index in each page
//   Last Index:   7   15  22  30  33  41  49  57   // last index in each page
//  Page Groups:
//     group 0: 3 pages [A - C] 23 items [ 0 - 22]
//     group 1: 6 pages [D - H] 35 items [23 - 57]
//
// The first page (ex.: A or D) and the last page (ex.: C or H) in a group may or may not be full.
// But all middle pages in a group (ex.: B, E, F, G) are always full.
//
// When adding an item (PushBack), it will just add the respective item in the last page (adding a
// new page if necessary).
//
// When inserting items, if the page where insertion occurs has unused items, the item will be
// inserted in the respective page (no additional pages or groups are created). If page does not have
// unused items, it will be split in two pages (also a new page group will be created).
//
// Over TPagedArrayPage implementation, this has the advantage of being much faster when inserting
// items. There will be insertions only in pages that have unused items. The downside is the extra
// memory allocated and not used.
//
////////////////////////////////////////////////////////////////////////////////////////////////////

// For debugging the TVariablePagedArray implementation.
#define DEBUG_VARIABLE_PAGED_ARRAY 0

#if DEBUG_VARIABLE_PAGED_ARRAY
	#define VARIABLE_PAGED_ARRAY_CHECK(x) check(x)
#else
	#define VARIABLE_PAGED_ARRAY_CHECK(x)
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////
// In a page, used items are [ItemOffset .. ItemOffset + ItemCount - 1].
// Unused items are [0 .. ItemOffset - 1] and [ItemOffset + ItemCount .. PageSize - 1].

template<typename ItemType>
struct TVariablePagedArrayPage
{
	ItemType* Items = nullptr;
	uint64 ItemOffset = 0;
	uint64 ItemCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename ItemType>
inline const ItemType* GetData(const TVariablePagedArrayPage<ItemType>& Page)
{
	return Page.Items + Page.ItemOffset;
}

template<typename ItemType>
inline SIZE_T GetNum(const TVariablePagedArrayPage<ItemType>& Page)
{
	return Page.ItemCount;
}

template<typename ItemType>
inline const ItemType* GetFirstItem(const TVariablePagedArrayPage<ItemType>& Page)
{
	return Page.Items + Page.ItemOffset;
}

template<typename ItemType>
inline const ItemType* GetLastItem(const TVariablePagedArrayPage<ItemType>& Page)
{
	if (Page.ItemCount)
	{
		return Page.Items + Page.ItemOffset + Page.ItemCount - 1;
	}
	else
	{
		return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename ItemType>
struct TVariablePagedArrayPageGroup
{
	uint64 FirstItemIndex = 0;
	uint64 ItemCount = 0;
	uint64 FirstPageIndex = 0;
	uint64 PageCount = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename ItemType>
class TVariablePagedArray;

template<typename ItemType>
class TVariablePagedArrayIterator
{
public:
	typedef TVariablePagedArrayPage<ItemType> PageType;
	typedef TVariablePagedArrayPageGroup<ItemType> PageGroupType;
	typedef TVariablePagedArray<ItemType> ArrayType;

public:
	TVariablePagedArrayIterator(const ArrayType& InOuter)
		: Outer(&InOuter)
	{
#if DEBUG_VARIABLE_PAGED_ARRAY
		Outer->CheckIntegrity();
#endif
		SetPositionAtFirstItem();
	}

	TVariablePagedArrayIterator(const ArrayType& InOuter, const uint64 ItemIndex)
		: Outer(&InOuter)
	{
#if DEBUG_VARIABLE_PAGED_ARRAY
		Outer->CheckIntegrity();
#endif

		// This does not validate implementation, but user input.
		check(ItemIndex < Outer->TotalItemCount);

		const PageType* Page;
		uint64 ItemIndexInPage;
		Outer->FindItemChecked(ItemIndex, Page, ItemIndexInPage);

		CurrentPage = Page;
		OnCurrentPageChanged();

		CurrentItemIndex = ItemIndex;

		VARIABLE_PAGED_ARRAY_CHECK(ItemIndexInPage < CurrentPage->ItemCount);
		CurrentItem = CurrentPage->Items + CurrentPage->ItemOffset + ItemIndexInPage;
	}

	const PageType* GetCurrentPage()
	{
		return CurrentPage;
	}

	const ItemType* GetCurrentItem()
	{
		return CurrentItem;
	}

	const uint64 GetCurrentItemIndex()
	{
		return CurrentItemIndex;
	}

	const ItemType& operator*() const
	{
		return *CurrentItem;
	}

	const ItemType* operator->() const
	{
		return CurrentItem;
	}

	explicit operator bool() const
	{
		return CurrentItem != nullptr;
	}

	const PageType* PrevPage()
	{
		CurrentItemIndex -= static_cast<uint64>(CurrentItem - CurrentPageFirstItem) + 1;
		if (CurrentPage != Outer->FirstPage)
		{
			--CurrentPage;
			OnCurrentPageChanged();
			CurrentItem = CurrentPageLastItem;
		}
		else
		{
			CurrentPage = nullptr;
			CurrentItemIndex = 0;
			CurrentItem = nullptr;
			CurrentPageFirstItem = nullptr;
			CurrentPageLastItem = nullptr;
		}
		return CurrentPage;
	}

	const PageType* NextPage()
	{
		CurrentItemIndex += static_cast<uint64>(CurrentPageLastItem - CurrentItem) + 1;
		if (CurrentPage != Outer->LastPage)
		{
			++CurrentPage;
			OnCurrentPageChanged();
			CurrentItem = CurrentPageFirstItem;
		}
		else
		{
			CurrentPage = nullptr;
			CurrentItemIndex = 0;
			CurrentItem = nullptr;
			CurrentPageFirstItem = nullptr;
			CurrentPageLastItem = nullptr;
		}
		return CurrentPage;
	}

	const ItemType* PrevItem()
	{
		if (CurrentItem == CurrentPageFirstItem)
		{
			PrevPage();
		}
		else
		{
			--CurrentItemIndex;
			--CurrentItem;
		}
		return CurrentItem;
	}

	TVariablePagedArrayIterator& operator--()
	{
		PrevItem();
		return *this;
	}

	TVariablePagedArrayIterator operator--(int)
	{
		TVariablePagedArrayIterator Tmp(*this);
		PrevItem();
		return Tmp;
	}

	const ItemType* NextItem()
	{
		if (CurrentItem == CurrentPageLastItem)
		{
			NextPage();
		}
		else
		{
			++CurrentItemIndex;
			++CurrentItem;
		}
		return CurrentItem;
	}

	TVariablePagedArrayIterator& operator++()
	{
		NextItem();
		return *this;
	}

	TVariablePagedArrayIterator operator++(int)
	{
		TVariablePagedArrayIterator Tmp(*this);
		NextItem();
		return Tmp;
	}

	const ItemType* SetPositionAtFirstItem()
	{
		// This does not validate implementation, but user input.
		check(Outer->TotalItemCount > 0);

		CurrentPage = Outer->FirstPage;
		OnCurrentPageChanged();
		CurrentItemIndex = 0;
		CurrentItem = CurrentPageFirstItem;
		return CurrentItem;
	}

	const ItemType* SetPositionAtLastItem()
	{
		// This does not validate implementation, but user input.
		check(Outer->TotalItemCount > 0);

		CurrentPage = Outer->LastPage;
		OnCurrentPageChanged();
		CurrentItemIndex = Outer->TotalItemCount - 1;
		CurrentItem = CurrentPageLastItem;
		return CurrentItem;
	}

	const ItemType* SetPosition(uint64 Index)
	{
		// This does not validate implementation, but user input.
		check(Index < Outer->TotalItemCount);

		PageType* Page;
		uint64 ItemIndexInPage;
		Outer->FindItemChecked(Index, Page, ItemIndexInPage);

		CurrentPage = Page;
		OnCurrentPageChanged();
		CurrentItemIndex = Index;
		CurrentItem = CurrentPageFirstItem + ItemIndexInPage;
		return CurrentItem;
	}

private:
	void OnCurrentPageChanged()
	{
		CurrentPageFirstItem = CurrentPage->Items + CurrentPage->ItemOffset;
		CurrentPageLastItem = CurrentPageFirstItem + CurrentPage->ItemCount - 1;
	}

	const ArrayType* Outer = nullptr;
	const PageType* CurrentPage = nullptr;
	uint64 CurrentItemIndex = 0;
	const ItemType* CurrentItem = nullptr;
	const ItemType* CurrentPageFirstItem = nullptr;
	const ItemType* CurrentPageLastItem = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename InItemType>
class TVariablePagedArray
{
public:
	typedef InItemType ItemType;
	typedef TVariablePagedArrayPage<InItemType> PageType;
	typedef TVariablePagedArrayPageGroup<InItemType> PageGroupType;
	typedef TVariablePagedArrayIterator<InItemType> TIterator;

	TVariablePagedArray(ILinearAllocator& InAllocator, uint64 InPageSize)
		: Allocator(InAllocator)
		, Pages()
		, FirstPage(nullptr)
		, LastPage(nullptr)
		, PageGroups()
		, FirstPageGroup(nullptr)
		, LastPageGroup(nullptr)
		, PageSize(InPageSize)
		, TotalItemCount(0)
	{
	}

	~TVariablePagedArray()
	{
		for (PageType& Page : Pages)
		{
			const ItemType* const PageStart = Page.Items + Page.ItemOffset;
			const ItemType* const PageEnd = PageStart + Page.ItemCount;
			for (const ItemType* Item = PageStart; Item != PageEnd; ++Item)
			{
				Item->~ItemType();
			}
		}
	}

	uint64 Num() const
	{
		return TotalItemCount;
	}

	uint64 GetPageSize() const
	{
		return PageSize;
	}

	uint64 NumPages() const
	{
		return Pages.Num();
	}

	uint64 NumItemsWasted() const
	{
		const uint64 AllocatedItemCount = Pages.Num() * PageSize;
		return AllocatedItemCount - TotalItemCount;
	}

	double WastedPercent() const
	{
		const uint64 AllocatedItemCount = Pages.Num() * PageSize;
		return (AllocatedItemCount == 0) ? 0.0 : static_cast<double>(AllocatedItemCount - TotalItemCount) / static_cast<double>(AllocatedItemCount);
	}

	ItemType& PushBack()
	{
		if (!LastPage)
		{
			CreateInitialPage();
		}

		if (LastPage->ItemOffset + LastPage->ItemCount == PageSize)
		{
			// Add new page.
			PageType& NewPage = Pages.AddDefaulted_GetRef();
			NewPage.Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
			FirstPage = Pages.GetData();
			LastPage = &NewPage;
			++LastPageGroup->PageCount;
		}

		ItemType* ItemPtr = LastPage->Items + LastPage->ItemOffset + LastPage->ItemCount;
		new (ItemPtr) ItemType();

		++LastPage->ItemCount;
		++LastPageGroup->ItemCount;
		++TotalItemCount;

		return *ItemPtr;
	}

	ItemType& Insert(uint64 Index)
	{
		if (Index >= TotalItemCount)
		{
			return PushBack();
		}

		uint64 PageGroupIndex = Algo::UpperBoundBy(PageGroups, Index, &PageGroupType::FirstItemIndex);
		VARIABLE_PAGED_ARRAY_CHECK(PageGroupIndex > 0 && PageGroupIndex <= PageGroups.Num());
		--PageGroupIndex;

		PageGroupType* PageGroup = PageGroups.GetData() + PageGroupIndex;
		VARIABLE_PAGED_ARRAY_CHECK(Index >= PageGroup->FirstItemIndex);
		VARIABLE_PAGED_ARRAY_CHECK(Index < PageGroup->FirstItemIndex + PageGroup->ItemCount);

		if (Index == PageGroup->FirstItemIndex && PageGroupIndex > 0)
		{
			return PushBackItemInPageGroup(PageGroupIndex - 1);
		}

		// Find page containing the insertion index.
		uint64 ItemIndex = Index - PageGroup->FirstItemIndex;
		PageType* Page = Pages.GetData() + PageGroup->FirstPageIndex;
		// The first page (and the last page) in a group may have less items than PageSize.
		if (ItemIndex >= Page->ItemCount)
		{
			// Skip the first page in group.
			ItemIndex -= Page->ItemCount;
			++Page;
		}
		Page += ItemIndex / PageSize;
		ItemIndex = ItemIndex % PageSize;
		VARIABLE_PAGED_ARRAY_CHECK(ItemIndex < Page->ItemCount);

		// Does the page have unused items?
		if (Page->ItemCount < PageSize)
		{
			// Are unused items at the end of the page?
			if (Page->ItemOffset + Page->ItemCount < PageSize)
			{
				// Yes. Insert new item. Move items to the right to make room for the new item.
				ItemType* ItemPtr = Page->Items + Page->ItemOffset + ItemIndex;
				memmove(ItemPtr + 1, ItemPtr, sizeof(ItemType) * (Page->ItemCount - ItemIndex));
				++Page->ItemCount;
				OnInsertedItem(PageGroupIndex);
				return *ItemPtr;
			}
			else
			{
				// No. We have unused items at the begining of the page.
				VARIABLE_PAGED_ARRAY_CHECK(Page->ItemOffset > 0);

				// Insert new item. Move items to the left to make room for the new item.
				if (ItemIndex > 0)
				{
					ItemType* const FirstItemPtr = Page->Items + Page->ItemOffset;
					memmove(FirstItemPtr - 1, FirstItemPtr, sizeof(ItemType) * ItemIndex);
				}
				--Page->ItemOffset;
				++Page->ItemCount;
				OnInsertedItem(PageGroupIndex);
				ItemType* ItemPtr = Page->Items + Page->ItemOffset + ItemIndex;
				return *ItemPtr;
			}
		}

		// Page is full (no unused items).
		VARIABLE_PAGED_ARRAY_CHECK(Page->ItemCount == PageSize);

		ItemType* ItemPtr;

		// Split page.
		uint64 LeftPageIndex = Page - Pages.GetData();
		PageType* LeftPage;
		PageType* RightPage;
		if (ItemIndex == 0)
		{
			// No need to split the page. Just add a new one in front of it.
			LeftPage = &Pages.InsertDefaulted_GetRef(static_cast<uint32>(LeftPageIndex));
			RightPage = LeftPage + 1;

			LeftPage->Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
			LeftPage->ItemCount = 1;

			// The new inserted item will be the first item in left page (but last item in the left page group).
			ItemPtr = LeftPage->Items;
		}
		else
		{
			// Add another page after current one.
			RightPage = &Pages.InsertDefaulted_GetRef(static_cast<uint32>(LeftPageIndex) + 1);
			LeftPage = RightPage - 1;

			RightPage->Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
			RightPage->ItemCount = LeftPage->ItemCount - ItemIndex;
			RightPage->ItemOffset = PageSize - RightPage->ItemCount;
			LeftPage->ItemCount = ItemIndex + 1;

			// The inserted item will be the last item in left page (also last item in the left page group).
			ItemPtr = LeftPage->Items + LeftPage->ItemOffset + ItemIndex;

			// Move items from current (left) page to new (right) page.
			memcpy(RightPage->Items + RightPage->ItemOffset, ItemPtr, sizeof(ItemType) * RightPage->ItemCount);
		}
		OnInsertedPage(PageGroupIndex);
		OnInsertedItem(PageGroupIndex);

		// Split the page group.
		PageGroupType* RightPageGroup = &PageGroups.InsertDefaulted_GetRef(static_cast<uint32>(PageGroupIndex) + 1);
		FirstPageGroup = PageGroups.GetData();
		LastPageGroup = PageGroups.GetData() + PageGroups.Num() - 1;
		PageGroupType* LeftPageGroup = RightPageGroup - 1;

		const uint64 DuoItemCount = LeftPageGroup->ItemCount;
		const uint64 DuoPageCount = LeftPageGroup->PageCount;

		//LeftPageGroup->FirstItemIndex = unchanged
		LeftPageGroup->ItemCount = Index - LeftPageGroup->FirstItemIndex + 1;
		//LeftPageGroup->FirstPageIndex = unchanged
		LeftPageGroup->PageCount = LeftPageIndex - LeftPageGroup->FirstPageIndex + 1;

		RightPageGroup->FirstItemIndex = LeftPageGroup->FirstItemIndex + LeftPageGroup->ItemCount;
		RightPageGroup->ItemCount = DuoItemCount - LeftPageGroup->ItemCount;
		RightPageGroup->FirstPageIndex = LeftPageGroup->FirstPageIndex + LeftPageGroup->PageCount;
		RightPageGroup->PageCount = DuoPageCount - LeftPageGroup->PageCount;

#if DEBUG_VARIABLE_PAGED_ARRAY
		CheckIntegrity();
#endif

		return *ItemPtr;
	}

	TIterator GetIterator() const
	{
		return TIterator(*this, 0);
	}

	TIterator GetIteratorFromItem(uint64 ItemIndex) const
	{
		return TIterator(*this, ItemIndex);
	}

	ItemType& operator[](uint64 Index)
	{
		PageType* Page;
		uint64 ItemIndexInPage;
		FindItemChecked(Index, Page, ItemIndexInPage);
		ItemType* Item = Page->Items + Page->ItemOffset + ItemIndexInPage;
		return *Item;
	}

	const ItemType& operator[](uint64 Index) const
	{
		return const_cast<TVariablePagedArray&>(*this)[Index];
	}

	ItemType& First()
	{
		ItemType* Item = FirstPage->Items + FirstPage->ItemOffset;
		return *Item;
	}

	const ItemType& First() const
	{
		const ItemType* Item = FirstPage->Items + FirstPage->ItemOffset;
		return *Item;
	}

	ItemType& Last()
	{
		ItemType* Item = LastPage->Items + LastPage->ItemOffset + LastPage->ItemCount - 1;
		return *Item;
	}

	const ItemType& Last() const
	{
		const ItemType* Item = LastPage->Items + LastPage->ItemOffset + LastPage->ItemCount - 1;
		return *Item;
	}

private:
	void FindItemChecked(const uint64 Index, const PageType*& OutPage, uint64& OutItemIndexInPage) const
	{
		VARIABLE_PAGED_ARRAY_CHECK(Index < TotalItemCount);

		// Find the page group containing the index.
		uint64 PageGroupIndex = Algo::UpperBoundBy(PageGroups, Index, &PageGroupType::FirstItemIndex);
		VARIABLE_PAGED_ARRAY_CHECK(PageGroupIndex > 0 && PageGroupIndex <= PageGroups.Num());
		--PageGroupIndex;

		const PageGroupType* PageGroup = PageGroups.GetData() + PageGroupIndex;
		VARIABLE_PAGED_ARRAY_CHECK(Index >= PageGroup->FirstItemIndex);
		VARIABLE_PAGED_ARRAY_CHECK(Index < PageGroup->FirstItemIndex + PageGroup->ItemCount);

		// Find the page containing the index.
		uint64 ItemIndex = Index - PageGroup->FirstItemIndex;
		const PageType* Page = Pages.GetData() + PageGroup->FirstPageIndex;
		// The first page (and the last page) in a group may have less items than PageSize.
		if (ItemIndex >= Page->ItemCount)
		{
			// Skip the first page in group.
			ItemIndex -= Page->ItemCount;
			++Page;
		}
		Page += ItemIndex / PageSize;
		ItemIndex = ItemIndex % PageSize;
		VARIABLE_PAGED_ARRAY_CHECK(ItemIndex < Page->ItemCount);

		OutPage = Page;
		OutItemIndexInPage = ItemIndex;
	}

	void CreateInitialPage()
	{
		// Create the initial page.
		VARIABLE_PAGED_ARRAY_CHECK(Pages.Num() == 0);
		PageType* Page = &Pages.AddDefaulted_GetRef();
		Page->Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
		FirstPage = Page;
		LastPage = Page;

		// Create the initial page group.
		VARIABLE_PAGED_ARRAY_CHECK(PageGroups.Num() == 0);
		PageGroupType* PageGroup = &PageGroups.AddDefaulted_GetRef();
		PageGroup->PageCount = 1;
		FirstPageGroup = PageGroup;
		LastPageGroup = PageGroup;
	}

	ItemType& PushBackItemInPageGroup(const uint64 PageGroupIndex)
	{
		PageGroupType& PageGroup = PageGroups[static_cast<uint32>(PageGroupIndex)];
		uint64 PageIndex = PageGroup.FirstPageIndex + PageGroup.PageCount - 1;
		PageType* Page = &Pages[static_cast<uint32>(PageIndex)];

		if (Page->ItemOffset + Page->ItemCount == PageSize) // if page is full
		{
			// Add a new page.
			++PageIndex;
			Page = &Pages.InsertDefaulted_GetRef(static_cast<uint32>(PageIndex));
			Page->Items = reinterpret_cast<ItemType*>(Allocator.Allocate(PageSize * sizeof(ItemType)));
			OnInsertedPage(PageGroupIndex);
		}

		// Push back a new item.
		ItemType* ItemPtr = Page->Items + Page->ItemOffset + Page->ItemCount;
		new (ItemPtr) ItemType();
		++Page->ItemCount;
		OnInsertedItem(PageGroupIndex);

		return *ItemPtr;
	}

	void OnInsertedPage(const uint64 PageGroupIndex)
	{
		FirstPage = Pages.GetData();
		LastPage = Pages.GetData() + Pages.Num() - 1;

		PageGroupType* const PageGroup = PageGroups.GetData() + PageGroupIndex;

		++PageGroup->PageCount;

		// Update the following page groups.
		PageGroupType* const StartPageGroup = PageGroup + 1;
		PageGroupType* const EndPageGroup = PageGroups.GetData() + PageGroups.Num();
		for (PageGroupType* CurrentPageGroup = StartPageGroup; CurrentPageGroup != EndPageGroup; ++CurrentPageGroup)
		{
			++CurrentPageGroup->FirstPageIndex;
		}
	}

	void OnInsertedItem(const uint64 PageGroupIndex)
	{
		PageGroupType* const PageGroup = PageGroups.GetData() + PageGroupIndex;

		++PageGroup->ItemCount;
		++TotalItemCount;

		// Update the following page groups.
		PageGroupType* const StartPageGroup = PageGroup + 1;
		PageGroupType* const EndPageGroup = PageGroups.GetData() + PageGroups.Num();
		for (PageGroupType* CurrentPageGroup = StartPageGroup; CurrentPageGroup != EndPageGroup; ++CurrentPageGroup)
		{
			++CurrentPageGroup->FirstItemIndex;
		}
	}

	void CheckIntegrity() const
	{
		check(PageSize > 0);

		if (TotalItemCount == 0)
		{
			check(Pages.Num() == 0);
			check(FirstPage == nullptr);
			check(LastPage == nullptr);
			check(PageGroups.Num() == 0);
			check(FirstPageGroup == nullptr);
			check(LastPageGroup == nullptr);
		}
		else
		{
			check(Pages.Num() > 0);
			check(FirstPage == Pages.GetData());
			check(LastPage == Pages.GetData() + Pages.Num() - 1);

			check(PageGroups.Num() > 0);
			check(FirstPageGroup == PageGroups.GetData());
			check(LastPageGroup == PageGroups.GetData() + PageGroups.Num() - 1);
		}

		// Verify pages.
		{
			uint64 ItemCount = 0;
			for (const PageType& Page : Pages)
			{
				check(Page.Items != nullptr);
				check(Page.ItemCount > 0);
				check(Page.ItemOffset + Page.ItemCount <= PageSize);
				ItemCount += Page.ItemCount;
			}
			check(ItemCount == TotalItemCount);
		}

		// Verify page groups.
		{
			uint64 ItemCount = 0;
			uint64 PageCount = 0;
			uint64 ItemIndex = 0;
			uint64 PageIndex = 0;
			for (const PageGroupType& PageGroup : PageGroups)
			{
				check(PageGroup.ItemCount > 0);
				ItemCount += PageGroup.ItemCount;
				check(PageGroup.FirstItemIndex == ItemIndex);
				ItemIndex += PageGroup.ItemCount;

				check(PageGroup.PageCount > 0);
				PageCount += PageGroup.PageCount;
				check(PageGroup.FirstPageIndex == PageIndex);
				PageIndex += PageGroup.PageCount;
			}
			check(ItemCount == TotalItemCount);
			check(PageCount == Pages.Num());
		}
	}

private:
	template<typename ItemType>
	friend class TVariablePagedArrayIterator;

	ILinearAllocator& Allocator;

	TArray<PageType> Pages;
	PageType* FirstPage;
	PageType* LastPage;

	TArray<PageGroupType> PageGroups;
	PageGroupType* FirstPageGroup;
	PageGroupType* LastPageGroup;

	uint64 PageSize;
	uint64 TotalItemCount;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace TraceServices
