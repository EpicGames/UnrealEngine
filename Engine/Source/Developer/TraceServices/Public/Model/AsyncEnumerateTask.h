// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"

#include "Model/MonotonicTimelineData.h"
#include "TraceServices/Containers/Timelines.h"

namespace TraceServices
{

template<typename InEventType, typename SettingsType>
struct FAsyncEnumerateTaskData
{
	uint32 TaskIndex;
	uint32 NumTasks;
	uint64 StartPageIndex; //Inclusive
	uint64 EndPageIndex; //Exclusive
	double StartTime;
	double EndTime;
	EEventSortOrder SortOrder;
	const FDetailLevel<InEventType, SettingsType>* DetailLevel;

	typename ITimeline<InEventType>::AsyncEventRangeCallback EventRangeCallback;
	typename ITimeline<InEventType>::AsyncEventCallback EventCallback;
};

template<typename InEventType, typename SettingsType>
class EnumerateAsyncAlgoritm
{
public:
	struct FEnumerationStackEntry
	{
		double StartTime;
		InEventType Event;
	};

	typedef TFunctionRef<void(int32 /*InitialStackDepth*/)> InitialStackCallback;

	static void EnumerateEventsDownSampled(FAsyncEnumerateTaskData<InEventType, SettingsType>& Data, InitialStackCallback InitCallback, typename ITimeline<InEventType>::EventCallback Callback)
	{
		const FDetailLevel<InEventType, SettingsType>& DetailLevel = *Data.DetailLevel;

		auto ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(Data.StartPageIndex);
		const FEventScopeEntryPage* ScopePage = ScopeEntryIterator.GetCurrentPage();
		const FEventScopeEntryPage* InitialPage = ScopePage;

		auto EventsIterator = DetailLevel.Events.GetIteratorFromItem(ScopePage->BeginEventIndex);
		FEnumerationStackEntry EventStack[SettingsType::MaxDepth];
		int32 CurrentStackDepth = ScopePage->InitialStackCount;

		// Init the enumeration EventStack using the start page initial stack.
		for (int32 InitialStackIndex = 0; InitialStackIndex < CurrentStackDepth; ++InitialStackIndex)
		{
			FEnumerationStackEntry& EnumerationStackEntry = EventStack[InitialStackIndex];
			const FEventStackEntry& EventStackEntry = ScopePage->InitialStack[InitialStackIndex];
			EnumerationStackEntry.StartTime = DetailLevel.GetScopeEntryTime(EventStackEntry.EnterScopeIndex);
			EnumerationStackEntry.Event = DetailLevel.GetEvent(EventStackEntry.EventIndex);
		}

		// Check if we must also send the initial stack events to the caller.
		if ((Data.SortOrder == EEventSortOrder::ByEndTime) ||
			(Data.TaskIndex == 0 && Data.SortOrder == EEventSortOrder::ByStartTime))
		{
			InitCallback(0);

			for (int32 StackIndex = 0; StackIndex < CurrentStackDepth; ++StackIndex)
			{
				FEnumerationStackEntry& StackEntry = EventStack[StackIndex];
				if (Callback(true, StackEntry.StartTime, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}
			}
		}
		else
		{
			InitCallback(CurrentStackDepth);
		}

		const FEventScopeEntry* ScopeEntry = ScopeEntryIterator.GetCurrentItem();
		const InEventType* Event = EventsIterator.GetCurrentItem();
		uint64 CurrentPageIndex = Data.StartPageIndex;

		// Iterate the pages owned by this task.
		while (ScopeEntry && CurrentPageIndex < Data.EndPageIndex)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEnumerationStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.Event = *Event;
				StackEntry.StartTime = -ScopeEntry->Time;

				if (Callback(true, -ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}

				Event = EventsIterator.NextItem();
			}
			else // "end" event
			{
				check(CurrentStackDepth > 0);
				--CurrentStackDepth;

				FEnumerationStackEntry& StackEntry = EventStack[CurrentStackDepth];
				if (Callback(false, ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}

			}
			ScopeEntry = ScopeEntryIterator.NextItem();
			CurrentPageIndex = ScopeEntryIterator.GetCurrentPageIndex();
		}

		// Check if we must find the end times for the events remaining on the enumeration stack after we have iterated
		// all pages owned by this task.
		if (Data.TaskIndex != Data.NumTasks - 1 && Data.SortOrder == EEventSortOrder::ByEndTime)
		{
			return;
		}

		if (ShouldStopSearchingForEndEvents(Data, EventStack, InitialPage, CurrentStackDepth, DetailLevel))
		{
			return;
		}

		if (ScopeEntry)
		{
			do
			{
				const FEventScopeEntryPage* CurrentScopePage = ScopeEntryIterator.GetCurrentPage();
				check(CurrentStackDepth <= CurrentScopePage->InitialStackCount);

				while (CurrentStackDepth > 0 && CurrentScopePage->InitialStack[CurrentStackDepth - 1].EndTime > 0)
				{
					--CurrentStackDepth;
					InEventType CurrentEvent = DetailLevel.GetEvent(CurrentScopePage->InitialStack[CurrentStackDepth].EventIndex);
					if (Callback(false, CurrentScopePage->InitialStack[CurrentStackDepth].EndTime, CurrentEvent) == EEventEnumerate::Stop)
					{
						return;
					}

					if (ShouldStopSearchingForEndEvents(Data, EventStack, InitialPage, CurrentStackDepth, DetailLevel))
					{
						return;
					}
				}
			} while (ScopeEntryIterator.NextPage());
		}

		// If there are still events on the stack at this point, it means analysis has not yet reached their end,
		// so we truncate them to the time of last received event.
		while (CurrentStackDepth > 0)
		{
			FEnumerationStackEntry& StackEntry = EventStack[--CurrentStackDepth];
			if (Callback(false, DetailLevel.InsertionState.LastTime, StackEntry.Event) == EEventEnumerate::Stop)
			{
				return;
			}

			if (ShouldStopSearchingForEndEvents(Data, EventStack, InitialPage, CurrentStackDepth, DetailLevel))
			{
				return;
			}
		}
	}

private:
	static bool ShouldStopSearchingForEndEvents(FAsyncEnumerateTaskData<InEventType, SettingsType>& Data, FEnumerationStackEntry* EventStack, const FEventScopeEntryPage* InitialPage, int32 CurrentStackDepth, const FDetailLevel<InEventType, SettingsType>& DetailLevel)
	{
		if (Data.SortOrder == EEventSortOrder::ByStartTime &&
			Data.TaskIndex > 0 &&
			CurrentStackDepth > 0 &&
			CurrentStackDepth <= InitialPage->InitialStackCount)
		{
			double InitialStartTime = DetailLevel.GetScopeEntryTime(InitialPage->InitialStack[CurrentStackDepth - 1].EnterScopeIndex);

			if (EventStack[CurrentStackDepth - 1].StartTime == InitialStartTime)
			{
				return true;
			}
		}

		return false;
	}

};

template<typename InEventType, typename SettingsType>
class FEnumerateAsyncTask : public FNonAbandonableTask
{
	using EventType = InEventType;

	friend class FAsyncTask<FEnumerateAsyncTask>;

public:
	FEnumerateAsyncTask(const FAsyncEnumerateTaskData<InEventType, SettingsType>& InData)
		: Data(InData)
	{
	}

	void EnumerateOrderedByStartTime()
	{
		struct FStackEntry
		{
			uint64 OutputEventIndex;
		};

		FStackEntry EventStack[SettingsType::MaxDepth];
		uint32 CurrentDepth = 0;
		uint32 MinimumDepth = 0;

		struct FOutputEvent
		{
			double StartTime;
			double EndTime;
			uint32 Depth;
			EventType Event;
		};

		TArray<FOutputEvent> OutputEvents;

		EnumerateAsyncAlgoritm<InEventType, SettingsType>::EnumerateEventsDownSampled(Data,
			[&CurrentDepth, &MinimumDepth]
			(int32 InInitialDepth)
			{
				MinimumDepth = InInitialDepth;
				CurrentDepth = InInitialDepth;
			},
			[this, &EventStack, &OutputEvents, &CurrentDepth, &MinimumDepth]
			(bool IsEnter, double Time, const EventType& Event)
			{
				if (IsEnter)
				{
					FStackEntry& StackEntry = EventStack[CurrentDepth];
					StackEntry.OutputEventIndex = OutputEvents.Num();
					FOutputEvent& OutputEvent = OutputEvents.AddDefaulted_GetRef();
					OutputEvent.StartTime = Time;
					OutputEvent.EndTime = Time;
					OutputEvent.Depth = CurrentDepth;
					OutputEvent.Event = Event;
					++CurrentDepth;
				}
				else
				{
					{
						FStackEntry& StackEntry = EventStack[--CurrentDepth];
						if (CurrentDepth >= MinimumDepth)
						{
							FOutputEvent* OutputEvent = OutputEvents.GetData() + StackEntry.OutputEventIndex;
							OutputEvent->EndTime = Time;
						}
					}
					if (CurrentDepth <= MinimumDepth)
					{
						MinimumDepth = CurrentDepth;
						for (FOutputEvent& OutputEvent : OutputEvents)
						{
							if (OutputEvent.EndTime >= Data.StartTime && OutputEvent.StartTime <= Data.EndTime)
							{
								if (Data.EventRangeCallback(OutputEvent.StartTime, OutputEvent.EndTime, OutputEvent.Depth, OutputEvent.Event, Data.TaskIndex) == EEventEnumerate::Stop)
								{
									return EEventEnumerate::Stop;
								}
							}
						}
						OutputEvents.Empty(OutputEvents.Num());
					}
				}

				return EEventEnumerate::Continue;
			});

		check(OutputEvents.Num() == 0);
	}

	void EnumerateOrderedByEndTime()
	{
		struct FStackEntry
		{
			double StartTime;
		};

		FStackEntry EventStack[SettingsType::MaxDepth];
		uint32 CurrentDepth = 0;

		EnumerateAsyncAlgoritm<InEventType, SettingsType>::EnumerateEventsDownSampled(Data,
			[&CurrentDepth]
			(int32 InInitialDepth)
			{
				CurrentDepth = InInitialDepth;
			},
			[this, &EventStack, &CurrentDepth]
			(bool IsEnter, double Time, const EventType& Event)
			{
				if (IsEnter)
				{
					FStackEntry& StackEntry = EventStack[CurrentDepth];
					StackEntry.StartTime = Time;
					++CurrentDepth;
				}
				else
				{
					FStackEntry& StackEntry = EventStack[--CurrentDepth];

					if (Time >= Data.StartTime && StackEntry.StartTime <= Data.EndTime)
					{
						if (Data.EventRangeCallback(StackEntry.StartTime, Time, CurrentDepth, Event, Data.TaskIndex) == EEventEnumerate::Stop)
						{
							return EEventEnumerate::Stop;
						}
					}
				}

				return EEventEnumerate::Continue;
			});
	}

	void DoWork()
	{
		if (Data.EventRangeCallback != nullptr)
		{
			if (Data.SortOrder == EEventSortOrder::ByEndTime)
			{
				EnumerateOrderedByEndTime();
			}
			else if (Data.SortOrder == EEventSortOrder::ByStartTime)
			{
				EnumerateOrderedByStartTime();
			}
		}
		else
		{
			EnumerateAsyncAlgoritm<InEventType, SettingsType>::EnumerateEventsDownSampled(Data,
				[](int32 InInitialDepth)
				{
				},
				[this](bool IsEnter, double Time, const EventType& Event)
				{
					return this->Data.EventCallback(IsEnter, Time, Event, Data.TaskIndex);
				});
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(EnumerateAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	FAsyncEnumerateTaskData<InEventType, SettingsType> Data;
};

} // namespace TraceServices
