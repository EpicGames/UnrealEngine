// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Containers/Timelines.h"
#include "Common/PagedArray.h"
#include "TraceServices/Containers/Allocators.h"
#include <limits>

namespace TraceServices
{

struct FIntervalTimelineDefaultSettings
{
	enum
	{
		EventsPerPage = 65536
	};
};

template<typename EventType, typename SettingsType = FIntervalTimelineDefaultSettings>
class TIntervalTimeline
	: public ITimeline<EventType>
{
public:
	TIntervalTimeline(ILinearAllocator& Allocator)
		: Events(Allocator, SettingsType::EventsPerPage)
	{
	}

	virtual ~TIntervalTimeline() = default;
	virtual uint64 GetModCount() const override { return ModCount; }
	virtual uint64 GetEventCount() const override { return Events.Num(); }
	virtual const EventType& GetEvent(uint64 InIndex) const override { return Events[InIndex].Event; }
	double GetEventStartTime(uint64 InIndex) const { return Events[InIndex].StartTime; }
	double GetEventEndTime(uint64 InIndex) const { return Events[InIndex].EndTime; }
	virtual double GetStartTime() const override { return Events.Num() > 0 ? Events[0].StartTime : 0.0; }
	virtual double GetEndTime() const override { return Events.Num() > 0 ? Events[Events.Num() - 1].EndTime : 0.0; }
	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, typename ITimeline<EventType>::EventCallback Callback) const override { check(false); }
	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, typename ITimeline<EventType>::EventRangeCallback Callback) const override { check(false); }
	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, typename  ITimeline<EventType>::EventCallback Callback) const override
	{
		EnumerateEvents(IntervalStart, IntervalEnd, [&Callback](double InStartTime, double InEndTime, uint32 InDepth, const EventType& InEvent)
		{
			Callback(true, InStartTime, InEvent);
			Callback(false, InEndTime, InEvent);
			return EEventEnumerate::Continue;
		});
	}
	
	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, typename ITimeline<EventType>::EventRangeCallback Callback) const override
	{
		auto EventIterator = Events.GetIteratorFromPage(0);
		const FEventInternal* Event = EventIterator.GetCurrentItem();
		while (Event)
		{
			const FEventPage* Page = EventIterator.GetCurrentPage();
			if ((Page->OpenEvents == 0 && Page->EndTime < IntervalStart) || IntervalEnd < Page->BeginTime)
			{
				if (!EventIterator.NextPage())
				{
					return;
				}
				Event = EventIterator.GetCurrentItem();
			}
			else
			{
				if (!(Event->EndTime < IntervalStart || IntervalEnd < Event->StartTime))
				{
					if (Callback(Event->StartTime, Event->EndTime, 0, Event->Event) == EEventEnumerate::Stop)
					{
						return;
					}
				}
				Event = EventIterator.NextItem();
			}
		}
	}
	
	uint64 AppendBeginEvent(double StartTime, const EventType& Event)
	{
		return EmplaceBeginEvent(StartTime, Event);
	}

	template <typename... ArgsType>
	uint64 EmplaceBeginEvent(double StartTime, ArgsType&&... Args)
	{
		//check(StartTime >= LastTime);
		//LastTime = StartTime;

		uint64 Index = Events.Num();
		Events.EmplaceBack(StartTime, std::numeric_limits<double>::infinity(), Forward<ArgsType>(Args)...);

		FEventPage* LastPage = Events.GetLastPage();
		if (LastPage != CurrentPage)
		{
			CurrentPage = LastPage;
			LastPage->BeginTime = StartTime;
			LastPage->EndTime = StartTime;
			LastPage->BeginEventIndex = Index;
		}
		else
		{
			LastPage->BeginTime = FMath::Min(LastPage->BeginTime, StartTime);
			LastPage->EndTime = FMath::Max(LastPage->EndTime, StartTime);
		}

		LastPage->EndEventIndex = Index + 1;

		++ModCount;
		++LastPage->OpenEvents;

		return Index;
	}

	EventType& EndEvent(uint64 EventIndex, double EndTime)
	{
		//check(EndTime >= LastTime);
		//LastTime = EndTime;
		FEventInternal& EventInternal = Events[EventIndex];
		check(EventIndex < Events.Num());
		EventInternal.EndTime = EndTime;
		FEventPage* Page = Events.GetItemPage(EventIndex);
		Page->EndTime = FMath::Max(Page->EndTime, EndTime);

		++ModCount;
		check(Page->OpenEvents>0);
		--Page->OpenEvents;

		return EventInternal.Event;
	}

	virtual bool GetEventInfo(double InTime, double DeltaTime, int32 Depth, typename ITimeline<EventType>::FTimelineEventInfo& EventInfo) const override
	{
		//Not implemented
		check(false);
		return false;
	}

private:
	struct FEventInternal
	{
		double StartTime;
		double EndTime;
		EventType Event;

		FEventInternal() = default;
		FEventInternal(double InStartTime, double InEndTime, EventType InEvent)
			: StartTime(InStartTime)
			, EndTime(InEndTime)
			, Event(MoveTempIfPossible(InEvent))
		{ }
	};

	struct FEventPage
	{
		double BeginTime = 0.0;
		double EndTime = 0.0;
		uint64 BeginEventIndex = 0;
		uint64 EndEventIndex = 0;
		FEventInternal* Items = nullptr;
		uint64 Count = 0;
		uint64 OpenEvents = 0;
	};

	TPagedArray<FEventInternal, FEventPage> Events;
	FEventPage* CurrentPage = nullptr;
	double LastTime = 0.0;
	uint64 ModCount = 0;
};

} // namespace TraceServices
