// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/QueuedThreadPool.h"

#include "Common/PagedArray.h"
#include "Model/AsyncEnumerateTask.h"
#include "Model/MonotonicTimelineData.h"
#include "TraceServices/Containers/Timelines.h"

namespace TraceServices
{

struct FMonotonicTimelineDefaultSettings
{
	enum
	{
		MaxDepth = 1024,
		ScopeEntriesPageSize = 65536,
		EventsPageSize = 65536,
		DetailLevelsCount = 6,
	};

	constexpr static double DetailLevelResolution(int32 Index)
	{
		const double DetailLevels[DetailLevelsCount] = { 0.0, 0.0001, 0.001, 0.008, 0.04, 0.2 };
		return DetailLevels[Index];
	}
};

/*
* An interface that can consume timed serial events (a timeline).
*/
template<typename InEventType>
class IEditableTimeline
{
public:
	virtual ~IEditableTimeline() = default;

	/**
	 * Begin a new timed event.
	 *
	 * @param StartTime	The starting timestamp of the event in seconds.
	 * @param Event		The event information.
	 */
	virtual void AppendBeginEvent(double StartTime, const InEventType& Event) = 0;

	/**
	 * End a new timed event. This ends the event started by the prior call to AppendBeginEvent.
	 *
	 * @param EndTime	The ending timestamp of the event in seconds.
	 */
	virtual void AppendEndEvent(double EndTime) = 0;

	/**
	 * Gets the last time added to the timeline.
	 *
	 * @return The last timestamp added to the timeline.
	 */
	virtual double GetLastTimestamp() const { return 0.0; }
};

template<typename InEventType, typename SettingsType = FMonotonicTimelineDefaultSettings>
class TMonotonicTimeline
	: public ITimeline<InEventType>
	, public IEditableTimeline<InEventType>
{
	friend class FEnumerateAsyncTask<InEventType, SettingsType>;

	typedef FEventInfoStackEntry<InEventType, SettingsType> FEventInfoStackEntry;
	typedef FDetailLevel<InEventType, SettingsType> FDetailLevel;
	typedef FDetailLevelDepthState<InEventType> FDetailLevelDepthState;
	typedef FEnumerateAsyncTask<InEventType, SettingsType> FEnumerateAsyncTask;

public:
	using EventType = InEventType;

	TMonotonicTimeline(ILinearAllocator& InAllocator)
		: Allocator(InAllocator)
	{
		for (int32 DetailLevelIndex = 0; DetailLevelIndex < SettingsType::DetailLevelsCount; ++DetailLevelIndex)
		{
			double Resolution = SettingsType::DetailLevelResolution(DetailLevelIndex);
			DetailLevels.Emplace(Allocator, Resolution);
		}
	}

	virtual ~TMonotonicTimeline() = default;

	virtual void AppendBeginEvent(double StartTime, const EventType& Event) override
	{
		int32 CurrentDepth = DetailLevels[0].InsertionState.CurrentDepth;
		if (CurrentDepth >= SettingsType::MaxDepth)
		{
			++ExtraDepthEvents;
			return;
		}

		AddScopeEntry(DetailLevels[0], StartTime, true);
		AddEvent(DetailLevels[0], Event);

		check(CurrentDepth < SettingsType::MaxDepth);

		FDetailLevelDepthState& Lod0DepthState = DetailLevels[0].InsertionState.DepthStates[CurrentDepth];
		Lod0DepthState.EnterTime = StartTime;
		Lod0DepthState.DominatingEvent = Event;
		//Lod0DepthState.DebugDominatingEventType = Owner.EventTypes[TypeId];

		for (int32 DetailLevelIndex = 1; DetailLevelIndex < SettingsType::DetailLevelsCount; ++DetailLevelIndex)
		{
			FDetailLevel& DetailLevel = DetailLevels[DetailLevelIndex];
			FDetailLevelInsertionState<EventType, SettingsType>& InsertionState = DetailLevel.InsertionState;
			FDetailLevelDepthState& CurrentDepthState = InsertionState.DepthStates[CurrentDepth];

			if (CurrentDepthState.PendingScopeEnterIndex < 0 || StartTime >= CurrentDepthState.EnterTime + DetailLevel.Resolution)
			{
				if (CurrentDepthState.PendingEventIndex >= 0)
				{
					check(InsertionState.PendingDepth < SettingsType::MaxDepth);
					for (int32 Depth = InsertionState.PendingDepth; Depth >= CurrentDepth; --Depth)
					{
						FDetailLevelDepthState& DepthState = InsertionState.DepthStates[Depth];
						check(DepthState.PendingScopeEnterIndex >= 0);
						AddScopeEntry(DetailLevel, DepthState.ExitTime, false);

						DepthState.PendingScopeEnterIndex = -1;
						DepthState.PendingEventIndex = -1;
					}
				}
				InsertionState.PendingDepth = CurrentDepth;

				uint64 EnterScopeIndex = DetailLevel.ScopeEntries.Num();
				uint64 EventIndex = DetailLevel.Events.Num();

				AddScopeEntry(DetailLevel, StartTime, true);
				AddEvent(DetailLevel, Event);

				CurrentDepthState.DominatingEventStartTime = StartTime;
				CurrentDepthState.DominatingEventEndTime = StartTime;
				CurrentDepthState.DominatingEventDuration = 0.0;
				CurrentDepthState.PendingScopeEnterIndex = EnterScopeIndex;
				CurrentDepthState.PendingEventIndex = EventIndex;
				CurrentDepthState.EnterTime = StartTime;
				CurrentDepthState.DominatingEvent = Event;
				//CurrentDepthState.DebugDominatingEventType = Owner.EventTypes[TypeId];
			}
			else if (CurrentDepth > InsertionState.PendingDepth)
			{
				InsertionState.PendingDepth = CurrentDepth;
			}
			DetailLevel.SetEvent(CurrentDepthState.PendingEventIndex, Event);
		}

		++ModCount;
	}

	virtual void AppendEndEvent(double EndTime) override
	{
		if (ExtraDepthEvents > 0)
		{
			--ExtraDepthEvents;
			return;
		}

		check(DetailLevels[0].InsertionState.CurrentDepth <= SettingsType::MaxDepth);

		AddScopeEntry(DetailLevels[0], EndTime, false);

		int32 CurrentDepth = DetailLevels[0].InsertionState.CurrentDepth;
		check(CurrentDepth < SettingsType::MaxDepth);

		for (int32 DetailLevelIndex = 1; DetailLevelIndex < SettingsType::DetailLevelsCount; ++DetailLevelIndex)
		{
			FDetailLevel& DetailLevel = DetailLevels[DetailLevelIndex];
			FDetailLevelInsertionState<EventType, SettingsType>& InsertionState = DetailLevel.InsertionState;

			InsertionState.DepthStates[CurrentDepth].ExitTime = EndTime;

			UpdateDominatingEvent(DetailLevel, CurrentDepth, EndTime);

			FDetailLevelDepthState& CurrentDepthState = InsertionState.DepthStates[CurrentDepth];
			check(CurrentDepthState.PendingScopeEnterIndex >= 0);
			if (EndTime >= CurrentDepthState.EnterTime + DetailLevel.Resolution)
			{
				check(InsertionState.PendingDepth < SettingsType::MaxDepth);
				for (int32 Depth = InsertionState.PendingDepth; Depth >= CurrentDepth; --Depth)
				{
					FDetailLevelDepthState& DepthState = InsertionState.DepthStates[Depth];
					check(DepthState.PendingScopeEnterIndex >= 0);
					AddScopeEntry(DetailLevel, DepthState.ExitTime, false);

					DepthState.PendingScopeEnterIndex = -1;
					DepthState.PendingEventIndex = -1;
				}
				InsertionState.PendingDepth = CurrentDepth - 1;
			}
		}

		++ModCount;
	}

	virtual uint64 GetModCount() const override
	{
		return ModCount;
	}

	virtual uint64 GetEventCount() const override
	{
		return DetailLevels[0].Events.Num();
	}

	virtual const EventType& GetEvent(uint64 InIndex) const override
	{
		return DetailLevels[0].Events[InIndex];
	}

	virtual double GetStartTime() const override
	{
		return DetailLevels[0].ScopeEntries.Num() > 0 ? FMath::Abs(DetailLevels[0].ScopeEntries[0].Time) : 0.0;
	}

	virtual double GetEndTime() const override
	{
		uint64 NumScopeEntries = DetailLevels[0].ScopeEntries.Num();
		return NumScopeEntries > 0 ? FMath::Abs(DetailLevels[0].ScopeEntries[NumScopeEntries - 1].Time) : 0.0;
	}

	virtual double GetLastTimestamp() const override
	{
		return DetailLevels[0].InsertionState.LastTime;
	}

	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, typename ITimeline<EventType>::EventCallback Callback) const override
	{
		int32 DetailLevelIndex = SettingsType::DetailLevelsCount - 1;
		for (; DetailLevelIndex > 0; --DetailLevelIndex)
		{
			if (DetailLevels[DetailLevelIndex].Resolution <= Resolution)
			{
				break;
			}
		}

		const FDetailLevel& DetailLevel = DetailLevels[DetailLevelIndex];
		if (DetailLevel.ScopeEntries.Num() == 0)
		{
			return;
		}

		// The page of the first scope...
		// i.e. the previous page of the first page with Page.BeginTime > IntervalStart
		uint64 FirstScopePageIndex = Algo::UpperBoundBy(DetailLevel.ScopeEntries, IntervalStart,
			[](const FEventScopeEntryPage& Page)
			{
				return Page.BeginTime;
			});
		if (FirstScopePageIndex > 0)
		{
			--FirstScopePageIndex;
		}
		const FEventScopeEntryPage* ScopePage = DetailLevel.ScopeEntries.GetPage(FirstScopePageIndex);
		if (ScopePage->BeginTime > IntervalEnd)
		{
			return;
		}
		if (ScopePage->EndTime < IntervalStart)
		{
			return;
		}

		struct FEnumerationStackEntry
		{
			double StartTime;
			EventType Event;
		};
		FEnumerationStackEntry EventStack[SettingsType::MaxDepth];
		int32 CurrentStackDepth;

		// Make a copy of the initial stack of the page.
		CurrentStackDepth = ScopePage->InitialStackCount;
		for (int32 InitialStackIndex = 0; InitialStackIndex < CurrentStackDepth; ++InitialStackIndex)
		{
			FEnumerationStackEntry& EnumerationStackEntry = EventStack[InitialStackIndex];
			const FEventStackEntry& EventStackEntry = ScopePage->InitialStack[InitialStackIndex];
			EnumerationStackEntry.StartTime = DetailLevel.GetScopeEntryTime(EventStackEntry.EnterScopeIndex);
			EnumerationStackEntry.Event = DetailLevel.GetEvent(EventStackEntry.EventIndex);
		}

		auto ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(FirstScopePageIndex);
		const FEventScopeEntry* ScopeEntry = ScopeEntryIterator.GetCurrentItem();

		auto EventsIterator = DetailLevel.Events.GetIteratorFromItem(ScopePage->BeginEventIndex);
		const EventType* Event = EventsIterator.GetCurrentItem();

		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) < IntervalStart)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEnumerationStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.Event = *Event;
				StackEntry.StartTime = -ScopeEntry->Time;
				Event = EventsIterator.NextItem();
			}
			else // "end" event
			{
				check(CurrentStackDepth > 0);
				--CurrentStackDepth;
			}

			ScopeEntry = ScopeEntryIterator.NextItem();
		}
		if (CurrentStackDepth == 1 && EventStack[0].StartTime > IntervalEnd)
		{
			return;
		}

		for (int32 StackIndex = 0; StackIndex < CurrentStackDepth; ++StackIndex)
		{
			FEnumerationStackEntry& StackEntry = EventStack[StackIndex];
			if (Callback(true, StackEntry.StartTime, StackEntry.Event) == EEventEnumerate::Stop)
			{
				return;
			}
		}
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) <= IntervalEnd)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEnumerationStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.Event = *Event;
				if (Callback(true, -ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}
				Event = EventsIterator.NextItem();
			}
			else // "end" event
			{
				check(CurrentStackDepth > 0);
				FEnumerationStackEntry& StackEntry = EventStack[--CurrentStackDepth];
				if (Callback(false, ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}
			}

			ScopeEntry = ScopeEntryIterator.NextItem();
		}

		bool bSearchEndTimeUsingPages = false;
		uint64 LastPageIndex = ScopeEntryIterator.GetCurrentPageIndex();
		uint32 ExitDepth = 0;
		while (CurrentStackDepth > 0 && ScopeEntry)
		{
			if (ScopeEntryIterator.GetCurrentPageIndex() != LastPageIndex)
			{
				bSearchEndTimeUsingPages = true;
				break;
			}

			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				++ExitDepth;
			}
			else // "end" event
			{
				if (ExitDepth == 0)
				{
					FEnumerationStackEntry& StackEntry = EventStack[--CurrentStackDepth];
					if (Callback(false, ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
					{
						return;
					}
				}
				else
				{
					--ExitDepth;
				}
			}

			LastPageIndex = ScopeEntryIterator.GetCurrentPageIndex();
			ScopeEntry = ScopeEntryIterator.NextItem();
		}

		if (bSearchEndTimeUsingPages)
		{
			const FEventScopeEntryPage* CurrentScopePage = ScopeEntryIterator.GetCurrentPage();
			do
			{
				check(CurrentStackDepth <= CurrentScopePage->InitialStackCount);
				while (CurrentStackDepth > 0 && CurrentScopePage->InitialStack[CurrentStackDepth - 1].EndTime > 0.0)
				{
					--CurrentStackDepth;
					EventType CurrentEvent = DetailLevel.GetEvent(CurrentScopePage->InitialStack[CurrentStackDepth].EventIndex);
					if (Callback(false, CurrentScopePage->InitialStack[CurrentStackDepth].EndTime, CurrentEvent) == EEventEnumerate::Stop)
					{
						return;
					}
				}
				CurrentScopePage = ScopeEntryIterator.NextPage();
			}
			while (CurrentScopePage != nullptr);
		}

		while (CurrentStackDepth > 0)
		{
			FEnumerationStackEntry& StackEntry = EventStack[--CurrentStackDepth];
			if (Callback(false, DetailLevel.InsertionState.LastTime, StackEntry.Event) == EEventEnumerate::Stop)
			{
				return;
			}
		}
	}

	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, typename ITimeline<EventType>::EventRangeCallback Callback) const override
	{
		struct FStackEntry
		{
			uint64 LocalEventIndex;
		};
		FStackEntry EventStack[SettingsType::MaxDepth];
		uint32 CurrentDepth = 0;

		struct FOutputEvent
		{
			double StartTime;
			double EndTime;
			uint32 Depth;
			EventType Event;
		};
		TArray<FOutputEvent> OutputEvents;

		EnumerateEventsDownSampled(IntervalStart, IntervalEnd, Resolution,
			[&EventStack, &OutputEvents, &CurrentDepth, Callback]
			(bool IsEnter, double Time, const EventType& Event)
			{
				if (IsEnter)
				{
					check(CurrentDepth < SettingsType::MaxDepth);
					FStackEntry& StackEntry = EventStack[CurrentDepth];
					StackEntry.LocalEventIndex = OutputEvents.Num();
					FOutputEvent& OutputEvent = OutputEvents.AddDefaulted_GetRef();
					OutputEvent.StartTime = Time;
					OutputEvent.EndTime = Time;
					OutputEvent.Depth = CurrentDepth;
					OutputEvent.Event = Event;
					++CurrentDepth;
				}
				else
				{
					{
						check(CurrentDepth > 0);
						FStackEntry& StackEntry = EventStack[--CurrentDepth];
						FOutputEvent* OutputEvent = OutputEvents.GetData() + StackEntry.LocalEventIndex;
						OutputEvent->EndTime = Time;
					}
					if (CurrentDepth == 0)
					{
						for (FOutputEvent& OutputEvent : OutputEvents)
						{
							EEventEnumerate Ret = Callback(OutputEvent.StartTime, OutputEvent.EndTime, OutputEvent.Depth, OutputEvent.Event);
							if (Ret != EEventEnumerate::Continue)
							{
								return Ret;
							}
						}
						OutputEvents.Empty(OutputEvents.Num());
					}
				}

				return EEventEnumerate::Continue;
			});
	}

	virtual void EnumerateEventsBackwardsDownSampled(double IntervalEnd, double IntervalStart, double Resolution, typename ITimeline<EventType>::EventCallback Callback) const override
	{
		int32 DetailLevelIndex = SettingsType::DetailLevelsCount - 1;
		for (; DetailLevelIndex > 0; --DetailLevelIndex)
		{
			if (DetailLevels[DetailLevelIndex].Resolution <= Resolution)
			{
				break;
			}
		}

		const FDetailLevel& DetailLevel = DetailLevels[DetailLevelIndex];
		if (DetailLevel.ScopeEntries.Num() == 0)
		{
			return;
		}

		// The page of the last scope...
		// i.e. the first page with Page.BeginTime > IntervalEnd
		uint64 LastScopePageIndex = Algo::UpperBoundBy(DetailLevel.ScopeEntries, IntervalEnd,
			[](const FEventScopeEntryPage& Page)
			{
				return Page.BeginTime;
			});

		struct FEnumerationStackEntry
		{
			double EndTime;
			EventType Event;
		};
		FEnumerationStackEntry EventStack[SettingsType::MaxDepth];
		int32 CurrentStackDepth = 0;

		const FEventScopeEntry* ScopeEntry = nullptr;

		// By default, we start from the very end of the session.
		auto ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromItem(DetailLevel.ScopeEntries.Num() - 1);
		auto EventsIterator = DetailLevel.Events.GetIteratorFromItem(DetailLevel.Events.Num() - 1);

		if (LastScopePageIndex > 0 && LastScopePageIndex < DetailLevel.ScopeEntries.NumPages())
		{
			// If we have a page we can start from, start enumerating backwards from the beginning of that page.
			ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(LastScopePageIndex);
			const FEventScopeEntryPage* ScopePage = DetailLevel.ScopeEntries.GetPage(LastScopePageIndex);

			EventsIterator = DetailLevel.Events.GetIteratorFromItem(ScopePage->BeginEventIndex);
			CurrentStackDepth = ScopePage->InitialStackCount;
			for (int32 InitialStackIndex = 0; InitialStackIndex < CurrentStackDepth; ++InitialStackIndex)
			{
				FEnumerationStackEntry& EnumerationStackEntry = EventStack[InitialStackIndex];
				const FEventStackEntry& EventStackEntry = ScopePage->InitialStack[InitialStackIndex];
				EnumerationStackEntry.EndTime = EventStackEntry.EndTime;
				if (EnumerationStackEntry.EndTime < 0.0)
				{
					// We need to search for the EndTime of the event using pages.
					auto PageIterator = ScopeEntryIterator;
					while (const FEventScopeEntryPage* Page = PageIterator.NextPage())
					{
						if (Page->InitialStack[InitialStackIndex].EndTime > 0.0)
						{
							EnumerationStackEntry.EndTime = Page->InitialStack[InitialStackIndex].EndTime;
							break;
						}
					}
					if (EnumerationStackEntry.EndTime < 0.0)
					{
						EnumerationStackEntry.EndTime = DetailLevel.InsertionState.LastTime;
					}
				}
				EnumerationStackEntry.Event = DetailLevel.GetEvent(EventStackEntry.EventIndex);
			}
			// We start enumerating from the previous page.
			ScopeEntry = ScopeEntryIterator.PrevItem();
			EventsIterator.PrevItem();
		}
		else
		{
			// If we start from the end of the session, we use InsertionState as the initial stack.
			const FDetailLevelInsertionState<EventType, SettingsType>& InsertionState = DetailLevel.InsertionState;
			CurrentStackDepth = InsertionState.CurrentDepth;
			for (int32 InitialStackIndex = 0; InitialStackIndex < CurrentStackDepth; ++InitialStackIndex)
			{
				FEnumerationStackEntry& EnumerationStackEntry = EventStack[InitialStackIndex];
				const FEventStackEntry& EventStackEntry = InsertionState.EventStack[InitialStackIndex];
				EnumerationStackEntry.EndTime = InsertionState.LastTime;
				EnumerationStackEntry.Event = DetailLevel.GetEvent(EventStackEntry.EventIndex);
			}
			ScopeEntry = ScopeEntryIterator.GetCurrentItem();
		}

		// Enumerate backwards until we reach IntervalEnd, without calling the Callback because these events are not in the provided interval.
		const EventType* Event = EventsIterator.GetCurrentItem();
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) > IntervalEnd)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth > 0);
				--CurrentStackDepth;
				Event = EventsIterator.PrevItem();
			}
			else // "end" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEnumerationStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.EndTime = ScopeEntry->Time;
			}

			ScopeEntry = ScopeEntryIterator.PrevItem();
		}

		// Call the callback for the events that are open at IntervalEnd.
		for (int32 StackIndex = 0; StackIndex < CurrentStackDepth; ++StackIndex)
		{
			FEnumerationStackEntry& StackEntry = EventStack[StackIndex];
			if (Callback(true, StackEntry.EndTime, StackEntry.Event) == EEventEnumerate::Stop)
			{
				return;
			}
		}

		// Enumerate backwards between IntervalEnd and IntervalStart.
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) >= IntervalStart)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth > 0);
				FEnumerationStackEntry& StackEntry = EventStack[--CurrentStackDepth];
				StackEntry.Event = *Event;
				if (Callback(false, -ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}
				Event = EventsIterator.PrevItem();
			}
			else // "end" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEnumerationStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.EndTime = ScopeEntry->Time;

				if (Callback(true, ScopeEntry->Time, StackEntry.Event) == EEventEnumerate::Stop)
				{
					return;
				}
			}

			ScopeEntry = ScopeEntryIterator.PrevItem();
		}

		// Find the StartTime of the events that are open at IntervalStart.
		bool bSearchStartTimeUsingPages = false;
		uint64 LastPageIndex = ScopeEntryIterator.GetCurrentPageIndex();
		uint32 ExitDepth = 0;
		while (CurrentStackDepth > 0 && ScopeEntry)
		{
			if (ScopeEntryIterator.GetCurrentPageIndex() != LastPageIndex)
			{
				bSearchStartTimeUsingPages = true;
				break;
			}
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				if (ExitDepth == 0)
				{
					FEnumerationStackEntry& StackEntry = EventStack[--CurrentStackDepth];
					if (Callback(false, -ScopeEntry->Time, *EventsIterator.GetCurrentItem()) == EEventEnumerate::Stop)
					{
						return;
					}
				}
				else
				{
					--ExitDepth;
				}
				EventsIterator.PrevItem();
			}
			else // "end" event
			{
				++ExitDepth;
			}

			LastPageIndex = ScopeEntryIterator.GetCurrentPageIndex();
			ScopeEntry = ScopeEntryIterator.PrevItem();
		}

		if (bSearchStartTimeUsingPages)
		{
			ScopeEntryIterator.NextPage();
			const FEventScopeEntryPage* CurrentScopePage = ScopeEntryIterator.GetCurrentPage();
			check(CurrentStackDepth <= CurrentScopePage->InitialStackCount);

			while (CurrentStackDepth > 0)
			{
				--CurrentStackDepth;
				EventType CurrentEvent = DetailLevel.GetEvent(CurrentScopePage->InitialStack[CurrentStackDepth].EventIndex);
				double StartTime = DetailLevel.ScopeEntries[CurrentScopePage->InitialStack[CurrentStackDepth].EnterScopeIndex].Time;
				if (Callback(false, StartTime, CurrentEvent) == EEventEnumerate::Stop)
				{
					return;
				}
			}
		}
	}

	virtual void EnumerateEventsBackwardsDownSampled(double IntervalEnd, double IntervalStart, double Resolution, typename ITimeline<EventType>::EventRangeCallback Callback) const override
	{
		struct FStackEntry
		{
			double EndTime;
			EventType Event;
		};
		FStackEntry EventStack[SettingsType::MaxDepth];
		uint32 CurrentDepth = 0;

		EnumerateEventsBackwardsDownSampled(IntervalEnd, IntervalStart, Resolution,
			[&EventStack, &CurrentDepth, Callback]
			(bool IsEnter, double Time, const EventType& Event)
			{
				if (IsEnter)
				{
					FStackEntry& StackEntry = EventStack[CurrentDepth];
					StackEntry.Event = Event;
					StackEntry.EndTime = Time;
					++CurrentDepth;
				}
				else
				{
					FStackEntry& StackEntry = EventStack[--CurrentDepth];
					EEventEnumerate Ret = Callback(Time, StackEntry.EndTime, CurrentDepth, Event);
					if (Ret != EEventEnumerate::Continue)
					{
						return Ret;
					}
				}

				return EEventEnumerate::Continue;
			});
	}

	virtual void EnumerateEventsDownSampledAsync(const typename ITimeline<EventType>::EnumerateAsyncParams& EnumerateAsyncParams) const override
	{
		// Exactly one of these 2 callbacks must be set.
		check((EnumerateAsyncParams.EventRangeCallback == nullptr) ^ (EnumerateAsyncParams.EventCallback == nullptr));

		if (EnumerateAsyncParams.IntervalEnd < EnumerateAsyncParams.IntervalStart)
		{
			return;
		}

		int32 DetailLevelIndex = SettingsType::DetailLevelsCount - 1;
		for (; DetailLevelIndex > 0; --DetailLevelIndex)
		{
			if (DetailLevels[DetailLevelIndex].Resolution <= EnumerateAsyncParams.Resolution)
			{
				break;
			}
		}

		const FDetailLevel& DetailLevel = DetailLevels[DetailLevelIndex];
		if (DetailLevel.ScopeEntries.Num() == 0)
		{
			return;
		}

		// The page of the first scope...
		// i.e. the previous page of the first page with Page.BeginTime > EnumerateAsyncParams.IntervalStart
		uint64 FirstScopePageIndex = Algo::UpperBoundBy(DetailLevel.ScopeEntries, EnumerateAsyncParams.IntervalStart,
			[](const FEventScopeEntryPage& Page)
			{
				return Page.BeginTime;
			});
		if (FirstScopePageIndex > 0)
		{
			--FirstScopePageIndex;
		}

		// The page of the last scope...
		// i.e. the first page with Page.BeginTime > EnumerateAsyncParams.IntervalEnd
		uint64 LastScopePageIndex = Algo::UpperBoundBy(DetailLevel.ScopeEntries, EnumerateAsyncParams.IntervalEnd,
			[](const FEventScopeEntryPage& Page)
			{
				return Page.BeginTime;
			});

		TArray<TSharedRef<FAsyncTask<FEnumerateAsyncTask>>> WorkerTasks;

		check(LastScopePageIndex >= FirstScopePageIndex);
		uint32 NumPages = FMath::Max(static_cast<uint32>(LastScopePageIndex - FirstScopePageIndex), 1u);
		uint32 NumThreads = GThreadPool->GetNumThreads();

		if (EnumerateAsyncParams.MaxOccupancy > 0.0)
		{
			NumThreads = FMath::Max(static_cast<uint32>(NumThreads * EnumerateAsyncParams.MaxOccupancy), 1u);
		}

		uint32 NumTasks = FMath::Min(NumThreads, NumPages);

		uint32 PagesPerTask = NumPages / NumTasks;
		uint32 RemainingPages = NumPages % NumTasks; // The remaining pages will be split between the first tasks

		EnumerateAsyncParams.SetupCallback(NumTasks);

		for (uint32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			uint32 NrPagesToProcess = TaskIndex < RemainingPages ? PagesPerTask + 1 : PagesPerTask;

			FAsyncEnumerateTaskData<EventType, SettingsType> TaskData;
			TaskData.TaskIndex = TaskIndex;
			TaskData.NumTasks = NumTasks;
			TaskData.StartPageIndex = FirstScopePageIndex;
			TaskData.EndPageIndex = FMath::Min(LastScopePageIndex, FirstScopePageIndex + NrPagesToProcess);
			TaskData.StartTime = EnumerateAsyncParams.IntervalStart;
			TaskData.EndTime = EnumerateAsyncParams.IntervalEnd;
			TaskData.SortOrder = EnumerateAsyncParams.SortOrder;
			TaskData.DetailLevel = &DetailLevel;
			TaskData.EventCallback = EnumerateAsyncParams.EventCallback;
			TaskData.EventRangeCallback = EnumerateAsyncParams.EventRangeCallback;

			TSharedRef<FAsyncTask<FEnumerateAsyncTask>> AsyncTask = MakeShared<FAsyncTask<FEnumerateAsyncTask>>(TaskData);
			WorkerTasks.Add(AsyncTask);
			AsyncTask->StartBackgroundTask();

			FirstScopePageIndex += NrPagesToProcess;
		}

		for (uint32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			WorkerTasks[TaskIndex]->EnsureCompletion();
		}
	}

	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, typename ITimeline<EventType>::EventCallback Callback) const override
	{
		EnumerateEventsDownSampled(IntervalStart, IntervalEnd, 0.0, Callback);
	}

	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, typename ITimeline<EventType>::EventRangeCallback Callback) const override
	{
		EnumerateEventsDownSampled(IntervalStart, IntervalEnd, 0.0, Callback);
	}

	virtual void EnumerateEventsBackwards(double IntervalEnd, double IntervalStart, typename ITimeline<EventType>::EventCallback Callback) const override
	{
		EnumerateEventsBackwardsDownSampled(IntervalEnd, IntervalStart, 0.0, Callback);
	}

	virtual void EnumerateEventsBackwards(double IntervalEnd, double IntervalStart, typename ITimeline<EventType>::EventRangeCallback Callback) const override
	{
		EnumerateEventsBackwardsDownSampled(IntervalEnd, IntervalStart, 0.0, Callback);
	}

	virtual bool GetEventInfo(double InTime, double DeltaTime, int32 Depth, typename ITimeline<InEventType>::FTimelineEventInfo& EventInfo) const override
	{
		if (Depth >= SettingsType::MaxDepth || Depth < 0)
		{
			return false;
		}

		const FDetailLevel& DetailLevel = DetailLevels[0];
		if (DetailLevel.ScopeEntries.Num() == 0)
		{
			return false;
		}

		if (DetailLevel.InsertionState.LastTime < InTime - DeltaTime)
		{
			return false;
		}

		// The page of the first scope...
		// i.e. the previous page of the first page with Page.BeginTime > InTime
		uint64 FirstScopePageIndex = Algo::UpperBoundBy(DetailLevel.ScopeEntries, InTime,
			[](const FEventScopeEntryPage& Page)
			{
				return Page.BeginTime;
			});
		if (FirstScopePageIndex > 0)
		{
			--FirstScopePageIndex;
		}

		auto ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(FirstScopePageIndex);

		FEventStackEntry OutScopeEntry;
		bool bIsFound = FindEventUsingPageInitialStack(ScopeEntryIterator, InTime, DeltaTime, Depth, DetailLevel, OutScopeEntry);
		if (bIsFound)
		{
			EventInfo.StartTime = DetailLevel.GetScopeEntryTime(OutScopeEntry.EnterScopeIndex);
			EventInfo.ExclTime = OutScopeEntry.ExclTime;
			EventInfo.EndTime = OutScopeEntry.EndTime;
			EventInfo.Event = DetailLevel.GetEvent(OutScopeEntry.EventIndex);

			if (EventInfo.EndTime < 0.0)
			{
				// The end of the event has not been reached by analysis.
				EventInfo.EndTime = DetailLevel.InsertionState.LastTime;
			}

			return true;
		}

		const FEventScopeEntryPage* ScopePage = ScopeEntryIterator.GetCurrentPage();
		double IntervalStart = FMath::Max(InTime - DeltaTime, 0.0);

		while (ScopePage->BeginTime > IntervalStart && FirstScopePageIndex > 0)
		{
			--FirstScopePageIndex;
			ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(FirstScopePageIndex);
			ScopePage = DetailLevel.ScopeEntries.GetPage(FirstScopePageIndex);
		}

		auto EventsIterator = DetailLevel.Events.GetIteratorFromItem(ScopePage->BeginEventIndex);
		FEventInfoStackEntry EventStack[SettingsType::MaxDepth];

		int32 CurrentStackDepth = ScopePage->InitialStackCount;
		for (int32 InitialStackIndex = 0; InitialStackIndex < CurrentStackDepth; ++InitialStackIndex)
		{
			EventStack[InitialStackIndex] = FEventInfoStackEntry(ScopePage->InitialStack[InitialStackIndex], DetailLevel);
		}

		const FEventScopeEntry* ScopeEntry = ScopeEntryIterator.GetCurrentItem();
		const EventType* Event = EventsIterator.GetCurrentItem();
		double LastTime = 0.0;
		auto LastTimeIterator = ScopeEntryIterator;
		if (LastTimeIterator.PrevItem())
		{
			LastTime = abs(LastTimeIterator.GetCurrentItem()->Time);
		}

		// Iterate from the start of the page to the start time of our interval.
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) <= IntervalStart)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEventInfoStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.Event = *Event;
				StackEntry.StartTime = -ScopeEntry->Time;
				StackEntry.ExclTime = 0.0;

				if (CurrentStackDepth > 1)
				{
					FEventInfoStackEntry& ParentStackEntry = EventStack[CurrentStackDepth - 2];
					ParentStackEntry.ExclTime += StackEntry.StartTime - LastTime;
				}

				Event = EventsIterator.NextItem();
				LastTime = -ScopeEntry->Time;
			}
			else // "end" event
			{
				check(CurrentStackDepth > 0);
				--CurrentStackDepth;

				if (CurrentStackDepth == Depth)
				{
					FEventInfoStackEntry& StackEntry = EventStack[CurrentStackDepth];
					StackEntry.ExclTime += ScopeEntry->Time - LastTime;
				}

				LastTime = ScopeEntry->Time;
			}

			ScopeEntry = ScopeEntryIterator.NextItem();
		}

		// Check if we have an event between InTime - InDeltaTime and InTime + InDeltaTime.
		FFindBestMatchEventInParams InParams;
		InParams.IterationState.ScopeIterator = ScopeEntryIterator;
		InParams.IterationState.EventsIterator = EventsIterator;
		InParams.IterationState.EventStack = EventStack;
		InParams.IterationState.StackDepth = CurrentStackDepth;
		InParams.IterationState.LastIterationTime = LastTime;

		InParams.TargetExactTime = InTime;
		InParams.TargetEndTime = InTime + DeltaTime;
		InParams.TargetDepth = Depth;

		FFindBestMatchEventOutParams OutParams;

		bool bMatchFound = FindBestMatchEvent(InParams, OutParams);
		if (!bMatchFound)
		{
			return false;
		}

		bool bMatchEventStartsInsidePage = true;
		if (ScopePage->InitialStackCount > Depth)
		{
			double StartTime = DetailLevel.GetScopeEntryTime(ScopePage->InitialStack[Depth].EnterScopeIndex);
			if (StartTime == OutParams.EventInfo.StartTime)
			{
				bMatchEventStartsInsidePage = false;
			}
		}

		// If we have found both start time and end time for our event, we can return the result.
		if (OutParams.bHasEndTime && bMatchEventStartsInsidePage)
		{
			EventInfo.StartTime = OutParams.EventInfo.StartTime;
			EventInfo.EndTime = OutParams.EndTime;
			EventInfo.ExclTime = OutParams.EventInfo.ExclTime;
			EventInfo.Event = OutParams.EventInfo.Event;

			if (EventInfo.EndTime < 0.0)
			{
				// The end of the event has not been reached by analysis.
				EventInfo.EndTime = DetailLevel.InsertionState.LastTime;
			}

			return true;
		}

		// We continue searching for the end scope event.
		ScopeEntryIterator = OutParams.IterationState.ScopeIterator;
		EventsIterator = OutParams.IterationState.EventsIterator;
		CurrentStackDepth = OutParams.IterationState.StackDepth;
		LastTime = OutParams.IterationState.LastIterationTime;

		FEventInfoStackEntry TargetEntry = OutParams.EventInfo;

		auto ScopeEntryIteratorAtEvent = ScopeEntryIterator;

		// We find the page where the target event ends.
		ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(FirstScopePageIndex);
		auto EventLastPageIterator = ScopeEntryIterator;

		while (const FEventScopeEntryPage* CurrentScopePage = ScopeEntryIterator.NextPage())
		{
			if (CurrentScopePage->InitialStackCount <= Depth)
			{
				break;
			}

			double StartTime = DetailLevel.GetScopeEntryTime(CurrentScopePage->InitialStack[Depth].EnterScopeIndex);
			if (StartTime != TargetEntry.StartTime)
			{
				break;
			}

			EventLastPageIterator = ScopeEntryIterator;
		}

		if (EventLastPageIterator.GetCurrentPageIndex() != FirstScopePageIndex || !bMatchEventStartsInsidePage)
		{
			// If the end scope event is on a different page than the start scope one we can get the event info from the InitialStack of the last page.
			ScopeEntryIterator = EventLastPageIterator;
			ScopePage = ScopeEntryIterator.GetCurrentPage();

			check(ScopePage->InitialStackCount > Depth);
			FEventStackEntry& TargetStackEntry = ScopePage->InitialStack[Depth];
			EventInfo.StartTime = DetailLevel.GetScopeEntryTime(TargetStackEntry.EnterScopeIndex);
			EventInfo.EndTime = TargetStackEntry.EndTime;
			EventInfo.ExclTime = TargetStackEntry.ExclTime;
			EventInfo.Event = DetailLevel.GetEvent(TargetStackEntry.EventIndex);

			if (EventInfo.EndTime < 0.0)
			{
				// The end of the event has not been reached by analysis.
				EventInfo.EndTime = DetailLevel.InsertionState.LastTime;
			}

			return true;
		}
		else
		{
			// If the end scope event is in the same page, we continue iterating from the point where FindBestMatchEvent stopped.
			ScopeEntryIterator = ScopeEntryIteratorAtEvent;
			ScopeEntry = ScopeEntryIterator.GetCurrentItem();
		}

		while (ScopeEntry && Depth < CurrentStackDepth)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(CurrentStackDepth < SettingsType::MaxDepth);
				FEventInfoStackEntry& StackEntry = EventStack[CurrentStackDepth++];
				StackEntry.Event = *Event;
				StackEntry.StartTime = -ScopeEntry->Time;

				//if (ensure(CurrentStackDepth > 1))
				{
					FEventInfoStackEntry& ParentStackEntry = EventStack[CurrentStackDepth - 2];
					ParentStackEntry.ExclTime += StackEntry.StartTime - LastTime;
				}

				Event = EventsIterator.NextItem();
				LastTime = -ScopeEntry->Time;
			}
			else // "end" event
			{
				check(CurrentStackDepth > 0);
				--CurrentStackDepth;

				if (CurrentStackDepth == Depth)
				{
					FEventInfoStackEntry& StackEntry = EventStack[CurrentStackDepth];
					StackEntry.ExclTime += ScopeEntry->Time - LastTime;
				}

				LastTime = ScopeEntry->Time;
			}

			ScopeEntry = ScopeEntryIterator.NextItem();
		}

		TargetEntry = EventStack[Depth];

		EventInfo.StartTime = TargetEntry.StartTime;
		EventInfo.EndTime = LastTime;
		EventInfo.ExclTime = TargetEntry.ExclTime;
		EventInfo.Event = TargetEntry.Event;

		return true;
	}

	virtual int32 GetDepthAt(double Time) const override
	{
		const FDetailLevel& DetailLevel = DetailLevels[0];
		if (DetailLevel.ScopeEntries.Num() == 0)
		{
			return 0;
		}

		// The page of the first scope...
		// i.e. the previous page of the first page with Page.BeginTime > Time
		uint64 FirstScopePageIndex = Algo::UpperBoundBy(DetailLevel.ScopeEntries, Time,
			[](const FEventScopeEntryPage& Page)
			{
				return Page.BeginTime;
			});
		if (FirstScopePageIndex > 0)
		{
			--FirstScopePageIndex;
		}

		else
		{
			return 0;
		}
		const FEventScopeEntryPage* ScopePage = DetailLevel.ScopeEntries.GetPage(FirstScopePageIndex);
		check(ScopePage->BeginTime <= Time);

		int32 CurrentStackDepth = ScopePage->InitialStackCount;

		auto ScopeEntryIterator = DetailLevel.ScopeEntries.GetIteratorFromPage(FirstScopePageIndex);
		const FEventScopeEntry* ScopeEntry = ScopeEntryIterator.GetCurrentItem();
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) < Time)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				CurrentStackDepth++;
			}
			else // "end" event
			{
				check(CurrentStackDepth > 0);
				--CurrentStackDepth;
			}
			ScopeEntry = ScopeEntryIterator.NextItem();
		}

		return CurrentStackDepth;
	}

private:

	struct FIterationState
	{
		typename TPagedArray<FEventScopeEntry, FEventScopeEntryPage>::TIterator ScopeIterator;
		typename TPagedArray<EventType>::TIterator EventsIterator;
		FEventInfoStackEntry* EventStack;
		int32 StackDepth;
		double LastIterationTime;
	};

	struct FFindBestMatchEventInParams
	{
		FIterationState IterationState;

		double TargetExactTime;
		double TargetEndTime;
		int32 TargetDepth;
	};

	struct FFindBestMatchEventOutParams
	{
		FIterationState IterationState;

		FEventInfoStackEntry EventInfo;
		bool bHasEndTime;
		double EndTime;
	};

	void UpdateDominatingEvent(FDetailLevel& DetailLevel, int32 Depth, double CurrentTime)
	{
		FDetailLevelDepthState& Lod0DepthState = DetailLevels[0].InsertionState.DepthStates[Depth];
		double Lod0EventDuration = CurrentTime - Lod0DepthState.EnterTime;
		FDetailLevelDepthState& CurrentDepthState = DetailLevel.InsertionState.DepthStates[Depth];
		if (Lod0EventDuration > CurrentDepthState.DominatingEventDuration)
		{
			check(CurrentDepthState.PendingScopeEnterIndex >= 0);
			check(CurrentDepthState.PendingEventIndex >= 0);

			CurrentDepthState.DominatingEvent = Lod0DepthState.DominatingEvent;
			CurrentDepthState.DominatingEventStartTime = Lod0DepthState.EnterTime;
			CurrentDepthState.DominatingEventEndTime = CurrentTime;
			CurrentDepthState.DominatingEventDuration = Lod0EventDuration;

			DetailLevel.SetEvent(CurrentDepthState.PendingEventIndex, CurrentDepthState.DominatingEvent);

			//CurrentDepthState.DebugDominatingEventType = Owner.EventTypes[CurrentDepthState.DominatingEventType];
		}
	}

	void AddScopeEntry(FDetailLevel& DetailLevel, double Time, bool IsEnter)
	{
		FDetailLevelInsertionState<EventType, SettingsType>& InsertionState = DetailLevel.InsertionState;
		checkf(Time >= InsertionState.LastTime, TEXT("Time=%.9f LastTime=%.9f"), Time, InsertionState.LastTime);

		uint64 EventIndex = DetailLevel.Events.Num();
		uint64 ScopeIndex = DetailLevel.ScopeEntries.Num();

		FEventScopeEntry& ScopeEntry = DetailLevel.ScopeEntries.PushBack();
		ScopeEntry.Time = IsEnter ? -Time : Time;
		FEventScopeEntryPage* LastPage = DetailLevel.ScopeEntries.GetLastPage();
		uint64 LastPageIndex = DetailLevel.ScopeEntries.NumPages() - 1;
		if (LastPageIndex != InsertionState.CurrentScopeEntryPageIndex)
		{
			// At the very first call, CurrentScopeEntryPage will be -1
			if (InsertionState.CurrentScopeEntryPageIndex != (uint64) -1)
			{
				check(InsertionState.CurrentDepth <= SettingsType::MaxDepth);

				FEventScopeEntryPage* CurrentScopeEntryPage = DetailLevel.ScopeEntries.GetPage(InsertionState.CurrentScopeEntryPageIndex);
				const int32 MaxEntryIndex = FMath::Min(CurrentScopeEntryPage->InitialStackCount, InsertionState.CurrentDepth);

				// Update the open scopes that were also open at the beginning of the last page so the values
				// represent stats up to and including the current page
				for (int32 EntryIndex = 0; EntryIndex < MaxEntryIndex; ++EntryIndex)
				{
					FEventStackEntry& InsertionStateStackEntry = InsertionState.EventStack[EntryIndex];
					FEventStackEntry& PreviousPageStackEntry = CurrentScopeEntryPage->InitialStack[EntryIndex];

					if (InsertionStateStackEntry.EnterScopeIndex == PreviousPageStackEntry.EnterScopeIndex &&
						InsertionStateStackEntry.EventIndex == PreviousPageStackEntry.EventIndex)
					{
						PreviousPageStackEntry.ExclTime = InsertionStateStackEntry.ExclTime;
					}
					else
					{
						break;
					}
				}
			}

			InsertionState.CurrentScopeEntryPageIndex = LastPageIndex;
			LastPage->BeginTime = Time;
			LastPage->BeginEventIndex = DetailLevel.Events.Num();
			LastPage->EndEventIndex = LastPage->BeginEventIndex;
			LastPage->InitialStackCount = InsertionState.CurrentDepth;
			if (LastPage->InitialStackCount)
			{
				LastPage->InitialStack = reinterpret_cast<FEventStackEntry*>(Allocator.Allocate(LastPage->InitialStackCount * sizeof(FEventStackEntry)));
				memcpy(LastPage->InitialStack, InsertionState.EventStack, LastPage->InitialStackCount * sizeof(FEventStackEntry));
			}
		}
		LastPage->EndTime = Time;

		if (IsEnter)
		{
			check(InsertionState.CurrentDepth < SettingsType::MaxDepth);
			FEventStackEntry& StackEntry = InsertionState.EventStack[InsertionState.CurrentDepth++];
			StackEntry.EventIndex = EventIndex;
			StackEntry.EnterScopeIndex = ScopeIndex;
			StackEntry.ExclTime = 0.0;
			StackEntry.EndTime = -1.0;

			if (InsertionState.CurrentDepth > 1)
			{
				FEventStackEntry& ParentStackEntry = InsertionState.EventStack[InsertionState.CurrentDepth - 2];
				ParentStackEntry.ExclTime += Time - InsertionState.LastTime;
			}
		}
		else
		{
			check(InsertionState.CurrentDepth > 0);
			--InsertionState.CurrentDepth;

			FEventStackEntry& StackEntry = InsertionState.EventStack[InsertionState.CurrentDepth];
			StackEntry.ExclTime += Time - InsertionState.LastTime;

			FEventScopeEntryPage* CurrentScopeEntryPage = DetailLevel.ScopeEntries.GetPage(InsertionState.CurrentScopeEntryPageIndex);
			if (InsertionState.CurrentDepth < CurrentScopeEntryPage->InitialStackCount)
			{
				FEventStackEntry& PreviousPageStackEntry = CurrentScopeEntryPage->InitialStack[InsertionState.CurrentDepth];

				if (StackEntry.EnterScopeIndex == PreviousPageStackEntry.EnterScopeIndex &&
					StackEntry.EventIndex == PreviousPageStackEntry.EventIndex)
				{
					PreviousPageStackEntry.ExclTime = StackEntry.ExclTime;
					PreviousPageStackEntry.EndTime = Time;
				}
			}
		}

		InsertionState.LastTime = Time;

		//ScopeEntry.DebugDepth = InsertionState.CurrentDepth;
	}

	void AddEvent(FDetailLevel& DetailLevel, const EventType& Event)
	{
		FEventScopeEntryPage* CurrentScopeEntryPage = DetailLevel.ScopeEntries.GetPage(DetailLevel.InsertionState.CurrentScopeEntryPageIndex);
		++CurrentScopeEntryPage->EndEventIndex;
		DetailLevel.Events.PushBack() = Event;

		//Event.DebugType = Owner.EventTypes[TypeIndex];
	}

	bool FindEventUsingPageInitialStack(typename TPagedArray<FEventScopeEntry, FEventScopeEntryPage>::TIterator ScopeEntryIterator,
										double Time,
										double DeltaTime,
										int32 Depth,
										const FDetailLevel& DetailLevel,
										FEventStackEntry& OutPageStackEntry) const
	{
		const FEventScopeEntryPage* CurrentScopePage = ScopeEntryIterator.GetCurrentPage();
		bool bIsInCurrentPageInitStack = false;

		if (CurrentScopePage->InitialStackCount > Depth)
		{
			FEventStackEntry& CurrentPageStackEntry = CurrentScopePage->InitialStack[Depth];
			if (CurrentPageStackEntry.EndTime < 0.0 || CurrentPageStackEntry.EndTime > Time)
			{
				bIsInCurrentPageInitStack = true;
			}
		}

		if (!bIsInCurrentPageInitStack)
		{
			const FEventScopeEntryPage* NextScopePage = ScopeEntryIterator.NextPage();
			if (NextScopePage != nullptr &&
				NextScopePage->InitialStackCount > Depth)
			{
				FEventStackEntry& NextPageStackEntry = NextScopePage->InitialStack[Depth];
				double StartTime = DetailLevel.GetScopeEntryTime(NextPageStackEntry.EnterScopeIndex);
				if (StartTime < Time)
				{
					CurrentScopePage = NextScopePage;
					bIsInCurrentPageInitStack = true;
				}
			}
		}

		if (!bIsInCurrentPageInitStack)
		{
			return false;
		}

		while (CurrentScopePage->InitialStack[Depth].EndTime < 0.0)
		{
			const FEventScopeEntryPage* NextScopePage = ScopeEntryIterator.NextPage();
			if (NextScopePage == nullptr)
			{
				break;
			}
			CurrentScopePage = NextScopePage;
		}

		OutPageStackEntry = CurrentScopePage->InitialStack[Depth];

		return true;
	}

	bool FindBestMatchEvent(const FFindBestMatchEventInParams &InParams, FFindBestMatchEventOutParams &OutParams) const
	{
		FIterationState IterationState = InParams.IterationState;
		const FEventScopeEntry* ScopeEntry = IterationState.ScopeIterator.GetCurrentItem();
		const EventType* Event = IterationState.EventsIterator.GetCurrentItem();

		FEventInfoStackEntry BestMatchEntry;
		double BestMatchEndTime = 0.0;
		bool bHasEndEvent = false;

		// In the first step, we iterate up to TargetExactTime, storing the last event with our target depth that ended during iteration.
		// ......]...]....]...TargetExactTime
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) <= InParams.TargetExactTime)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(IterationState.StackDepth < SettingsType::MaxDepth);
				FEventInfoStackEntry& StackEntry = IterationState.EventStack[IterationState.StackDepth++];
				StackEntry.Event = *Event;
				StackEntry.StartTime = -ScopeEntry->Time;
				StackEntry.ExclTime = 0.0;

				if (IterationState.StackDepth > 1)
				{
					FEventInfoStackEntry& ParentStackEntry = IterationState.EventStack[IterationState.StackDepth - 2];
					ParentStackEntry.ExclTime += StackEntry.StartTime - IterationState.LastIterationTime;
				}

				Event = IterationState.EventsIterator.NextItem();
				IterationState.LastIterationTime = -ScopeEntry->Time;
			}
			else // "end" event
			{
				check(IterationState.StackDepth > 0);
				--IterationState.StackDepth;

				if (IterationState.StackDepth == InParams.TargetDepth)
				{
					FEventInfoStackEntry& StackEntry = IterationState.EventStack[IterationState.StackDepth];
					StackEntry.ExclTime += ScopeEntry->Time - IterationState.LastIterationTime;
					BestMatchEndTime = ScopeEntry->Time;
					BestMatchEntry = StackEntry;
					bHasEndEvent = true;
				}

				IterationState.LastIterationTime = ScopeEntry->Time;
			}

			ScopeEntry = IterationState.ScopeIterator.NextItem();
		}

		// If the iteration stack depth is as deep as our target depth, that we have a perfect match,
		// an event that is ongoing at TargetExactTime, so we just return it.
		// ....[..TargetExactTime.....]
		// The strict '>' comparison is needed because CurrentStackDepth is "RealDepth + 1".
		if (IterationState.StackDepth > InParams.TargetDepth)
		{
			OutParams.EventInfo = IterationState.EventStack[InParams.TargetDepth];
			OutParams.bHasEndTime = false;
			OutParams.IterationState = IterationState;

			return true;
		}

		// We continue iterating until TargetEndTime or until we find the start of an event with the target depth.
		// TargetExactTime.....[
		bool bHasStartEvent = false;
		while (ScopeEntry && FMath::Abs(ScopeEntry->Time) <= InParams.TargetEndTime)
		{
			if (ScopeEntry->Time < 0.0) // "begin" event
			{
				check(IterationState.StackDepth < SettingsType::MaxDepth);
				FEventInfoStackEntry& StackEntry = IterationState.EventStack[IterationState.StackDepth++];
				StackEntry.Event = *Event;
				StackEntry.StartTime = -ScopeEntry->Time;
				StackEntry.ExclTime = 0.0;

				if (IterationState.StackDepth > 1)
				{
					FEventInfoStackEntry& ParentStackEntry = IterationState.EventStack[IterationState.StackDepth - 2];
					ParentStackEntry.ExclTime += StackEntry.StartTime - IterationState.LastIterationTime;
				}

				Event = IterationState.EventsIterator.NextItem();
				IterationState.LastIterationTime = -ScopeEntry->Time;

				if (IterationState.StackDepth == InParams.TargetDepth + 1)
				{
					bHasStartEvent = true;
					ScopeEntry = IterationState.ScopeIterator.NextItem();
					break;
				}
			}
			else // "end" event
			{
				check(IterationState.StackDepth > 0);
				--IterationState.StackDepth;

				if (IterationState.StackDepth == InParams.TargetDepth)
				{
					FEventInfoStackEntry& StackEntry = IterationState.EventStack[IterationState.StackDepth];
					StackEntry.ExclTime += ScopeEntry->Time - IterationState.LastIterationTime;
				}

				IterationState.LastIterationTime = ScopeEntry->Time;
			}

			ScopeEntry = IterationState.ScopeIterator.NextItem();
		}

		if (bHasStartEvent == false && bHasEndEvent == false)
		{
			return false;
		}

		// We choose the event that is closer to TargetExactTime.
		if (bHasStartEvent)
		{
			double EndTimeDelta = InParams.TargetExactTime - BestMatchEndTime;
			double StartTimeDelta = IterationState.LastIterationTime - InParams.TargetExactTime;
			if (!bHasEndEvent || StartTimeDelta < EndTimeDelta)
			{
				//The event that just started is the best match
				bHasEndEvent = false;
				BestMatchEntry = IterationState.EventStack[InParams.TargetDepth];
			}
		}

		OutParams.EventInfo = BestMatchEntry;
		OutParams.bHasEndTime = bHasEndEvent;
		if (bHasEndEvent)
		{
			OutParams.EndTime = BestMatchEndTime;
		}

		OutParams.IterationState = IterationState;
		return true;
	}

	ILinearAllocator& Allocator;
	TArray<FDetailLevel> DetailLevels;
	int32 ExtraDepthEvents = 0; // the number of events virtually pushed on the stack when depth exceeds SettingsType::MaxDepth
	uint64 ModCount = 0; // a serial number increased each time the timeline is modified
};

} // namespace TraceServices
