// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/PagedArray.h"

namespace TraceServices
{

struct FEventScopeEntry
{
	double Time;
};

struct FEventStackEntry
{
	uint64 EnterScopeIndex;
	uint64 EventIndex;
	double ExclTime = 0.0;
	double EndTime = -1.0; //By convention a negative EndTime means the event does not end in the current page
};

struct FEventScopeEntryPage
{
	FEventScopeEntry* Items = nullptr;
	uint64 Count = 0;
	double BeginTime = 0.0;
	double EndTime = 0.0;
	uint64 BeginEventIndex = 0;
	uint64 EndEventIndex = 0;
	FEventStackEntry* InitialStack = nullptr;
	uint16 InitialStackCount = 0;
};

template<typename EventType>
struct FDetailLevelDepthState
{
	int64 PendingScopeEnterIndex = -1;
	int64 PendingEventIndex = -1;

	EventType DominatingEvent;
	double DominatingEventStartTime = 0.0;
	double DominatingEventEndTime = 0.0;
	double DominatingEventDuration = 0.0;

	double EnterTime = 0.0;
	double ExitTime = 0.0;
};

template<typename EventType, typename SettingsType>
struct FDetailLevelInsertionState
{
	double LastTime = -1.0;
	uint16 CurrentDepth = 0;
	int32 PendingDepth = -1;
	FDetailLevelDepthState<EventType> DepthStates[SettingsType::MaxDepth];
	FEventStackEntry EventStack[SettingsType::MaxDepth];
	uint64 CurrentScopeEntryPageIndex = (uint64)-1;
};

template<typename EventType, typename SettingsType>
struct FDetailLevel
{
	FDetailLevel(ILinearAllocator& Allocator, double InResolution)
		: Resolution(InResolution)
		, ScopeEntries(Allocator, SettingsType::ScopeEntriesPageSize)
		, Events(Allocator, SettingsType::EventsPageSize)
	{

	}

	double GetScopeEntryTime(uint64 Index) const
	{
		const FEventScopeEntry& ScopeEntry = ScopeEntries[Index];
		return ScopeEntry.Time < 0 ? -ScopeEntry.Time : ScopeEntry.Time;
	}

	void SetEvent(uint64 Index, const EventType& Event)
	{
		Events[Index] = Event;
	}

	EventType GetEvent(uint64 Index) const
	{
		return Events[Index];
	}

	double Resolution;
	TPagedArray<FEventScopeEntry, FEventScopeEntryPage> ScopeEntries;
	TPagedArray<EventType> Events;

	FDetailLevelInsertionState<EventType, SettingsType> InsertionState;
};

template<typename EventType, typename SettingsType>
struct FEventInfoStackEntry
{
	FEventInfoStackEntry() = default;
	FEventInfoStackEntry(const FEventStackEntry& EventStackEntry, const FDetailLevel<EventType, SettingsType>& DetailLevel)
	{
		StartTime = DetailLevel.GetScopeEntryTime(EventStackEntry.EnterScopeIndex);
		Event = DetailLevel.GetEvent(EventStackEntry.EventIndex);
		ExclTime = EventStackEntry.ExclTime;
	}

	EventType Event;
	double StartTime = 0.0;
	double ExclTime = 0.0;
};

} // namespace TraceServices
