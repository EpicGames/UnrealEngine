// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Containers/Timelines.h"
#include "Common/PagedArray.h"
#include "TraceServices/Containers/Allocators.h"
#include <limits>

namespace TraceServices
{

struct FPointTimelineDefaultSettings
{
	enum
	{
		EventsPerPage = 65536
	};
};

template<typename EventType, typename SettingsType = FPointTimelineDefaultSettings>
class TPointTimeline
	: public ITimeline<EventType>
{
public:
	TPointTimeline(ILinearAllocator& Allocator)
		: Events(Allocator, SettingsType::EventsPerPage)
	{
	}

	virtual ~TPointTimeline() = default;
	virtual uint64 GetModCount() const override { return ModCount; }
	virtual uint64 GetEventCount() const override { return Events.Num(); }
	virtual const EventType& GetEvent(uint64 InIndex) const override { return Events[InIndex].Event; }
	virtual double GetStartTime() const override { return Events.Num() > 0 ? Events[0].Time : 0.0; }
	virtual double GetEndTime() const override { return Events.Num() > 0 ? Events[Events.Num() - 1].Time : 0.0; }
	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, typename ITimeline<EventType>::EventCallback Callback) const override { check(false); }
	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, typename ITimeline<EventType>::EventRangeCallback Callback) const override { check(false); }
	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, typename ITimeline<EventType>::EventCallback Callback) const override
	{
		EnumerateEvents(IntervalStart, IntervalEnd, [&Callback](double InStartTime, double InEndTime, uint32 InDepth, const EventType& InEvent)
		{
			Callback(true, InStartTime, InEvent);
			Callback(false, InEndTime, InEvent);
			return EEventEnumerate::Continue;
		});
	}
	
	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, typename ITimeline<EventType>::EventRangeCallback Callback) const override
	{
		bool bIntersected = false;
		auto EventIterator = Events.GetIteratorFromPage(0);
		const FEventInternal* Event = EventIterator.GetCurrentItem();
		const FEventInternal* PreviousEvent = Event;
		while (Event)
		{
			const FEventPage* Page = EventIterator.GetCurrentPage();
			if (Page->EndTime < IntervalStart || IntervalEnd < Page->BeginTime)
			{
				if (!EventIterator.NextPage())
				{
					return;
				}

				PreviousEvent = Event;
				Event = EventIterator.GetCurrentItem();
			}
			else
			{
				if (!(Event->Time < IntervalStart || IntervalEnd < Event->Time))
				{
					if(!bIntersected && bEnumerateOutsideRange && PreviousEvent != Event)
					{
						if (Callback(PreviousEvent->Time, PreviousEvent->Time, 0, PreviousEvent->Event) == EEventEnumerate::Stop)
						{
							return;
						}
					}

					if (Callback(Event->Time, Event->Time, 0, Event->Event) == EEventEnumerate::Stop)
					{
						return;
					}
					bIntersected = true;
				}
				else if(bEnumerateOutsideRange && bIntersected)
				{
					if (Callback(Event->Time, Event->Time, 0, Event->Event) == EEventEnumerate::Stop)
					{
						return;
					}
				}

				PreviousEvent = Event;
				Event = EventIterator.NextItem();
			}
		}
	}
	
	uint64 AppendEvent(double Time, const EventType& Event)
	{
		return EmplaceEvent(Time, Event);
	}

	template <typename... ArgsType>
	uint64 EmplaceEvent(double Time, ArgsType&&... Args)
	{
		uint64 Index = Events.Num();
		Events.EmplaceBack(Time, Forward<ArgsType>(Args)...);

		FEventPage* LastPage = Events.GetLastPage();
		if (LastPage != CurrentPage)
		{
			CurrentPage = LastPage;
			LastPage->BeginTime = Time;
			LastPage->EndTime = Time;
			LastPage->BeginEventIndex = Index;
		}
		else
		{
			LastPage->BeginTime = FMath::Min(LastPage->BeginTime, Time);
			LastPage->EndTime = FMath::Max(LastPage->EndTime, Time);
		}

		LastPage->EndEventIndex = Index + 1;

		++ModCount;

		return Index;
	}

	// Set whether we enumerate one event each side of the passed-in time range.
	// This is useful for cases such as drawing where connected events need to be drawn outside of
	// the viewport.
	void SetEnumerateOutsideRange(bool bInEnumerateOutsideRange)
	{
		bEnumerateOutsideRange = bInEnumerateOutsideRange;
	}

	virtual bool GetEventInfo(double InTime, double DeltaTime, int32 Depth, typename ITimeline<EventType>::FTimelineEventInfo& EventInfo) const override
	{
		//Not implemented
		check(false);
		return false;
	}

private:
	struct FEventInternal
	{
		double Time;
		EventType Event;

		FEventInternal() = default;
		FEventInternal(double InTime, EventType InEvent)
			: Time(InTime)
			, Event(MoveTempIfPossible(InEvent))
		{ }
	};

	struct FEventPage
	{
		double BeginTime = 0.0;
		double EndTime = 0.0;
		uint64 BeginEventIndex = 0;
		uint64 EndEventIndex = 0;
		FEventInternal* Items = nullptr;
		uint64 Count = 0;
	};

	TPagedArray<FEventInternal, FEventPage> Events;
	FEventPage* CurrentPage = nullptr;
	double LastTime = 0.0;
	uint64 ModCount = 0;
	bool bEnumerateOutsideRange = false;
};

} // namespace TraceServices
