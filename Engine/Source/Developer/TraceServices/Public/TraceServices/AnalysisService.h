// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Trace/DataStream.h"

namespace UE
{
namespace Trace
{
	class IInDataStream;
}
}

namespace TraceServices
{

class IAnalysisSession;

class IAnalysisService
{
public:

	/**
	 * Start analysis from a file and wait for it to finish.
	 * @param SessionUri Path to trace file.
	 * @return New completed analysis session 
	 */
	virtual TSharedPtr<const IAnalysisSession> Analyze(const TCHAR* SessionUri) = 0;

	/**
	 * Start analysis reading from a file.
	 * @param SessionUri Path to trace file
	 * @return New analysis session
	 */
	virtual TSharedPtr<const IAnalysisSession> StartAnalysis(const TCHAR* SessionUri) = 0;
	
	/**
	 * Start analysis from a trace which can be identified with a trace id.
	 * @param TraceId Unique trace id. If no id is available set ~0.
	 * @param SessionName Display name for this session
	 * @param DataStream Stream to read from
	 * @return New analysis session
	 */
	virtual TSharedPtr<const IAnalysisSession> StartAnalysis(uint32 TraceId, const TCHAR* SessionName, TUniquePtr<UE::Trace::IInDataStream>&& DataStream) = 0;
};

} // namespace TraceServices
