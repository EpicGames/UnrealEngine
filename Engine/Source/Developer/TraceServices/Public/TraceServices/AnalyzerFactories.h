// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

namespace UE
{
namespace Trace
{

class IAnalyzer;
class IInDataStream;

} // namepsace Trace
} // namespace UE

namespace TraceServices
{

class IAnalysisSession;
class IEditableBookmarkProvider;
class IEditableCounterProvider;
class IEditableTimingProfilerProvider;
class IEditableThreadProvider;

/*
* Create an IAnalysisSession object for external analysis.
* 
* @param InTraceId		A user defined identifier for this session.
* @param InSessionName	A user defined string to present to the user for this session.
* @param InDataStream	The data stream interface for this analysis session.
*/
TRACESERVICES_API TSharedPtr<IAnalysisSession> CreateAnalysisSession(uint32 InTraceId, const TCHAR* InSessionName, TUniquePtr<UE::Trace::IInDataStream>&& InDataStream);

/*
* Create an IAnalyzer object for analyzing bookmark events.
*
* @param InSession					The session object to interact with.
* @param InEditableBookmarkProvider	The bookmark provider interface to receive the event callbacks.
*/
TRACESERVICES_API TSharedPtr<UE::Trace::IAnalyzer> CreateBookmarksAnalyzer(IAnalysisSession& InSession, IEditableBookmarkProvider& InEditableBookmarkProvider);

/*
* Create an IAnalyzer object for analyzing counter events.
*
* @param InSession					The session object to interact with.
* @param InEditableCounterProvider	The counter provider interface to receive the event callbacks.
*/
TRACESERVICES_API TSharedPtr<UE::Trace::IAnalyzer> CreateCountersAnalyzer(IAnalysisSession& InSession, IEditableCounterProvider& InEditableCounterProvider);

/*
* Create an IAnalyzer object for external analysis.
*
* @param InSession							The session object to interact with.
* @param InEditableTimingProfilerProvider	The timing profiler provider interface to receive the cpu timing event callbacks.
* @param InEditableThreadProvider			The thread provider interface to receive the thread event callbacks.
*/
TRACESERVICES_API TSharedPtr<UE::Trace::IAnalyzer> CreateCpuProfilerAnalyzer(IAnalysisSession& InSession, IEditableTimingProfilerProvider& InEditableTimingProfilerProvider, IEditableThreadProvider& InEditableThreadProvider);

} // namespace TraceServices
