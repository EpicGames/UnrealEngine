// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

namespace TraceServices
{

class FCancellationToken
{
public:
	FCancellationToken()
		: bCancel(false)
	{}

	bool ShouldCancel() { return bCancel.load(); }
	void Cancel() { bCancel.store(true); }
	void Reset() { bCancel.store(false); }

private:
	std::atomic<bool> bCancel;
};

} //namespace TraceServices
