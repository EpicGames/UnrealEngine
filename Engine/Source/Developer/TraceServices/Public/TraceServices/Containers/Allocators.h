// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/StringFwd.h"

namespace TraceServices
{
	
class ILinearAllocator
{
public:
	virtual ~ILinearAllocator() = default;
	virtual void* Allocate(uint64 Size) = 0;
};

class IStringStore
{
public:
	virtual ~IStringStore() = default;
	virtual const TCHAR* Find(const TCHAR* String) const = 0;
	virtual const TCHAR* Find(const FStringView& String) const = 0;
	virtual const TCHAR* Store(const TCHAR* String) = 0;
	virtual const TCHAR* Store(const FStringView& String) = 0;
};

} // namespace TraceServices
