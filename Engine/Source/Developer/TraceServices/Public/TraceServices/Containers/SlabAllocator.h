// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "HAL/UnrealMemory.h"
#include "Misc/ScopeLock.h"
#include "TraceServices/Containers/Allocators.h"

namespace TraceServices
{

class FSlabAllocator
	: public ILinearAllocator
{
public:
	FSlabAllocator(uint64 InSlabSize)
		: SlabSize(InSlabSize)
	{
	}

	~FSlabAllocator()
	{
		FScopeLock _(&Cs);
		for (void* Slab : Slabs)
		{
			FMemory::Free(Slab);
		}
	}

	virtual void* Allocate(uint64 Size) override
	{
		FScopeLock _(&Cs);
		TotalUsedSize += Size;
		uint64 AllocationSize = Size + (-int64(Size) & 15);
		if (AllocationSize <= SlabSize)
		{
			if (!CurrentSlab || CurrentSlabAllocatedSize + AllocationSize > SlabSize)
			{
				TotalAllocatedSize += SlabSize;
				CurrentSlab = reinterpret_cast<uint8*>(FMemory::Malloc(SlabSize, 16));
				CurrentSlabAllocatedSize = 0;
				Slabs.Add(CurrentSlab);
			}
			void* Allocation = CurrentSlab + CurrentSlabAllocatedSize;
			CurrentSlabAllocatedSize += AllocationSize;
			return Allocation;
		}
		else
		{
			TotalAllocatedSize += AllocationSize;
			void* Allocation = FMemory::Malloc(AllocationSize, 16);
			Slabs.Add(Allocation);
			return Allocation;
		}
	}

	uint64 GetUsedSize() const
	{
		FScopeLock _(&Cs);
		return TotalUsedSize;
	}

	uint64 GetAllocatedSize() const
	{
		FScopeLock _(&Cs);
		return TotalAllocatedSize + Slabs.GetAllocatedSize();
	}

private:
	mutable FCriticalSection Cs;
	TArray<void*> Slabs;
	uint8* CurrentSlab = nullptr;
	const uint64 SlabSize;
	uint64 CurrentSlabAllocatedSize = 0;
	uint64 TotalUsedSize = 0;
	uint64 TotalAllocatedSize = 0;
};

} // namespace TraceServices
