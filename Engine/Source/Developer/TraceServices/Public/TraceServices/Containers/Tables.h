// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace TraceServices
{

enum ETableColumnType
{
	TableColumnType_Invalid,
	TableColumnType_Bool,
	TableColumnType_Int,
	TableColumnType_Float,
	TableColumnType_Double,
	TableColumnType_CString,
};

enum ETableColumnDisplayHint
{
	TableColumnDisplayHint_Time = 1 << 0,
	TableColumnDisplayHint_Memory = 1 << 1,
	TableColumnDisplayHint_Summable = 1 << 31
};

class ITableLayout
{
public:
	virtual ~ITableLayout() = default;
	virtual uint64 GetColumnCount() const = 0;
	virtual const TCHAR* GetColumnName(uint64 ColumnIndex) const = 0;
	virtual ETableColumnType GetColumnType(uint64 ColumnIndex) const = 0;
	virtual uint32 GetColumnDisplayHintFlags(uint64 ColumnIndex) const = 0;
};

class IUntypedTableReader
{
public:
	virtual ~IUntypedTableReader() = default;
	virtual bool IsValid() const = 0;
	virtual void NextRow() = 0;
	virtual void SetRowIndex(uint64 RowIndex) = 0;
	virtual bool GetValueBool(uint64 ColumnIndex) const = 0;
	virtual int64 GetValueInt(uint64 ColumnIndex) const = 0;
	virtual float GetValueFloat(uint64 ColumnIndex) const = 0;
	virtual double GetValueDouble(uint64 ColumnIndex) const = 0;
	virtual const TCHAR* GetValueCString(uint64 ColumnIndex) const = 0;
};

template<typename RowType>
class ITableReader
	: public IUntypedTableReader
{
public:
	virtual const RowType* GetCurrentRow() const = 0;
};

class IUntypedTable
{
public:
	virtual ~IUntypedTable() = default;
	virtual const ITableLayout& GetLayout() const = 0;
	virtual uint64 GetRowCount() const = 0;
	virtual IUntypedTableReader* CreateReader() const = 0;
};

template<typename RowType>
class ITable
	: public IUntypedTable
{
public:
	virtual ~ITable() = default;
	virtual ITableReader<RowType>* CreateReader() const = 0;
};

TRACESERVICES_API bool Table2Csv(const IUntypedTable& Table, const TCHAR* Filename);

} // namespace TraceServices
