// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"

namespace TraceServices
{

enum class EEventEnumerate
{
	Continue,
	Stop,
};

enum class EEventSortOrder
{
	ByStartTime,
	ByEndTime,
};

template<typename InEventType>
class ITimeline
{
public:
	typedef InEventType EventType;
	typedef TFunctionRef<EEventEnumerate(bool /*bStart*/, double /*Time*/, const EventType& /*Event*/)> EventCallback;
	typedef TFunctionRef<EEventEnumerate(double /*StartTime*/, double /*EndTime*/, uint32 /*Depth*/, const EventType&/*Event*/)> EventRangeCallback;
	typedef TFunction<EEventEnumerate(double /*StartTime*/, double /*EndTime*/, uint32 /*Depth*/, const EventType&/*Event*/, uint32 /*TaskIndex*/)> AsyncEventRangeCallback;
	typedef TFunction<EEventEnumerate(bool /*bStart*/, double /*Time*/, const EventType& /*Event*/, uint32 /*TaskIndex*/)> AsyncEventCallback;
	typedef TFunction<void(uint32 /*NumTasks*/)> TaskInfoCallback;

	struct FTimelineEventInfo
	{
		double StartTime = 0.0;
		double EndTime = 0.0;
		double ExclTime = 0.0;
		EventType Event;
	};

	struct EnumerateAsyncParams
	{
		double IntervalStart = 0.0;
		double IntervalEnd = -1.0;
		double Resolution = 0.0;
		double MaxOccupancy = 0.875; // Leave 1/8 threads free to avoid starvation.
		EEventSortOrder SortOrder = EEventSortOrder::ByEndTime;
		typename ITimeline<EventType>::TaskInfoCallback SetupCallback;

		// Only one of these 2 callbacks must be set.
		typename ITimeline<EventType>::AsyncEventRangeCallback EventRangeCallback;
		typename ITimeline<EventType>::AsyncEventCallback EventCallback;
	};

	virtual ~ITimeline() = default;

	virtual uint64 GetModCount() const = 0;

	virtual uint64 GetEventCount() const = 0;
	virtual const InEventType& GetEvent(uint64 InIndex) const = 0;

	virtual double GetStartTime() const = 0;
	virtual double GetEndTime() const = 0;

	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, EventCallback Callback) const = 0;
	virtual void EnumerateEventsDownSampled(double IntervalStart, double IntervalEnd, double Resolution, EventRangeCallback Callback) const = 0;

	virtual void EnumerateEventsBackwardsDownSampled(double IntervalEnd, double IntervalStart, double Resolution, EventCallback Callback) const {}
	virtual void EnumerateEventsBackwardsDownSampled(double IntervalEnd, double IntervalStart, double Resolution, EventRangeCallback Callback) const {}

	virtual void EnumerateEventsDownSampledAsync(const EnumerateAsyncParams& EnumerateAsyncParams) const {}

	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, EventCallback Callback) const = 0;
	virtual void EnumerateEvents(double IntervalStart, double IntervalEnd, EventRangeCallback Callback) const = 0;

	virtual void EnumerateEventsBackwards(double IntervalEnd, double IntervalStart, EventCallback Callback) const {}
	virtual void EnumerateEventsBackwards(double IntervalEnd, double IntervalStart, EventRangeCallback Callback) const {}

	virtual int32 GetDepthAt(double Time) const { return 0; }

	/**
	 * Finds event information for the event closest to InTime from the interval [InTime - DeltaTime, InTime + DeltaTime]
	 * @param InTime - The time used to query for the event
	 * @param DeltaTime - Events from interval [InTime - DeltaTime, InTime + DeltaTime] will be considered. The one closest to InTime will be returned
	 * @param Depth - The Depth used to query for the event
	 * @return True if an event was found, False if no event was found for the specified input parameters
	 */
	virtual bool GetEventInfo(double InTime, double DeltaTime, int32 Depth, FTimelineEventInfo& EventInfo) const = 0;
};

} // namespace TraceServices
