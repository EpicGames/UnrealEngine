// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

namespace TraceServices
{

class IAnalysisService;
class IModuleService;

} // namespace TraceServices

class ITraceServicesModule
	: public IModuleInterface
{
public:
	virtual TSharedPtr<TraceServices::IAnalysisService> GetAnalysisService() = 0;
	virtual TSharedPtr<TraceServices::IModuleService> GetModuleService() = 0;
	virtual TSharedPtr<TraceServices::IAnalysisService> CreateAnalysisService() = 0;
	virtual TSharedPtr<TraceServices::IModuleService> CreateModuleService() = 0;

	virtual ~ITraceServicesModule() = default;
};
