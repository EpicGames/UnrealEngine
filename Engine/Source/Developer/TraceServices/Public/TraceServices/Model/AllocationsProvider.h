// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

#include <new>

#define UE_API TRACESERVICES_API

namespace TraceServices
{

// Id type for tags
typedef uint32 TagIdType;

class IAllocationsProvider : public IProvider
{
public:
	// Allocation query rules.
	// The enum uses the following naming convention:
	//     A, B, C, D = time markers
	//     a = time when "alloc" event occurs
	//     f = time when "free" event occurs (can be infinite)
	// Ex.: "AaBf" means "all memory allocations allocated between time A and time B and freed after time B".
	enum class EQueryRule
	{
		aAf,     // active allocs at A
		afA,     // before
		Aaf,     // after
		aAfB,    // decline
		AaBf,    // growth
		aAfaBf,  // decline + growth (used as "growth vs. decline")
		AfB,     // free events
		AaB,     // alloc events
		AafB,    // short living allocs
		aABf,    // long living allocs
		AaBCf,   // memory leaks
		AaBfC,   // limited lifetime
		aABfC,   // decline of long living allocs
		AaBCfD,  // specific lifetime
		AoB,     // all paged out allocs between A and B
		AiB,     // all paged in allocs between A and B
		//A_vs_B,  // compare A vs. B; {aAf} vs. {aBf}
		//A_or_B,  // live at A or at B; {aAf} U {aBf}
		//A_xor_B, // live either at A or at B; ({aAf} U {aBf}) \ {aABf}
	};

	enum class ETimelineU64
	{
		// The Minimum Total Allocated Memory
		MinTotalAllocatedMemory,

		// The Maximum Total Allocated Memory
		MaxTotalAllocatedMemory,

		// The Minimum Total Swap Memory
		MinTotalSwapMemory,

		// The Maximum Total Swap Memory
		MaxTotalSwapMemory,

		// The Minimum Total Compressed Swap Memory
		MinTotalCompressedSwapMemory,

		// The Maximum Total Compressed Swap Memory
		MaxTotalCompressedSwapMemory,

		Count
	};

	enum class ETimelineU32
	{
		// The Minimum Number of Live Allocations
		MinLiveAllocations,

		// The Maximum Number of Live Allocations
		MaxLiveAllocations,

		// The Number of Alloc Events
		AllocEvents,

		// The Number of Free Events
		FreeEvents,

		// The Number of Page In Events
		PageInEvents,

		// The Number of Page Out Events
		PageOutEvents,

		// The Number of Swap Free Events
		SwapFreeEvents,

		Count
	};

	struct FQueryParams
	{
		EQueryRule Rule;
		double TimeA;
		double TimeB;
		double TimeC;
		double TimeD;
	};

	struct FAllocation
	{
		UE_API uint32 GetStartEventIndex() const;
		UE_API uint32 GetEndEventIndex() const;
		UE_API double GetStartTime() const;
		UE_API double GetEndTime() const;
		UE_API uint64 GetAddress() const;
		UE_API uint64 GetSize() const;
		UE_API uint32 GetAlignment() const;
		UE_API uint32 GetAllocThreadId() const;
		UE_API uint32 GetFreeThreadId() const;
		UE_API uint32 GetAllocCallstackId() const;
		UE_API uint32 GetFreeCallstackId() const;
		UE_API uint32 GetMetadataId() const;
		UE_API TagIdType GetTag() const;
		UE_API HeapId GetRootHeap() const;
		UE_API bool IsHeap() const;
		UE_API bool IsSwap() const;
	};

	class FAllocations
	{
	public:
		UE_API void operator delete (void* Address);
		UE_API uint32 Num() const;
		UE_API const FAllocation* Get(uint32 Index) const;
	};

	typedef TUniquePtr<const FAllocations> FQueryResult;

	enum class EQueryStatus
	{
		Unknown,
		Done,
		Working,
		Available,
	};

	struct FQueryStatus
	{
		UE_API FQueryResult NextResult() const;

		EQueryStatus Status;
		mutable UPTRINT Handle;
	};

	struct FHeapSpec
	{
		HeapId Id;
		FHeapSpec* Parent;
		TArray<FHeapSpec*> Children;
		const TCHAR* Name;
		EMemoryTraceHeapFlags Flags;
	};

	typedef UPTRINT FQueryHandle;

public:
	virtual ~IAllocationsProvider() = default;

	virtual void BeginRead() const = 0;
	virtual void EndRead() const = 0;
	virtual void ReadAccessCheck() const = 0;

	virtual bool IsInitialized() const = 0;

	// Returns true if provider has processed at least one Alloc, Free or Heap event.
	virtual bool HasAllocationEvents() const = 0;

	// Returns true if provider has processed at least one Swap Op event.
	virtual bool HasSwapOpEvents() const = 0;

	// Enumerates the discovered tags.
	virtual void EnumerateTags(TFunctionRef<void(const TCHAR*, const TCHAR*, TagIdType, TagIdType)> Callback) const = 0;

	// Returns the display name of the specified LLM tag.
	// Lifetime of returned string matches the session lifetime.
	virtual const TCHAR* GetTagName(TagIdType Tag) const = 0;
	virtual const TCHAR* GetTagFullPath(TagIdType Tag) const = 0;

	virtual void EnumerateRootHeaps(TFunctionRef<void(HeapId Id, const FHeapSpec&)> Callback) const = 0;
	virtual void EnumerateHeaps(TFunctionRef<void(HeapId Id, const FHeapSpec&)> Callback) const = 0;

	// Returns the number of points in each timeline (Min/Max Total Allocated Memory, Min/Max Live Allocations, Total Alloc Events, Total Free Events).
	virtual int32 GetTimelineNumPoints() const = 0;

	// Returns the inclusive index range [StartIndex, EndIndex] for a time range [StartTime, EndTime].
	// Index values are in range { -1, 0, .. , N-1, N }, where N = GetTimelineNumPoints().
	virtual void GetTimelineIndexRange(double StartTime, double EndTime, int32& StartIndex, int32& EndIndex) const = 0;

#if 1 // deprecated API
	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt64, ...) instead")
	virtual void EnumerateMinTotalAllocatedMemoryTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU64::MinTotalAllocatedMemory, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt64, ...) instead")
	virtual void EnumerateMaxTotalAllocatedMemoryTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU64::MaxTotalAllocatedMemory, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumerateMinLiveAllocationsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::MinLiveAllocations, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumerateMaxLiveAllocationsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::MaxLiveAllocations, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt64, ...) instead")
	virtual void EnumerateMinTotalSwapMemoryTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU64::MinTotalSwapMemory, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt64, ...) instead")
	virtual void EnumerateMaxTotalSwapMemoryTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU64::MaxTotalSwapMemory, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt64, ...) instead")
	virtual void EnumerateMinTotalCompressedSwapMemoryTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU64::MinTotalCompressedSwapMemory, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt64, ...) instead")
	virtual void EnumerateMaxTotalCompressedSwapMemoryTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU64::MaxTotalCompressedSwapMemory, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumerateAllocEventsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::AllocEvents, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumerateFreeEventsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::FreeEvents, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumeratePageInEventsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::PageInEvents, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumeratePageOutEventsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::PageOutEvents, StartIndex, EndIndex, Callback);
	}

	UE_DEPRECATED(5.7, "Use EnumerateTimeline(ETimelineUInt32, ...) instead")
	virtual void EnumerateSwapFreeEventsTimeline(int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const
	{
		EnumerateTimeline(ETimelineU32::SwapFreeEvents, StartIndex, EndIndex, Callback);
	}
#endif

	// Enumerates points (time-value pairs) of a UInt64 timeline, in the inclusive index interval [StartIndex, EndIndex].
	virtual void EnumerateTimeline(ETimelineU64 Timeline, int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint64 Value)> Callback) const = 0;

	// Enumerates points (time-value pairs) of a UInt32 timeline, in the inclusive index interval [StartIndex, EndIndex].
	virtual void EnumerateTimeline(ETimelineU32 Timeline, int32 StartIndex, int32 EndIndex, TFunctionRef<void(double Time, double Duration, uint32 Value)> Callback) const = 0;

	virtual FQueryHandle StartQuery(const FQueryParams& Params) const = 0;
	virtual void CancelQuery(FQueryHandle Query) const = 0;
	virtual const FQueryStatus PollQuery(FQueryHandle Query) const = 0;

	virtual uint64 GetPlatformPageSize() const = 0;
};

TRACESERVICES_API FName GetAllocationsProviderName();
TRACESERVICES_API const IAllocationsProvider* ReadAllocationsProvider(const IAnalysisSession& Session);

} // namespace TraceServices

inline const TCHAR* LexToString(const TraceServices::IAllocationsProvider::EQueryRule Rule)
{
	using EQueryRule = TraceServices::IAllocationsProvider::EQueryRule;
	switch (Rule)
	{
		case EQueryRule::aAf:       return TEXT("aAf");
		case EQueryRule::afA:       return TEXT("afA");
		case EQueryRule::Aaf:       return TEXT("Aaf");
		case EQueryRule::aAfB:      return TEXT("aAfB");
		case EQueryRule::AaBf:      return TEXT("AaBf");
		case EQueryRule::aAfaBf:    return TEXT("aAfaBf");
		case EQueryRule::AfB:       return TEXT("AfB");
		case EQueryRule::AaB:       return TEXT("AaB");
		case EQueryRule::AafB:      return TEXT("AafB");
		case EQueryRule::aABf:      return TEXT("aABf");
		case EQueryRule::AaBCf:     return TEXT("AaBCf");
		case EQueryRule::AaBfC:     return TEXT("AaBfC");
		case EQueryRule::aABfC:     return TEXT("aABfC");
		case EQueryRule::AaBCfD:    return TEXT("AaBCfD");
		case EQueryRule::AoB:       return TEXT("AoB");
		case EQueryRule::AiB:       return TEXT("AiB");
		//case EQueryRule::A_vs_B:    return TEXT("A_vs_B");
		//case EQueryRule::A_or_B:    return TEXT("A_or_B");
		//case EQueryRule::A_xor_B:   return TEXT("A_xor_B");
		default:                    return TEXT("");
	}
}

#undef UE_API
