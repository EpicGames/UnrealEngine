// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Memory/MemoryView.h"

namespace TraceServices
{
	/**
	* An iterator for segments
	*/
	typedef TArrayView<FMemoryView> FSegmentIterator;


	/**
	 * Flags defining the behaviour of a cache entry
	 */
	enum ECacheFlags : uint16
	{
		// When set blocks are not kept in as part of the global caching.
		ECacheFlags_NoGlobalCaching = 1 << 1,
	};

	/**
	 * A unique id for a named cache entry.
	 */
	typedef uint32 FCacheId;
	
	/**
	 * Analysis cache
	 *
	 * Allows users of the system to create tracked fixed size blocks associated with a "cache id", created using a string.
	 * After creating block(s) users are free to write arbitrary data into them. Once all references to the block has been
	 * dropped the buffer is committed to persistent storage (e.g. cache file).
	 *
	 * On consecutive analysis sessions the blocks can be retrieved using the cache id and the block index. Any block can be
	 * requested at any point in the session in any order.
	 *
	 * Analysis is not thread safe and is designed to be used from the analysis thread only.
	 *
	 * The container classes \ref TraceServices::TCachedPagedArray or \ref TraceServices::FCachedStringStore are provided as
	 * user facing abstractions on top of the cache.
	 *
	 */
	class IAnalysisCache
	{
	public:
		virtual ~IAnalysisCache() = default;
	
		/**
		 * Block size in bytes
		 */
		constexpr static uint64 BlockSizeBytes = 8 * 1024;
		
		/**
		* Returns the unique id for a named entry. If the entry does not yet exists and new id is reserved.
		* @param Name String identifier
		* @param Flags Details about the entry from \ref ECacheFlags.
		* @return A valid index for the identifier
		*/
		virtual FCacheId GetCacheId(const TCHAR* Name, uint16 Flags = 0) = 0;

		/**
		 * Gets a pointer to the user data a given cache id. The owner of the cache id is able to store arbitrary
		 * data which does not fit as block data, for example element counts. The user can write to this memory at any
		 * point during the session. The block is stored to disk at the end of the session.
		 * @param CacheId Unique cache id
		 * @return A valid memory view of the user data block, an empty view if the index was not found.
		 */
		virtual FMutableMemoryView GetUserData(FCacheId CacheId) = 0;
			
		/**
		* Create blocks of tracked memory. The requested blocks form a contiguous memory range. The block written
		* to disk when no more references are alive.
		* @param CacheId Unique cache id
		* @param Count Number of blocks to use
		* @return Buffer mapping to the blocks. If the cache id is invalid an empty buffer is returned.
		*/
		[[nodiscard]] virtual FSharedBuffer CreateBlocks(FCacheId CacheId, uint32 Count) = 0;

		/**
		* Load blocks into memory. The requested range of blocks will be loaded into a contiguous memory range.
		* @param CacheId Unique cache id
		* @param BlockIndexStart Block index for this cache entry
		* @param BlockCount Consecutive block indices to load
		* @return Buffer mapping to the blocks. If the block range is invalid the buffer is empty.
		*/
		[[nodiscard]] virtual FSharedBuffer GetBlocks(FCacheId CacheId, uint32 BlockIndexStart, uint32 BlockCount = 1) = 0;
};
	
}