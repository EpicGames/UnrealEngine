// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"

class FName;

namespace UE {
namespace Trace {

class IAnalyzer;

} // namespace Trace
} // namespace UE

namespace TraceServices
{

class IAnalysisCache;
class ILinearAllocator;
class IStringStore;

class IProvider
{
public:
	virtual ~IProvider() = default;

	virtual void BeginRead() const { unimplemented(); }
	virtual void EndRead() const { unimplemented(); }
	virtual void ReadAccessCheck() const { unimplemented(); }
};

class IEditableProvider
{
public:
	virtual ~IEditableProvider() = default;

	virtual void BeginEdit() const { unimplemented(); }
	virtual void EndEdit() const { unimplemented(); }
	virtual void EditAccessCheck() const { unimplemented(); }
};

struct FTraceSessionMetadata
{
	enum class EType
	{
		Int64,
		Double,
		String
	};

	FName Name;
	EType Type;
	union
	{
		int64 Int64Value;
		double DoubleValue;
	};
	FString StringValue;
};

struct FAnalysisMessage
{
	EMessageSeverity::Type Severity;
	FString Message;
};

class IAnalysisSession
{
public:
	virtual ~IAnalysisSession() = default;
	virtual void Stop(bool bAndWait=false) const = 0;
	virtual void Wait() const = 0;

	virtual const TCHAR* GetName() const = 0;
	virtual uint32 GetTraceId() const = 0;
	virtual bool IsAnalysisComplete() const = 0;

	virtual double GetDurationSeconds() const = 0;
	/**
	 * Update the internal estimation of the session duration with a new timestamp.
	 * This function should be called with any timestamps from individual events so the full session duration can be
	 * estimated.
	 */
	virtual void UpdateDurationSeconds(double Duration) = 0;

	/**
	 * Getter for the unix timestamp at the start of all analysis.
	 * This function should be used as an offset for any relative (session) timestamps to calculate
	 * real UTC timestamps.
	 */
	virtual double GetBaseDateTime() const = 0;
	virtual void SetBaseDateTime(double InBaseDateTime) = 0;

	virtual uint32 GetMetadataCount() const = 0;
	virtual void EnumerateMetadata(TFunctionRef<void(const FTraceSessionMetadata& Metadata)> Callback) const = 0;
	virtual void AddMetadata(FName InName, int64 InValue) = 0;
	virtual void AddMetadata(FName InName, double InValue) = 0;
	virtual void AddMetadata(FName InName, FString InValue) = 0;

	/** Get the number of pending messages. Can be called without lock */
	virtual uint32 GetNumPendingMessages() const = 0;
	/** Moves pending messages to the return array. */
	virtual TArray<FAnalysisMessage> DrainPendingMessages() = 0;

	virtual void BeginRead() const = 0;
	virtual void EndRead() const = 0;
	virtual void ReadAccessCheck() const = 0;

	virtual void BeginEdit() = 0;
	virtual void EndEdit() = 0;
	virtual void WriteAccessCheck() = 0;

	virtual ILinearAllocator& GetLinearAllocator() = 0;
	virtual IStringStore& GetStringStore() = 0;
	virtual const TCHAR* StoreString(const TCHAR* String) = 0;
	virtual const TCHAR* StoreString(const FStringView& String) = 0;

	virtual IAnalysisCache& GetCache() = 0;

	virtual void AddAnalyzer(UE::Trace::IAnalyzer* Analyzer) = 0;
	virtual void AddAnalyzer(TSharedRef<UE::Trace::IAnalyzer> Analyzer) = 0;
	virtual void AddProvider(const FName& Name, TSharedPtr<IProvider> Provider, TSharedPtr<IEditableProvider> EditableProvider = nullptr) = 0;

	template<typename ProviderType>
	const ProviderType* ReadProvider(const FName& Name) const
	{
		return static_cast<const ProviderType*>(ReadProviderPrivate(Name));
	}

	template<typename ProviderType>
	ProviderType* EditProvider(const FName& Name)
	{
		return static_cast<ProviderType*>(EditProviderPrivate(Name));
	}

private:
	virtual const IProvider* ReadProviderPrivate(const FName& Name) const = 0;
	virtual IEditableProvider* EditProviderPrivate(const FName& Name) = 0;
};

struct FAnalysisSessionReadScope
{
	FAnalysisSessionReadScope(const IAnalysisSession& InAnalysisSession)
		: AnalysisSession(InAnalysisSession)
	{
		AnalysisSession.BeginRead();
	}

	~FAnalysisSessionReadScope()
	{
		AnalysisSession.EndRead();
	}

private:
	const IAnalysisSession& AnalysisSession;
};

struct FAnalysisSessionEditScope
{
	FAnalysisSessionEditScope(IAnalysisSession& InAnalysisSession)
		: AnalysisSession(InAnalysisSession)
	{
		AnalysisSession.BeginEdit();
	}

	~FAnalysisSessionEditScope()
	{
		AnalysisSession.EndEdit();
	}

	IAnalysisSession& AnalysisSession;
};

} // namespace TraceServices
