// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FBookmark
{
	double Time;
	const TCHAR* Text;
	uint32 CallstackId;
};

class IBookmarkProvider
	: public IProvider
{
public:
	virtual ~IBookmarkProvider() = default;
	virtual uint64 GetBookmarkCount() const = 0;
	virtual void EnumerateBookmarks(double IntervalStart, double IntervalEnd, TFunctionRef<void(const FBookmark&)> Callback) const = 0;
};

/*
* The interface to a provider that can consume mutations of bookmark events from a session.
*/
class IEditableBookmarkProvider
	: public IEditableProvider
{
public:
	virtual ~IEditableBookmarkProvider() = default;

	/*
	* Update the information in the Bookmark.
	*
	* @param BookmarkPoint	The unique identity (memory address) of the bookmark.
	* @param FormatString	The format string of the bookmark's message.
	* @param File			The source file the bookmark is in.
	* @param Line			The line in the source file the bookmark is on.
	*/
	virtual void UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line) = 0;

	/*
	* Append a new instance of a bookmark from the trace session.
	*
	* @param BookmarkPoint	The unique identity (memory address) of the bookmark.
	* @param Time			The time in seconds of the event.
	* @param FormatArgs		The arguments to use in conjunction with the spec's FormatString.
	*/
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatArgs) = 0;

	/*
	* Append a new instance of a bookmark from the trace session.
	*
	* @param BookmarkPoint	The unique identity (memory address) of the bookmark.
	* @param Time			The time in seconds of the event.
	* @param Text			The fully formatted bookmark string for this instance. This may vary between instances.
	*						This pointer is valid until the IAnalysisSession is deleted.
	*/
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text) = 0;

	UE_DEPRECATED(5.6, "Use AppendBookmark that has CallstackId parameter")
	void AppendBookmark(uint64 BookmarkPoint, double Time, const uint8* FormatArgs) { AppendBookmark(BookmarkPoint, Time, 0, FormatArgs); }
	UE_DEPRECATED(5.6, "Use AppendBookmark that has CallstackId parameter")
	void AppendBookmark(uint64 BookmarkPoint, double Time, const TCHAR* Text) { AppendBookmark(BookmarkPoint, Time, 0, Text); }
};

TRACESERVICES_API FName GetBookmarkProviderName();
TRACESERVICES_API const IBookmarkProvider& ReadBookmarkProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableBookmarkProvider& EditBookmarkProvider(IAnalysisSession& Session);

} // namespace TraceServices
