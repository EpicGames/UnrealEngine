// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Memory.h"
#include "TraceServices/Model/Modules.h"
#include "UObject/NameTypes.h"

#define UE_API TRACESERVICES_API

namespace TraceServices
{

/////////////////////////////////////////////////////////////////////
struct FStackFrame
{
	uint64 Addr;
	const FResolvedSymbol* Symbol;
};

/////////////////////////////////////////////////////////////////////
struct FCallstack
{
	/** Creates the default empty callstack (id 0). */
	FCallstack();

	/** Creates an empty callstack but with a non-zero id. */
	FCallstack(uint64 Id);

	/** Creates an callstack initialized with a certain number of stack frames. */
	UE_API FCallstack(const FStackFrame* FirstEntry, uint8 FrameCount);

	/** Initializes the callstack with a certain number of stack frames. */
	UE_API void Init(const FStackFrame* FirstEntry, uint8 FrameCount);

	/** Initializes an empty callstack but with a non-zero id. */
	void InitEmpty(uint64 Id);

	/** Gets the callstack id of an empty callstack. */
	uint64 GetEmptyId() const;

	/** Return true if the number of stack frames is zero. */
	bool IsEmpty() const;

	/** Gets the number of stack frames in callstack. */
	uint32 Num() const;

	/** Gets the address at a given stack depth. */
	uint64 Addr(uint8 Depth) const;

	/** Gets the cached symbol name at a given stack depth. */
	const TCHAR* Name(uint8 Depth) const;

	/** Gets the entire frame at given depth. */
	const FStackFrame* Frame(uint8 Depth) const;

private:
	enum : uint64
	{
		EntryLenShift = 56,
		EntryLenMask = uint64(0xff) << EntryLenShift,
	};

	uint64 CallstackLenIndex;
};

static_assert(sizeof(FCallstack) == 8, "struct FCallstack is too large");

/////////////////////////////////////////////////////////////////////
class ICallstacksProvider
	: public IProvider
{
public:
	virtual ~ICallstacksProvider() = default;

	/**
	  * Queries a callstack id.
	  * @param CallstackId Callstack id to query.
	  * @return Callstack information. If id is not found a special "Unknown" callstack is returned.
	  */
	virtual const FCallstack* GetCallstack(uint32 CallstackId) const = 0;

	/**
	  * Queries a set of callstack ids.
	  * @param CallstackIds List of callstack ids to query
	  * @param OutCallstacks Output list of callstacks. Caller is responsible for allocating space according to CallstackIds length.
	  */
	virtual void GetCallstacks(const TArrayView<uint32>& CallstackIds, FCallstack const** OutCallstacks) const = 0;

	/**
	  * Gets a callstack id for a registered callstack hash.
	  * @param CallstackHash The callstack hash.
	  * @returns The callstack id.
	  */
	virtual uint32 GetCallstackIdForHash(uint64 CallstackHash) const = 0;
};

/////////////////////////////////////////////////////////////////////

TRACESERVICES_API FName GetCallstacksProviderName();
TRACESERVICES_API const ICallstacksProvider* ReadCallstacksProvider(const IAnalysisSession& Session);

/////////////////////////////////////////////////////////////////////
inline FCallstack::FCallstack()
: CallstackLenIndex(0)
{
}

/////////////////////////////////////////////////////////////////////
inline FCallstack::FCallstack(uint64 Id)
{
	InitEmpty(Id);
}

/////////////////////////////////////////////////////////////////////
inline FCallstack::FCallstack(const FStackFrame* InFirstFrame, uint8 InFrameCount)
{
	Init(InFirstFrame, InFrameCount);
}

/////////////////////////////////////////////////////////////////////
inline void FCallstack::Init(const FStackFrame* InFirstFrame, uint8 InFrameCount)
{
	check((uint64(InFirstFrame) & EntryLenMask) == 0);
	CallstackLenIndex = (uint64(InFrameCount) << EntryLenShift) | (~EntryLenMask & uint64(InFirstFrame));
}

/////////////////////////////////////////////////////////////////////
inline void FCallstack::InitEmpty(uint64 Id)
{
	check((Id & EntryLenMask) == 0);
	CallstackLenIndex = Id & ~EntryLenMask;
}

/////////////////////////////////////////////////////////////////////
inline uint64 FCallstack::GetEmptyId() const
{
	return CallstackLenIndex & ~EntryLenMask;
}

/////////////////////////////////////////////////////////////////////
inline bool FCallstack::IsEmpty() const
{
	return (CallstackLenIndex >> EntryLenShift) == 0;
}

/////////////////////////////////////////////////////////////////////
inline uint32 FCallstack::Num() const
{
	return CallstackLenIndex >> EntryLenShift;
}

/////////////////////////////////////////////////////////////////////
inline uint64 FCallstack::Addr(uint8 Depth) const
{
	const FStackFrame* FirstFrame = (const FStackFrame *) (~EntryLenMask & CallstackLenIndex);
	const uint32 FrameCount = Num();
	return (Depth < FrameCount) ? (FirstFrame + Depth)->Addr : 0;
}

/////////////////////////////////////////////////////////////////////
inline const TCHAR* FCallstack::Name(uint8 Depth) const
{
	const FStackFrame* FirstFrame = (const FStackFrame*)(~EntryLenMask & CallstackLenIndex);
	const uint32 FrameCount = Num();
	return (Depth < FrameCount) ? (FirstFrame + Depth)->Symbol->Name : 0;
}

/////////////////////////////////////////////////////////////////////
inline const FStackFrame* FCallstack::Frame(uint8 Depth) const
{
	const FStackFrame* FirstFrame = (const FStackFrame*)(~EntryLenMask & CallstackLenIndex);
	const uint32 FrameCount = Num();
	return (Depth < FrameCount) ? (FirstFrame + Depth) : nullptr;
}

} // namespace TraceServices

#undef UE_API
