// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FChannelEntry
{
	uint32 Id;
	FString Name;
	bool bIsEnabled;
	bool bReadOnly;
};

class IChannelProvider
	: public IProvider
{
public:
	virtual ~IChannelProvider() = default;
	virtual uint64 GetChannelCount() const = 0;
	virtual const TArray<FChannelEntry>& GetChannels() const = 0;
	virtual FDateTime GetTimeStamp() const = 0;
};

TRACESERVICES_API FName GetChannelProviderName();
TRACESERVICES_API const IChannelProvider* ReadChannelProvider(const IAnalysisSession& Session);

} // namespace TraceServices
