// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FContextSwitch
{
	double Start;
	double End;
	uint32 CoreNumber;
};

struct FCpuCoreInfo
{
	uint32 CoreNumber;
};

struct FCpuCoreEvent
{
	double Start;
	double End;
	uint32 SystemThreadId;
};

enum class EContextSwitchEnumerationResult
{
	Continue,
	Stop,
};

class IContextSwitchesProvider
	: public IProvider
{
public:
	typedef TFunctionRef<void(const FCpuCoreInfo&)> CpuCoreCallback;
	typedef TFunctionRef<EContextSwitchEnumerationResult(const FCpuCoreEvent&)> CpuCoreEventCallback;
	typedef TFunctionRef<EContextSwitchEnumerationResult(const FContextSwitch&)> ContextSwitchCallback;

	virtual ~IContextSwitchesProvider() = default;
	virtual bool HasData() const = 0;
	virtual bool GetSystemThreadId(uint32 ThreadId, uint32& OutSystemThreadId) const = 0;
	virtual bool GetThreadId(uint32 SystemThreadId, uint32& OutThreadId) const = 0;
	virtual bool GetSystemThreadId(uint32 CoreNumber, double Time, uint32& OutSystemThreadId) const = 0;
	virtual bool GetThreadId(uint32 CoreNumber, double Time, uint32& OutThreadId) const = 0;
	virtual bool GetCoreNumber(uint32 ThreadId, double Time, uint32& OutCoreNumber) const = 0;
	virtual uint64 GetThreadsSerial() const = 0;
	virtual uint64 GetCpuCoresSerial() const = 0;
	virtual void EnumerateCpuCores(CpuCoreCallback Callback) const = 0;
	virtual void EnumerateCpuCoreEvents(uint32 CoreNumber, double StartTime, double EndTime, CpuCoreEventCallback Callback) const = 0;
	virtual void EnumerateCpuCoreEventsBackwards(uint32 CoreNumber, double EndTime, double StartTime, CpuCoreEventCallback Callback) const = 0;
	virtual void EnumerateContextSwitches(uint32 ThreadId, double StartTime, double EndTime, ContextSwitchCallback Callback) const = 0;
};

TRACESERVICES_API FName GetContextSwitchesProviderName();
TRACESERVICES_API const IContextSwitchesProvider* ReadContextSwitchesProvider(const IAnalysisSession& Session);

} // namespace TraceServices
