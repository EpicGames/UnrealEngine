// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ProfilingDebugging/CookStats.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FPackageData
{
	FPackageData(uint64 InId);
	uint64 Id = -1;
	const TCHAR* Name;

	double LoadTimeIncl = 0;
	double LoadTimeExcl = 0;

	double SaveTimeIncl = 0;
	double SaveTimeExcl = 0;

	double BeginCacheForCookedPlatformDataIncl = 0; // BeginCacheForCookedPlatformData is the name of the function from the cooker.
	double BeginCacheForCookedPlatformDataExcl = 0;

	double IsCachedCookedPlatformDataLoadedIncl = 0; // IsCachedCookedPlatformDataLoaded is the name of the function from the cooker.
	double IsCachedCookedPlatformDataLoadedExcl = 0;

	const TCHAR* AssetClass;
};

class ICookProfilerProvider : public IProvider
{
public:
	typedef TFunctionRef<bool(const FPackageData& /*Package*/)> EnumeratePackagesCallback;

	virtual ~ICookProfilerProvider() = default;

	virtual void BeginRead() const = 0;
	virtual void EndRead() const = 0;
	virtual void ReadAccessCheck() const = 0;

	virtual uint32 GetNumPackages() const = 0;
	virtual void EnumeratePackages(double StartTime, double EndTime, EnumeratePackagesCallback Callback) const = 0;
	
	virtual void CreateAggregation(TArray64<FPackageData>& OutPackages) const = 0;
};

class IEditableCookProfilerProvider : public IEditableProvider
{
public:
	virtual ~IEditableCookProfilerProvider() = default;

	virtual void BeginEdit() const = 0;
	virtual void EndEdit() const = 0;
	virtual void EditAccessCheck() const = 0;

	virtual FPackageData* EditPackage(uint64 Id) = 0;

	virtual void AddScopeEntry(uint32 ThreadId, uint64 InPackageId, double Timestamp, EPackageEventStatType InType, bool InIsEnterScope) = 0;
};

TRACESERVICES_API FName GetCookProfilerProviderName();
TRACESERVICES_API const ICookProfilerProvider* ReadCookProfilerProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableCookProfilerProvider* EditCookProfilerProvider(IAnalysisSession& Session);

} // namespace TraceServices
