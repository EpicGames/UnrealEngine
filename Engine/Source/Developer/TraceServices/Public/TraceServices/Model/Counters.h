// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

enum ECounterDisplayHint
{
	CounterDisplayHint_None,
	CounterDisplayHint_Memory,
	CounterDisplayHint_Bandwidth,
	CounterDisplayHint_Percent,
};

enum class ECounterOpType : uint8
{
	Set = 0,
	Add = 1,
};

class ICounter
{
public:
	virtual ~ICounter() = default;

	virtual const TCHAR* GetName() const = 0;
	virtual const TCHAR* GetGroup() const = 0;
	virtual const TCHAR* GetDescription() const = 0;
	virtual bool IsFloatingPoint() const = 0;
	virtual bool IsResetEveryFrame() const { return false; }
	virtual ECounterDisplayHint GetDisplayHint() const = 0;
	virtual void EnumerateValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, int64)> Callback) const = 0;
	virtual void EnumerateFloatValues(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, double)> Callback) const = 0;
	virtual void EnumerateOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, int64)> Callback) const = 0;
	virtual void EnumerateFloatOps(double IntervalStart, double IntervalEnd, bool bIncludeExternalBounds, TFunctionRef<void(double, ECounterOpType, double)> Callback) const = 0;
};

class IEditableCounterProvider;

/*
* An interface that can consume mutations of the state of a counter.
*/
class IEditableCounter
{
public:
	virtual ~IEditableCounter() = default;

	/*
	* Sets the name of the counter.
	*
	* @param Name	The name of the counter.
	*/
	virtual void SetName(const TCHAR* Name) = 0;

	/*
	* Sets the group of the counter.
	*
	* @param Group	The group name of the counter.
	*/
	virtual void SetGroup(const TCHAR* Group) = 0;

	/*
	* Sets the description of the counter.
	*
	* @param Description	The description of the counter.
	*/
	virtual void SetDescription(const TCHAR* Description) = 0;

	/*
	* Sets whether the counter is an integral or floating point number.
	*
	* @param bIsFloatingPoint	True if the counter is floating point, false if it is integral.
	*/
	virtual void SetIsFloatingPoint(bool bIsFloatingPoint) = 0;

	/*
	* Sets whether the counter value is reset every frame. This can be used for counters polled from stats.
	*
	* @param bInIsResetEveryFrame	True if the counter is reset every frame.
	*/
	virtual void SetIsResetEveryFrame(bool bInIsResetEveryFrame) = 0;

	/*
	* Sets the display hint for this counter.
	*
	* @param DisplayHint	The display hint for this counter.
	*/
	virtual void SetDisplayHint(ECounterDisplayHint DisplayHint) = 0;

	/*
	* Add a value to the value counter.
	*
	* @param Time	The time at which the value was added in seconds.
	* @param Value	The value to add to the current value of the counter.
	*/
	virtual void AddValue(double Time, int64 Value) = 0;

	/*
	* Add a value to the value counter.
	*
	* @param Time	The time at which the value was added in seconds.
	* @param Value	The value to add to the current value of the counter.
	*/
	virtual void AddValue(double Time, double Value) = 0;

	/*
	* Set the value of the counter.
	*
	* @param Time	The time at which the value was set in seconds.
	* @param Value	The new value for the counter.
	*/
	virtual void SetValue(double Time, int64 Value) = 0;

	/*
	* Set the value of the counter.
	*
	* @param Time	The time at which the value was set in seconds.
	* @param Value	The new value for the counter.
	*/
	virtual void SetValue(double Time, double Value) = 0;
};

class ICounterProvider
	: public IProvider
{
public:
	virtual ~ICounterProvider() = default;
	virtual uint64 GetCounterCount() const = 0;
	virtual bool ReadCounter(uint32 CounterId, TFunctionRef<void(const ICounter&)> Callback) const = 0;
	virtual void EnumerateCounters(TFunctionRef<void(uint32, const ICounter&)> Callback) const = 0;
};

/*
* The interface to a provider that can consume mutations of counter events from a session.
*/
class IEditableCounterProvider
	: public IEditableProvider
{
public:
	virtual ~IEditableCounterProvider() = default;

	/*
	* Retrieve the counter interface to an editable counter.
	*
	* @return The interface the counter.
	*         This can be null if the provider doesn't care about deriving new counters from existing editable counters.
	*/
	virtual const ICounter* GetCounter(IEditableCounter* EditableCounter) = 0;

	/*
	* Create a new counter mutation interface.
	*
	* @return The interface to mutate the counter.
	*/
	virtual IEditableCounter* CreateEditableCounter() = 0;

	/*
	* Add a custom counter to the provider.
	*/
	virtual void AddCounter(const ICounter* Counter) = 0;
};

TRACESERVICES_API FName GetCounterProviderName();
TRACESERVICES_API const ICounterProvider& ReadCounterProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableCounterProvider& EditCounterProvider(IAnalysisSession& Session);

} // namespace TraceServices
