// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

class IUntypedTable;

struct FCaptureInfo
{
	const TMap<const TCHAR*, const TCHAR*, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, const TCHAR*>>& Metadata;
	const TCHAR* Filename = nullptr;
	uint32 Id = uint32(-1);
	uint32 FrameCount = 0;
};

class ICsvProfilerProvider
	: public IProvider
{
public:
	virtual ~ICsvProfilerProvider() = default;
	virtual void EnumerateCaptures(TFunctionRef<void(const FCaptureInfo&)> Callback) const = 0;
	virtual const IUntypedTable& GetTable(uint32 CaptureId) const = 0;
};

TRACESERVICES_API FName GetCsvProfilerProviderName();
TRACESERVICES_API const ICsvProfilerProvider* ReadCsvProfilerProvider(const IAnalysisSession& Session);

} // namespace TraceServices
