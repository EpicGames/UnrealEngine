// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Trace/Analyzer.h" // TraceAnalysis
#include "UObject/NameTypes.h"

namespace TraceServices
{

/**
 * Allows users to publish "definitions", structs representing a definition event. This data can then be queried when
 * resolving a reference field in another event.
 *
 * \note If your definition type reference other definitions it is encouraged to resolve those during analysis to
 * avoid having to resolve complex chains on every lookup.
 *
 * Example usage:
 * This code creates a definition, initializes the instance, resolves a reference and finally registers it.
 * \code
 *		FMyDefinitionClass* Instance = DefinitionProvider->Create<FMyDefinitionClass>();
 *		Instance->SomeValue = EventData.GetValue<uint32>("SomeValue");
 *		const FEventRef Reference = EventData.GetReferenceValue("SomeString");
 *		Instance->SomeString = DefinitionProvider->Get<FStringDefinition>(Reference)->Display;
 *		...
 *
 *		const UE::Trace::FEventRef Id(EventData.GetDefinitionId(), EventData.GetTypeInfo().GetId());
 *		DefinitionProvider->Register<FMyDefinitionClass>(Instance, Id);
 * \endcode
 */
class IDefinitionProvider
	: public IProvider
	, public IEditableProvider
{
public:
	virtual ~IDefinitionProvider() = default;

	//////////////////////////////////////////////////
	// Edit operations

	/**
	 * Allocates memory for the definition. The memory is guaranteed to be valid during the lifetime of the
	 * analysis session.
	 * @return Pointer to zeroed memory enough to hold an instance of T.
	 */
	template<typename T>
	T* Create()
	{
		const uint32 Size = sizeof(T);
		const uint32 Alignment = alignof(T);
		T* Allocated = (T*) Allocate(Size, Alignment);
		return Allocated;
	}

	/**
	 * Makes the instance of a definition visible to queries.
	 * @param Instance Pointer to the instance
	 * @param Id Id that should be used to reference the definition.
	 */
	template<typename T, typename DefinitionType>
	void Register(const T* Instance, const UE::Trace::TEventRef<DefinitionType>& Id)
	{
		Stringifiers.FindOrAdd(Id.RefTypeId, &T::ToString);
		AddEntry(Id.GetHash(), Instance);
	}

	//////////////////////////////////////////////////
	// Read operations

	/**
	 * Attempts to retrieve a previously registered instance of a definition using a reference.
	 * @param Reference Id used to uniquely identify the definition.
	 * @return Pointer to the definition or null if the id was not found.
	 */
	template<typename T, typename DefinitionType>
	const T* Get(const UE::Trace::TEventRef<DefinitionType>& Reference) const
	{
		const void* Value = FindEntry(Reference.GetHash());
		return (T*) Value;
	}

	/**
	 * Gets the definition as a owned string.
	 * @param Reference Id used to uniquely identify the definition.
	 * @return A string describing the referenced value
	 */
	template<typename DefinitionType>
	FString GetAsString(const UE::Trace::TEventRef<DefinitionType>& Reference) const
	{
		const void* Entry = FindEntry(Reference.GetHash());
		auto Stringifier = Stringifiers.Find(Reference.RefTypeId);
		if (Entry && Stringifier)
		{
			return (*Stringifier)(Entry);
		}
		return FString();
	}

protected:
	virtual void* Allocate(uint32 Size, uint32 Alignment) = 0;
	virtual void AddEntry(uint64 Hash, const void* Ptr) = 0;
	virtual const void* FindEntry(uint64 Hash) const = 0;

protected:
	using StringifierFn = TFunction<FString (const void*)>;
	TMap<uint32, StringifierFn> Stringifiers;
};

TRACESERVICES_API FName GetDefinitionProviderName();
TRACESERVICES_API const IDefinitionProvider* ReadDefinitionProvider(const IAnalysisSession& Session);
TRACESERVICES_API IDefinitionProvider* EditDefinitionProvider(IAnalysisSession& Session);

} // namespace TraceServices
