// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FSessionInfo
{
	FString Platform;
	FString AppName;
	FString ProjectName;
	FString EngineVersion;
	FString BuildVersion;
	FString Branch;
	FString CommandLine;
	FString VFSPaths;
	uint32 Changelist = 0;
	EBuildConfiguration ConfigurationType = EBuildConfiguration::Unknown;
	EBuildTargetType TargetType = EBuildTargetType::Unknown;
	FGuid InstanceId;
};

class IDiagnosticsProvider : public IProvider
{
public:
	virtual ~IDiagnosticsProvider() = default;

	virtual bool IsSessionInfoAvailable() const = 0;

	virtual const FSessionInfo& GetSessionInfo() const = 0;
};

TRACESERVICES_API FName GetDiagnosticsProviderName();
TRACESERVICES_API const IDiagnosticsProvider* ReadDiagnosticsProvider(const IAnalysisSession& Session);

} // namespace TraceServices
