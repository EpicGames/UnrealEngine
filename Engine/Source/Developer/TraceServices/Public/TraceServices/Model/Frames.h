// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FFrame
{
	uint64 Index;
	double StartTime;
	double EndTime;
	ETraceFrameType FrameType;
};

class IFrameProvider
	: public IProvider
{
public:
	virtual ~IFrameProvider() = default;

	/**
	 * Gets the number of frames, for the specified frame type.
	 * @param FrameType - type of frames
	 * @returns number of frames
	 */
	virtual uint64 GetFrameCount(ETraceFrameType FrameType) const = 0;

	/**
	 * Enumerates frames in the [Start, End) index range, for the specified frame type.
	 * @param FrameType - type of frames
	 * @param Start     - inclusive start index
	 * @param End       - exclusive end index
	 * @param Callback  - a callback called for each frame
	 */
	virtual void EnumerateFrames(ETraceFrameType FrameType, uint64 Start, uint64 End, TFunctionRef<void(const FFrame&)> Callback) const = 0;
	/**
	* Enumerate frames whose duration intersects [StartTime, EndTime].
	*
	* @param FrameType	The frame type to enumerate.
	* @param StartTime	The start timestamp in seconds.
	* @param EndTime	The end timestamp in seconds.
	* @param Callback	The callback to be called for each frame.
	*/
	virtual void EnumerateFrames(ETraceFrameType FrameType, double StartTime, double EndTime, TFunctionRef<void(const FFrame&)> Callback) const = 0;

	/**
	 * Gets the array of frame start times, for the specified frame type.
	 * @param FrameType - type of frames
	 * @returns a const reference to array of time values
	 */
	virtual const TArray64<double>& GetFrameStartTimes(ETraceFrameType FrameType) const = 0;

	/**
	 * Gets the first frame with start time <= specified time, for the specified frame type.
	 * Note: This function does not check the end time of frames.
	 * @param FrameType - type of frames
	 * @param Time      - time value, in seconds
	 * @param OutFrame  - the output frame, if this function returns true
	 * @returns true if there is a frame with start time <= time or false otherwise
	 */
	virtual bool GetFrameFromTime(ETraceFrameType FrameType, double Time, FFrame& OutFrame) const = 0;

	/**
	 * Gets the frame at the specified index, for the specified frame type.
	 * @param FrameType - type of frames
	 * @param Index     - frame index
	 * @returns a const pointer to the frame or nullptr if index is invalid
	 */
	virtual const FFrame* GetFrame(ETraceFrameType FrameType, uint64 Index) const = 0;

	/**
	 * Gets the index of the first frame with start time <= specified time, for the specified frame type.
	 * Note: This function does not check the end time of frames.
	 * @param FrameType - type of frames
	 * @param Time      - time value, in seconds
	 * @returns index of the first frame with start time <= time; also returns 0 when time < start time of the first frame or if frame count is zero
	 */
	virtual uint32 GetFrameNumberForTimestamp(ETraceFrameType FrameType, double Time) const = 0;
};

TRACESERVICES_API FName GetFrameProviderName();
TRACESERVICES_API const IFrameProvider& ReadFrameProvider(const IAnalysisSession& Session);

} // namespace TraceServices
