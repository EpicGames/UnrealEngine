// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Serialization/LoadTimeTrace.h"
#include "Templates/Function.h"
#include "TraceServices/Containers/Tables.h"
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

template <typename RowType> class ITable;

struct FFileInfo
{
	uint32 Id = uint32(-1);
	const TCHAR* Path = nullptr;
};

enum EFileActivityType
{
	FileActivityType_Open,
	FileActivityType_ReOpen,
	FileActivityType_Close,
	FileActivityType_Read,
	FileActivityType_Write,

	FileActivityType_Count,
	FileActivityType_Invalid = FileActivityType_Count
};

TRACESERVICES_API const TCHAR* GetFileActivityTypeString(EFileActivityType ActivityType);

struct FFileActivity
{
	const FFileInfo* File = nullptr;
	double StartTime = 0.0;
	double EndTime = 0.0;
	uint64 Offset = 0;
	uint64 Size = 0;
	uint64 ActualSize = 0;
	uint32 ThreadId = 0;
	EFileActivityType ActivityType = FileActivityType_Invalid;
	bool Failed = false;
	uint64 FileHandle;
	uint64 ReadWriteHandle;
};

class IFileActivityProvider
	: public IProvider
{
public:
	typedef ITimeline<FFileActivity*> Timeline;

	virtual ~IFileActivityProvider() = default;
	virtual void EnumerateFileActivity(TFunctionRef<bool(const FFileInfo&, const Timeline&)> Callback) const = 0;
	virtual const ITable<FFileActivity>& GetFileActivityTable() const = 0;
};

struct FPackageSummaryInfo
{
	uint32 TotalHeaderSize = 0;
	uint32 ImportCount = 0;
	uint32 ExportCount = 0;
	int32 Priority = 0;
};

struct FClassInfo
{
	const TCHAR* Name;
};

struct FPackageInfo;

struct FPackageExportInfo
{
	uint32 Id;
	const FClassInfo* Class = nullptr;
	uint64 SerialSize = 0;
	const FPackageInfo* Package = nullptr;
};

struct FPackageInfo
{
	uint32 Id;
	const TCHAR* Name = nullptr;
	FPackageSummaryInfo Summary;
	TArray<const FPackageInfo*> ImportedPackages;
	TArray<const FPackageExportInfo*> Exports;
	uint64 TotalExportsSerialSize = 0;
	uint64 RequestId = 0;
};

struct FLoadRequest
{
	uint64 Id = 0;
	const TCHAR* Name = nullptr;
	uint32 ThreadId = uint32(-1);
	double StartTime = 0.0;
	double EndTime = 0.0;
	TArray<const FPackageInfo*> Packages;
};

enum ELoadTimeProfilerObjectEventType
{
	LoadTimeProfilerObjectEventType_Create,
	LoadTimeProfilerObjectEventType_Serialize,
	LoadTimeProfilerObjectEventType_PostLoad,
	LoadTimeProfilerObjectEventType_None
};

struct FExportsTableRow
{
	const FPackageExportInfo* ExportInfo = nullptr;
	uint64 SerializedSize = 0;
	double MainThreadTime = 0.0;
	double AsyncLoadingThreadTime = 0.0;
	ELoadTimeProfilerObjectEventType EventType = LoadTimeProfilerObjectEventType_None;
};

struct FPackagesTableRow
{
	const FPackageInfo* PackageInfo = nullptr;
	uint64 TotalSerializedSize = 0;
	uint64 SerializedHeaderSize = 0;
	uint64 SerializedExportsCount = 0;
	uint64 SerializedExportsSize = 0;
	double MainThreadTime = 0.0;
	double AsyncLoadingThreadTime = 0.0;
};

struct FRequestsTableRow
{
	uint64 Id = 0;
	const TCHAR* Name = nullptr;
	double StartTime = 0.0;
	double Duration = 0.0;
	TArray<const FPackageInfo*> Packages;
};

struct FLoadTimeProfilerCpuEvent
{
	const FPackageInfo* Package = nullptr;
	const FPackageExportInfo* Export = nullptr;
	ELoadTimeProfilerObjectEventType EventType = LoadTimeProfilerObjectEventType_None;
};

struct FLoadTimeProfilerAggregatedStats
{
	const TCHAR* Name;
	uint64 Count;
	double Total;
	double Min;
	double Max;
	double Average;
	double Median;
};

TRACESERVICES_API const TCHAR* GetLoadTimeProfilerObjectEventTypeString(ELoadTimeProfilerObjectEventType EventType);

class ILoadTimeProfilerProvider
	: public IProvider
{
public:
	typedef ITimeline<FLoadTimeProfilerCpuEvent> CpuTimeline;

	virtual ~ILoadTimeProfilerProvider() = default;
	virtual uint32 GetTimelineCount() const = 0;
	virtual bool GetCpuThreadTimelineIndex(uint32 ThreadId, uint32& OutTimelineIndex) const = 0;
	virtual bool ReadTimeline(uint32 Index, TFunctionRef<void(const CpuTimeline&)> Callback) const = 0;
	virtual ITable<FLoadTimeProfilerAggregatedStats>* CreateEventAggregation(double IntervalStart, double IntervalEnd) const = 0;
	virtual ITable<FLoadTimeProfilerAggregatedStats>* CreateObjectTypeAggregation(double IntervalStart, double IntervalEnd) const = 0;
	virtual ITable<FPackagesTableRow>* CreatePackageDetailsTable(double IntervalStart, double IntervalEnd) const = 0;
	virtual ITable<FExportsTableRow>* CreateExportDetailsTable(double IntervalStart, double IntervalEnd) const = 0;
	virtual ITable<FRequestsTableRow>* CreateRequestsTable(double IntervalStart, double IntervalEnd) const = 0;
	virtual const ITable<FLoadRequest>& GetRequestsTable() const = 0;
};

TRACESERVICES_API FName GetLoadTimeProfilerProviderName();
TRACESERVICES_API const ILoadTimeProfilerProvider* ReadLoadTimeProfilerProvider(const IAnalysisSession& Session);

TRACESERVICES_API FName GetFileActivityProviderName();
TRACESERVICES_API const IFileActivityProvider* ReadFileActivityProvider(const IAnalysisSession& Session);

} // namespace TraceServices
