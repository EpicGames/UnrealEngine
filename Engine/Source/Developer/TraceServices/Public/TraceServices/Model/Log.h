// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Logging/LogVerbosity.h"
#include "Templates/Function.h"
#include "TraceServices/Containers/Tables.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{
class IUntypedTable;

struct FLogCategoryInfo
{
	const TCHAR* Name = nullptr;
	ELogVerbosity::Type DefaultVerbosity;
};

struct FLogMessageInfo
{
	uint64 Index;
	double Time;
	const FLogCategoryInfo* Category = nullptr;
	const TCHAR* File = nullptr;
	const TCHAR* Message = nullptr;
	int32 Line;
	ELogVerbosity::Type Verbosity;
};

class ILogProvider
	: public IProvider
{
public:
	virtual ~ILogProvider() = default;

	/**
	 * Gets the number of log messages.
	 *
	 * @returns The current number of log messages.
	 */
	virtual uint64 GetMessageCount() const = 0;

	/**
	 * Reads information for a single message specified by index.
	 *
	 * @param Index The index of the message to read.
	 * @param Callback The function to be called with the log message's information.
	 * @returns Whether Index is valid or not.
	 *
	 * @note The current number of messages could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual bool ReadMessage(uint64 Index, TFunctionRef<void(const FLogMessageInfo&)> Callback) const = 0;

	/**
	 * Enumerates messages in the specified index interval [StartIndex, EndIndex).
	 *
	 * @param StartIndex The inclusive start index of the interval.
	 * @param EndIndex The exclusive end index of the interval.
	 * @param Callback The function to be called for each log message with the message's information.
	 *
	 * @note The current number of messages could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual void EnumerateMessagesByIndex(uint64 StartIndex, uint64 EndIndex, TFunctionRef<void(const FLogMessageInfo&)> Callback) const = 0;

	/**
	 * Enumerates messages with timestamp in the specified time interval [StartTime, EndTime].
	 *
	 * @param StartTime The inclusive start time of the interval.
	 * @param EndTime The inclusive end time of the interval.
	 * @param Callback The function to be called for each log message with the message's information.
	 *
	 * @note The current number of messages could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual void EnumerateMessages(double StartTime, double EndTime, TFunctionRef<void(const FLogMessageInfo&)> Callback) const = 0;

	/**
	 * Enumerates messages in reverse with timestamp in the specified time interval [EndTime, StartTime] until either it finds the last message
	 * or is stopped on callback.
	 *
	 * @param StartTime The inclusive start time of the interval.
	 * @param EndTime The inclusive end time of the interval.
	 * @param Callback The function to be called for each log message with the message's information. Return True to signify a stop, False to continue.
	 *
	 * @note The current number of messages could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual void ReverseEnumerateMessages(double StartTime, double EndTime, TFunctionRef<bool(const FLogMessageInfo&)> Callback) const = 0;

	/**
	 * Performs binary search, resulting in position of the first log message with time >= provided time value.
	 *
	 * @param Time The time value, in seconds.
	 * @returns The index of the first log message with time >= provided time value; [0, MessageCount]. Returns ~0ull if not implemented.
	 *
	 * @note The current number of messages (MessageCount) could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual uint64 LowerBoundByTime(double Time) const { return ~0ull; }

	/**
	 * Performs binary search, resulting in position of the first log message with time > provided time value.
	 *
	 * @param Time The time value, in seconds.
	 * @returns The index of the first log message with time > provided time value; [0, MessageCount]. Returns ~0ull if not implemented.
	 *
	 * @note The current number of messages (MessageCount) could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual uint64 UpperBoundByTime(double Time) const { return ~0ull; }

	/**
	 * Finds the log message with closest timestamp to the specified time.
	 *
	 * @param Time The time value, in seconds.
	 * @returns The index of the message with the closest timestamp to the specified time.
	 *          If MessageCount is 0, this function returns 0; otherwise it returns a valid index in range [0, MessageCount-1]. Returns ~0ull if not implemented.
	 *
	 * @note The current number of messages (MessageCount) could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual uint64 BinarySearchClosestByTime(double Time) const { return ~0ull; }

	/**
	 * Gets the number of log message specs.
	 *
	 * @returns The current number of log message specs.
	 */
	virtual uint64 GetMessageSpecCount() const { return 0; }

	/**
	 * Gets the number of log categories.
	 *
	 * @returns The current number of log categories.
	 */
	virtual uint64 GetCategoryCount() const = 0;

	/**
	 * Enumerates the log categories.
	 *
	 * @param Callback The function to be called for each log category with the category's information.
	 *
	 * @note The current number of messages could be different than the value returned by the last call to GetMessageCount().
	 */
	virtual void EnumerateCategories(TFunctionRef<void(const FLogCategoryInfo&)> Callback) const = 0;

	/**
	 * Gets the untyped table for log messages.
	 *
	 * @returns The IUntypedTable reference to the messages table.
	 */
	virtual const IUntypedTable& GetMessagesTable() const = 0;

	/**
	 * Gets the number of inserts (when a message is inserted before other messages).
	 *
	 * @returns The number of inserts.
	 */
	virtual uint64 GetInsertCount() const { return 0; }

	/**
	 * Gets the total number of messages received with unknown message spec.
	 *
	 * @returns The total number of messages received with unknown message spec.
	 */
	virtual uint64 GetUnknownSpecTotalCount() const { return 0; }

	/**
	 * Gets the number of messages received with unknown message spec and still unresolved.
	 *
	 * @returns The number of messages received with unknown message spec and still unresolved.
	 */
	virtual uint64 GetUnresolvedSpecCount() const { return 0; }
};

class IEditableLogProvider
	: public IEditableProvider
{
public:
	virtual ~IEditableLogProvider() = default;

	/**
	 * Registers a new log message category.
	 *
	 * @return The category identity.
	 */
	virtual uint64 RegisterCategory() = 0;

	/**
	 * Fetches the data structure for a log category.
	 *
	 * @param CategoryPointer The unique identity (memory address) of the category instrumentation.
	 * @return A reference to the category structure.
	 */
	virtual FLogCategoryInfo& GetCategory(uint64 CategoryPointer) = 0;

	/**
	 * Updates the Category information for a log message.
	 *
	 * @param LogPoint The log message to update.
	 * @param InCategoryPointer The category.
	 */
	virtual void UpdateMessageCategory(uint64 LogPoint, uint64 InCategoryPointer) = 0;

	/**
	 * Updates the format string for a log message.
	 *
	 * @param LogPoint The log message to update.
	 * @param InFormatString The format string whose memory is stored in the Session.
	 */
	virtual void UpdateMessageFormatString(uint64 LogPoint, const TCHAR* InFormatString) = 0;

	/**
	 * Updates the file location for a log message.
	 *
	 * @param LogPoint The log message to update.
	 * @param InFile The file path of the message's location.
	 * @param InLine The line number of the message's location.
	 */
	virtual void UpdateMessageFile(uint64 LogPoint, const TCHAR* InFile, int32 InLine) = 0;

	/**
	 * Updates the verbosity for a log message.
	 *
	 * @param LogPoint The log message to update.
	 * @param InVerbosity The verbosity of the message.
	 */
	virtual void UpdateMessageVerbosity(uint64 LogPoint, ELogVerbosity::Type InVerbosity) = 0;

	/**
	 * Updates information for a log message.
	 *
	 * @param LogPoint The log message to update.
	 * @param InCategoryPointer The category.
	 * @param InFormatString The format string whose memory is stored in the Session.
	 * @param InFile The file path of the message's location.
	 * @param InLine The line number of the message's location.
	 * @param InVerbosity The verbosity of the message.
	 */
	virtual void UpdateMessageSpec(uint64 LogPoint, uint64 InCategoryPointer, const TCHAR* InFormatString, const TCHAR* InFile, int32 InLine, ELogVerbosity::Type InVerbosity) = 0;

	/**
	 * Updates information for a log message after message spec was updated.
	 */
	virtual void UpdateMessageSpec(uint64 LogPoint) {}

	/**
	 * Appends a new instance of a message from the trace session.
	 *
	 * @param LogPoint The unique identity (memory address) of the message instrumentation.
	 * @param Time The time in seconds of the event.
	 * @param FormatArgs The arguments to use in conjunction with the spec's FormatString.
	 */
	UE_DEPRECATED(5.8, "Use AppendMessage(uint64, double, TArrayView<const uint8>) instead.")
	virtual void AppendMessage(uint64 LogPoint, double Time, const uint8* FormatArgs) {}

	/**
	 * Appends a new instance of a message from the trace session.
	 *
	 * @param LogPoint The unique identity (memory address) of the message instrumentation.
	 * @param Time The time in seconds of the event.
	 * @param FormatArgsView The arguments to use in conjunction with the spec's FormatString.
	 */
	virtual void AppendMessage(uint64 LogPoint, double Time, TArrayView<const uint8> FormatArgsView) {}

	/**
	 * Appends a new instance of a message from the trace session.
	 *
	 * @param LogPoint The unique identity (memory address) of the message instrumentation.
	 * @param Time The time in seconds of the event.
	 * @param StoredText The persistent message text.
	 */
	virtual void AppendMessage(uint64 LogPoint, double Time, const TCHAR* StoredText) = 0;
};

TRACESERVICES_API FName GetLogProviderName();
TRACESERVICES_API const ILogProvider& ReadLogProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableLogProvider& EditLogProvider(IAnalysisSession& Session);
TRACESERVICES_API void FormatString(TCHAR* OutputString, uint32 OutputStringCount, const TCHAR* FormatString, const uint8* FormatArgs);

} // namespace TraceServices
