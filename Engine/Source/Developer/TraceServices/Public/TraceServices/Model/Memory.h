// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

typedef int32 FMemoryTrackerId;
typedef int32 FMemoryTagSetId;
typedef int64 FMemoryTagId;

struct FMemoryTrackerInfo
{
	static constexpr FMemoryTrackerId InvalidTrackerId = -1;
	static constexpr int32 MaxTrackers = 8;

	// The unique identifier of the memory tracker. Can also be used as an index, limited to [0 .. MaxTrackers-1] range.
	FMemoryTrackerId Id;

	// The name of the memory tracker.
	FString Name;
};

struct FMemoryTagSetInfo
{
	static constexpr FMemoryTagSetId InvalidTagSetId = -1;
	static constexpr int32 MaxTagSets = 8;

	// The unique identifier of the memory tag set. Can also be used as an index, limited to [0 .. MaxTagSets-1] range.
	FMemoryTagSetId Id;

	FString Name;
};

struct FMemoryTagInfo
{
	static constexpr FMemoryTagId InvalidTagId = 0;

	// The unique identifier of the memory tag.
	FMemoryTagId Id;

	// The id of the parent tag, 0 if no parent.
	FMemoryTagId ParentId;

	// The set id of the memory tag.
	FMemoryTagSetId TagSetId;

	// Bit flags for trackers using this memory tag.
	// The bit position represents the tracker id; this limits the valid tracker ids to range [0 .. 63].
	// It can be updated during analysis (as new trackers / snapshots are analyzed).
	uint64 Trackers;

	// The name of the memory tag.
	FString Name;
};

struct FMemoryTagSample
{
	// Value at sample time.
	int64 Value;
};

class IMemoryProvider : public IProvider
{
public:
	virtual ~IMemoryProvider() = default;

	virtual void BeginRead() const = 0;
	virtual void EndRead() const = 0;
	virtual void ReadAccessCheck() const = 0;

	/**
	 * @returns Whether the provider is initialized or not.
	 */
	virtual bool IsInitialized() const = 0;

	/**
	 * When this provider is completed it cannot be further modified.
	 * @returns Whether this provider is completed or not.
	 */
	virtual bool IsCompleted() const = 0;

	/**
	 * Unique serial index that changes when new tags are registered or when the Trackers flags is updated for a tag.
	 * @return Unique index
	 */
	virtual uint32 GetTagSerial() const = 0;

	/**
	 * Return the number of registered tags.
	 * @return Number of registered tags.
	 */
	virtual uint32 GetTagCount() const = 0;

	/**
	 * Enumerate the registered tags.
	 * @param Callback Function that will be called for each registered tag.
	 */
	virtual void EnumerateTags(TFunctionRef<void(const FMemoryTagInfo&)> Callback) const = 0;

	/**
	 * Enumerate the registered tags for a specified tag set.
	 * @param Callback Function that will be called for each registered tag.
	 */
	virtual void EnumerateTags(FMemoryTagSetId TagSetId, TFunctionRef<void(const FMemoryTagInfo&)> Callback) const = 0;

	/**
	 * Returns the meta data for a memory tag specified by id.
	 * @param TagId The id of the memory tag.
	 * @return Memory tag information.
	 */
	virtual const FMemoryTagInfo* GetTag(FMemoryTagId TagId) const = 0;

	/**
	 * Gets the number of samples for a given tag from a given tracker.
	 * @param Tracker Tracer index.
	 * @param TagId The id of the memory tag.
	 * @return Number of samples that has been recorded.
	 */
	virtual uint64 GetTagSampleCount(FMemoryTrackerId Tracker, FMemoryTagId TagId) const = 0;

	/**
	 * Return the number of registered trackers.
	 * @return Number of trackers.
	 */
	virtual uint32 GetTrackerCount() const = 0;

	/**
	 * Enumerate the registered trackers.
	 * @param Callback Function that is called for each registered tracker.
	 */
	virtual void EnumerateTrackers(TFunctionRef<void(const FMemoryTrackerInfo&)> Callback) const = 0;

	/**
	 * Return the number of registered tag sets.
	 * @return Number of tag sets.
	 */
	virtual uint32 GetTagSetCount() const = 0;

	/**
	 * Enumerate the registered tag sets.
	 * @param Callback Function that is called for each registered tag set.
	 */
	virtual void EnumerateTagSets(TFunctionRef<void(const FMemoryTagSetInfo&)> Callback) const = 0;

	/** Enumerates samples (values) for a specified LLM tag.
	 * @param Tracker The tracker index.
	 * @param TagId The id of the memory tag.
	 * @param StartSample The inclusive start index in the sample array of specified memory tag.
	 * @param EndSample The exclusive end index in the sample array of specified memory tag.
	 * @param Callback A callback function called for each sample enumerated.
	 * @param bIncludeRangeNeighbors Includes the sample immediately before and after the selected range.
	 */
	virtual void EnumerateTagSamples(FMemoryTrackerId Tracker, FMemoryTagId TagId, double StartTime, double EndTime, bool bIncludeRangeNeighbors, TFunctionRef<void(double Time, double Duration, const FMemoryTagSample&)> Callback) const = 0;
};

TRACESERVICES_API FName GetMemoryProviderName();
TRACESERVICES_API const IMemoryProvider* ReadMemoryProvider(const IAnalysisSession& Session);

} // namespace TraceServices
