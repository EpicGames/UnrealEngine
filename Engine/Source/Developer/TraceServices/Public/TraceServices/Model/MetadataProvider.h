// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Memory/SharedBuffer.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

class FMetadataSchema
{
public:
	enum class EFieldType : uint8
	{
		Integer,
		SignedInteger,
		FloatingPoint,
		WideStringPtr,
		Reference,
	};

	struct FField
	{
		FName			Name;
		EFieldType		Type;
		uint8			Offset;
		uint8			Size;
	};

	/**
	 * Helper class that assist in defining a schema.
	 */
	struct FBuilder
	{
		FBuilder(FField* InBuffer, uint8 InMaxFieldCount)
			: Buffer(InBuffer), MaxFieldCount(InMaxFieldCount), CurrentFieldCount(0), CurrentOffset(0)
		{
		}

		template<typename T>
		FBuilder& AddField(FName FieldName, EFieldType Type)
		{
			check(CurrentFieldCount < MaxFieldCount);
			FField* Field = &Buffer[CurrentFieldCount++];
			Field->Name = FieldName;
			Field->Type = Type;
			Field->Offset = CurrentOffset;
			Field->Size = sizeof(T);
			CurrentOffset += sizeof(T);
			return *this;
		}

		void Finish()
		{
		}

	private:
		FField* Buffer;
		uint8 MaxFieldCount;
		uint8 CurrentFieldCount;
		uint8 CurrentOffset;
	};

	/**
	 * Helper class that helps writing data based on a schema.
	 */
	struct FWriter
	{
		FWriter(const FField* InBuffer, uint8 InFieldCount)
			: Buffer(InBuffer), FieldCount(InFieldCount), Offset(0)
		{
		}

		template<typename ArrayType>
		void WriteField(uint8 Index, const void* Src, uint8 Size, ArrayType& Dst)
		{
			const FField& Field = Buffer[Index];
			if (Dst.Num() < Field.Offset + Size)
			{
				Dst.AddZeroed(Size);
			}
			void* Dest = Dst.GetData() + Field.Offset;
			FMemory::Memcpy(Dest, Src, Size);
		}

	private:
		const FField* Buffer;
		uint8 FieldCount;
		uint8 Offset;
	};

	/**
	 * Helper class that helps reading fields from data based on a schema.
	 */
	struct FReader
	{
		FReader(const FField* InBuffer, uint8 InFieldCount)
			: Buffer(InBuffer), FieldCount(InFieldCount)
		{}

		uint8 GetFieldCount() const { return FieldCount; }
		const FField& GetFieldDesc(uint8 Index) const { return Buffer[Index]; }

		FString GetString(uint8* MetaData, uint8 Index) const
		{
			const FField& Field = Buffer[Index];
			switch (Field.Type)
			{
			case EFieldType::Integer:
				{
					switch (Field.Size)
					{
					case 1: return FString::FromInt(*GetValue<uint8>(MetaData, Index));
					case 2: return FString::FromInt(*GetValue<uint16>(MetaData, Index));
					case 4: return FString::FromInt(*GetValue<uint32>(MetaData, Index));
					default: return FString::Printf(TEXT("%llu"), *GetValue<uint64>(MetaData, Index));
					}
				}
			case EFieldType::SignedInteger:
				{
					switch (Field.Size)
					{
					case 1: return FString::FromInt(*GetValue<int8>(MetaData, Index));
					case 2: return FString::FromInt(*GetValue<int16>(MetaData, Index));
					case 4: return FString::FromInt(*GetValue<int32>(MetaData, Index));
					default: return FString::Printf(TEXT("%lli"), *GetValue<int64>(MetaData, Index));
					}
				}
			case EFieldType::FloatingPoint:
				{
					switch (Field.Size)
					{
					case 4: return FString::Format(TEXT("{0}"), {*GetValue<float>(MetaData, Index)});
					default: return FString::Format(TEXT("{0}"), {*GetValue<double>(MetaData, Index)});
					}
				}
			case EFieldType::WideStringPtr:
				{
					const TCHAR* String = *GetValue<TCHAR*>(MetaData, Index);
					return FString(String);
				}
			case EFieldType::Reference:
				{
					switch (Field.Size)
					{
					case 1: { const auto* Ref = GetValueAs<UE::Trace::FEventRef8>(MetaData, Index); return FString::Format(TEXT("Ref({0},{1})"), {Ref->RefTypeId, Ref->Id});}
					case 2: { const auto* Ref = GetValueAs<UE::Trace::FEventRef16>(MetaData, Index); return FString::Format(TEXT("Ref({0},{1})"), {Ref->RefTypeId, Ref->Id});}
					case 4: { const auto* Ref = GetValueAs<UE::Trace::FEventRef32>(MetaData, Index); return FString::Format(TEXT("Ref({0},{1})"), {Ref->RefTypeId, Ref->Id});}
					default: { const auto* Ref = GetValueAs<UE::Trace::FEventRef64>(MetaData, Index); return FString::Format(TEXT("Ref({0},{1})"), {Ref->RefTypeId, Ref->Id});}
					}
				}
			}
			checkNoEntry();
			return FString();
		}

		FText GetText(uint8* MetaData, uint8 Index) const
		{
			const FField& Field = Buffer[Index];
			switch (Field.Type)
			{
			case EFieldType::Integer:
				{
					switch (Field.Size)
					{
					case 1: return FText::AsNumber(*GetValue<uint8>(MetaData, Index));
					case 2: return FText::AsNumber(*GetValue<uint16>(MetaData, Index));
					case 4: return FText::AsNumber(*GetValue<uint32>(MetaData, Index));
					default: return FText::AsNumber(*GetValue<uint64>(MetaData, Index));
					}
				}
				break;
			case EFieldType::SignedInteger:
				{
					switch (Field.Size)
					{
					case 1: return FText::AsNumber(*GetValue<int8>(MetaData, Index));
					case 2: return FText::AsNumber(*GetValue<int16>(MetaData, Index));
					case 4: return FText::AsNumber(*GetValue<int32>(MetaData, Index));
					default: return FText::AsNumber(*GetValue<int64>(MetaData, Index));
					}
				}
			case EFieldType::FloatingPoint:
				{
					switch (Field.Size)
					{
					case 4: return FText::AsNumber(*GetValue<float>(MetaData, Index));
					default: return FText::AsNumber(*GetValue<double>(MetaData, Index));
					}
				}
			case EFieldType::WideStringPtr:
				{
					const TCHAR* String = *GetValue<TCHAR*>(MetaData, Index);
					return FText::FromString(FString(String));
				}
				break;
			case EFieldType::Reference:
				{
					static const FTextFormat Fmt = FTextFormat::FromString(TEXT("Ref({0},{1})"));
					switch (Field.Size)
					{
					case 1: { const auto* Ref = GetValueAs<UE::Trace::FEventRef8>(MetaData, Index); return FText::Format(Fmt, Ref->RefTypeId, Ref->Id); }
					case 2: { const auto* Ref = GetValueAs<UE::Trace::FEventRef16>(MetaData, Index); return FText::Format(Fmt, Ref->RefTypeId, Ref->Id); }
					case 4: { const auto* Ref = GetValueAs<UE::Trace::FEventRef32>(MetaData, Index); return FText::Format(Fmt, Ref->RefTypeId, Ref->Id); }
					default: { const auto* Ref = GetValueAs<UE::Trace::FEventRef64>(MetaData, Index); return FText::Format(Fmt, Ref->RefTypeId, Ref->Id); }
					}
				}
			}
			checkNoEntry();
			return FText();
		}

		template<typename T>
		const T* GetValueAs(const uint8* Metadata, uint8 Index) const
		{
			return GetValue<T>(Metadata, Index);
		}

	private:
		template<typename T>
		const T* GetValue(const uint8* Metadata, uint8 Index) const
		{
			const FField& Field = Buffer[Index];
			return (T*)(Metadata + Field.Offset);
		}

	private:
		const FField* Buffer;
		uint8 FieldCount;
	};

	explicit FMetadataSchema(uint8 InFieldCount)
	{
		Fields.AddZeroed(InFieldCount);
	}

	FMetadataSchema(const FMetadataSchema& InSchema)
		: Fields(InSchema.Fields)
	{
		check(Fields.Num() < 256);
	}

	FBuilder Builder()
	{
		return FBuilder(Fields.GetData(), static_cast<uint8>(Fields.Num()));
	}

	FWriter Writer() const
	{
		return FWriter(Fields.GetData(), static_cast<uint8>(Fields.Num()));
	}

	FReader Reader() const
	{
		return FReader(Fields.GetData(), static_cast<uint8>(Fields.Num()));
	}

private:
	FMetadataSchema() = delete;

	TArray<FField> Fields;
};

class IMetadataProvider : public IProvider
{
public:
	static constexpr uint32 InvalidMetadataId = 0xFFFFFFFF;
	static constexpr uint16 InvalidMetadataType = 0xFF;

	virtual ~IMetadataProvider() = default;

	virtual void BeginRead() const = 0;
	virtual void EndRead() const = 0;
	virtual void ReadAccessCheck() const = 0;

	virtual uint16 GetRegisteredMetadataType(FName Name) const = 0;
	virtual FName GetRegisteredMetadataName(uint16 Type) const = 0;
	virtual const FMetadataSchema* GetRegisteredMetadataSchema(uint16) const = 0 ;

	virtual uint32 GetMetadataStackSize(uint32 InThreadId, uint32 InMetadataId) const = 0;
	virtual bool GetMetadata(uint32 InThreadId, uint32 InMetadataId, uint32 InStackDepth, uint16& OutType, const void*& OutData, uint32& OutSize) const = 0;
	virtual void EnumerateMetadata(uint32 InThreadId, uint32 InMetadataId, TFunctionRef<bool(uint32 StackDepth, uint16 Type, const void* Data, uint32 Size)> Callback) const = 0;
};

class IEditableMetadataProvider : public IEditableProvider
{
public:
	virtual ~IEditableMetadataProvider() = default;

	virtual void BeginEdit() const = 0;
	virtual void EndEdit() const = 0;
	virtual void EditAccessCheck() const = 0;

	virtual uint16 RegisterMetadataType(const TCHAR* InName, const FMetadataSchema& InSchema) = 0;

	virtual void PushScopedMetadata(uint32 InThreadId, uint16 InType, const void* InData, uint32 InSize) = 0;
	virtual void PopScopedMetadata(uint32 InThreadId, uint16 InType) = 0;

	virtual void BeginClearStackScope(uint32 InThreadId) = 0;
	virtual void EndClearStackScope(uint32 InThreadId) = 0;

	virtual void SaveStack(uint32 InThreadId, uint32 InSavedStackId) = 0;
	virtual void BeginRestoreSavedStackScope(uint32 InThreadId, uint32 InSavedStackId) = 0;
	virtual void EndRestoreSavedStackScope(uint32 InThreadId) = 0;

	// Pins the metadata stack and returns an id for it.
	virtual uint32 PinAndGetId(uint32 InThreadId) = 0;

	virtual void OnAnalysisCompleted() { }
};

TRACESERVICES_API FName GetMetadataProviderName();
TRACESERVICES_API const IMetadataProvider* ReadMetadataProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableMetadataProvider* EditMetadataProvider(IAnalysisSession& Session);

} // namespace TraceServices
