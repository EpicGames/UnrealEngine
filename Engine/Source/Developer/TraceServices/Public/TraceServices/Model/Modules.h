// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

#include <atomic>

namespace TraceServices
{

////////////////////////////////////////////////////////////////////////////////
/**
 * Result of a query. Since symbol resolving can be deferred this signals if a
 * symbol has been resolved, waiting to be resolved or wasn't found at all.
 */
enum class ESymbolQueryResult : uint8
{
	Pending,	// Symbol is pending resolution
	OK,			// Symbol has been correctly resolved
	NotLoaded,	// Module debug data could not be loaded or found
	Mismatch,	// Module debug data could not be loaded because debug data did not match traced binary
	NotFound,	// Symbol was not found in module debug data
	StatusNum
};

////////////////////////////////////////////////////////////////////////////////
/**
 * Helper method to get a string representation of the query result.
 */
inline const TCHAR* QueryResultToString(ESymbolQueryResult Result)
{
	static const TCHAR* DisplayStrings[] = {
		TEXT("Pending..."),
		TEXT("Ok"),
		TEXT("Not loaded"),
		TEXT("Version mismatch"),
		TEXT("Not found")
	};
	static_assert(UE_ARRAY_COUNT(DisplayStrings) == (uint8) ESymbolQueryResult::StatusNum, "Missing QueryResult");
	return DisplayStrings[(uint8)Result];
}

////////////////////////////////////////////////////////////////////////////////

enum class EResolvedSymbolFilterStatus : uint8
{
	Unknown = 0,
	Filtered,
	NotFiltered,
};

////////////////////////////////////////////////////////////////////////////////
/**
 * Represent a resolved symbol. The resolve status and string values may change
 * over time, but string pointers returned from the methods are guaranteed to live
 * during the entire analysis session.
 */
struct FResolvedSymbol
{
	const TCHAR* Module;
	const TCHAR* Name;
	const TCHAR* File;
	uint16 Line;
	std::atomic<ESymbolQueryResult> Result;
	std::atomic<EResolvedSymbolFilterStatus> FilterStatus;

	inline ESymbolQueryResult GetResult() const
	{
		return Result.load(std::memory_order_acquire);
	}

	FResolvedSymbol(ESymbolQueryResult InResult, const TCHAR* InModule, const TCHAR* InName, const TCHAR* InFile, uint16 InLine, EResolvedSymbolFilterStatus InFilterStatus)
		: Module(InModule)
		, Name(InName)
		, File(InFile)
		, Line(InLine)
		, Result(InResult)
		, FilterStatus(InFilterStatus)
	{}
};

////////////////////////////////////////////////////////////////////////////////
class IResolvedSymbolFilter
{
public:
	virtual void Update(FResolvedSymbol& InSymbol) const = 0;
};

////////////////////////////////////////////////////////////////////////////////
enum class EModuleStatus : uint8
{
	Discovered,			// Symbols are discovered for this module
	Pending,			// Module is pending load
	Loaded,				// Module has been successfully loaded
	VersionMismatch,	// Debug data was found, but did not match traced version
	NotFound,			// Debug data for module was not found
	Failed,				// Unable to parse debug information
	StatusNum,
	FailedStatusStart = VersionMismatch
};

////////////////////////////////////////////////////////////////////////////////
inline const TCHAR* ModuleStatusToString(EModuleStatus Result)
{
	static const TCHAR* DisplayStrings[] = {
		TEXT("Discovered"),
		TEXT("Pending..."),
		TEXT("Loaded"),
		TEXT("Version mismatch"),
		TEXT("Not found"),
		TEXT("Failed")
	};
	static_assert(UE_ARRAY_COUNT(DisplayStrings) == (uint8) EModuleStatus::StatusNum, "Missing QueryResult");
	return DisplayStrings[(uint8)Result];
}

////////////////////////////////////////////////////////////////////////////////
/**
 * Represents information about a module (engine/game/system dll or monolithic binary)
 * and how debug information has been loaded.
 */
struct FModule
{
	// Name of the module.
	const TCHAR*				Name;
	// Full name as reported by the event
	const TCHAR*				FullName;
	// Base address
	uint64						Base;
	// Size in memory
	uint32						Size;
	// If the module was unloaded or not
	std::atomic<bool>			bIsUnloaded;
	// Status of loading debug information
	std::atomic<EModuleStatus>	Status;
	// If status is loaded, contains the path to the debug info. If failure state contains an error message.
	const TCHAR*				StatusMessage;
	// Statistics about the module
	struct SymbolStats
	{
		std::atomic<uint32>		Discovered = 0; // number of symbols discovered in the trace (used symbols)
		std::atomic<uint32>		Cached = 0;     // number of symbols loaded from cache
		std::atomic<uint32>		Resolved = 0;   // number of symbols successfully resolved from the loaded module
		std::atomic<uint32>		Failed = 0;     // number of symbols failed to resolve
		std::atomic<uint32>		Available = 0;  // number of symbols available in the loaded module's debug information
	} Stats;

	FModule(const TCHAR* InName, const TCHAR* InFullName, uint64 InBase, uint32 InSize, EModuleStatus InStatus)
		: Name(InName)
		, FullName(InFullName)
		, Base(InBase)
		, Size(InSize)
		, bIsUnloaded(false)
		, Status(InStatus)
		, StatusMessage(nullptr)
	{
	}
};

////////////////////////////////////////////////////////////////////////////////
class IModuleProvider
	: public IProvider
	, public IEditableProvider
{
public:
	struct FStats
	{
		uint32 ModulesDiscovered;
		uint32 ModulesLoaded;
		uint32 ModulesFailed;
		uint32 SymbolsDiscovered;
		uint32 SymbolsCached;
		uint32 SymbolsResolved;
		uint32 SymbolsFailed;
	};

	virtual ~IModuleProvider() = default;

	/** Queries the name of the symbol at address. This function returns immediately,
	 * but the lookup is async. See \ref FResolvedSymbol for details. It assumed that
	 * all calls to this function happens before analysis has ended.
	 */
	virtual const FResolvedSymbol* GetSymbol(uint64 Address) = 0;

	/**
	 * Gets the number of discovered modules.
	 * @return Number of modules that were discovered so far.
	 */
	virtual uint32 GetNumModules() const = 0;

	/**
	 * Enumerates all detected modules and their state. Modules are listed in the order they
	 * were found and is guaranteed to retain it's index in the list and address in memory.
	 * @param Start Starting index
	 * @param Callback Issued for each module
	 */
	virtual void EnumerateModules(uint32 Start, TFunctionRef<void(const FModule& Module)> Callback) const = 0;

	/**
	 * Trigger a manual attempt to load symbols for a module given a search path. This operation
	 * is asynchronous and if successful resolve symbols that previously failed.
	 * @param Base Base address of the module
	 * @param Path Path where debug data can be located
	 * @return Waitable graph event for the operation
	 */
	virtual FGraphEventRef LoadSymbolsForModuleUsingPath(uint64 Base, const TCHAR* Path) = 0;

	/**
	 * Gets the search paths used to find debug symbols.
	 * @param Callback Called for each search path.
	 */
	virtual void EnumerateSymbolSearchPaths(TFunctionRef<void(FStringView Path)> Callback) const = 0;

	/** Gets statistics from provider. */
	virtual void GetStats(FStats* OutStats) const = 0;

	/** Checks if all symbols have been resolved in resolver queues. Only useful after analysis has completed (e.g., OnAnalysisComplete). */
	virtual bool HasFinishedResolving() const = 0;
};

////////////////////////////////////////////////////////////////////////////////

TRACESERVICES_API FName GetModuleProviderName();
TRACESERVICES_API const IModuleProvider* ReadModuleProvider(const IAnalysisSession& Session);

} // namespace TraceServices
