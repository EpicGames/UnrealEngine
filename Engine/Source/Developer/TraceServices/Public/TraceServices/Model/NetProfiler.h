// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Containers/Tables.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

#include <limits>

namespace TraceServices
{

enum class ENetProfilerAggregationMode : uint32
{
	None,
	Aggregate,
	InstanceMax,
};

enum class ENetProfilerDeliveryStatus : uint8
{
	Unknown,
	Delivered,
	Dropped
};

enum ENetProfilerConnectionMode : uint8
{
	Outgoing = 0,
	Incoming = 1,
	Count
};

// Mirrored from EChannelCloseReason on the runtime side
enum class ENetProfilerChannelCloseReason : uint8
{
	Destroyed = 0,
	Dormancy,
	LevelUnloaded,
	Relevancy,
	TearOff,
};

// Mirrored from EConnectionState on the runtime side
enum class ENetProfilerConnectionState : uint8
{
	USOCK_Invalid = 0, // Connection is invalid, possibly uninitialized.
	USOCK_Closed = 1, // Connection permanently closed.
	USOCK_Pending = 2, // Connection is awaiting connection.
	USOCK_Open = 3, // Connection is open.
};

TRACESERVICES_API const TCHAR* LexToString(const ENetProfilerChannelCloseReason Value);
TRACESERVICES_API const TCHAR* LexToString(const ENetProfilerConnectionState Value);

// Mirror from FNetDebugNameId on the runtime side.
typedef uint32 FNetProfilerNameIndexType;

struct FNetProfilerName
{
	const TCHAR* Name;	// Name
	FNetProfilerNameIndexType NameIndex;	// Index in the type array?
};

// This is our event type separate per level as the same name might be used on different levels
struct FNetProfilerEventType
{
	const TCHAR* Name;
	uint32 EventTypeIndex;
	FNetProfilerNameIndexType NameIndex;	
	uint16 Level;
};

typedef double FNetProfilerTimeStamp;

struct FNetProfilerLifeTime
{
	FNetProfilerTimeStamp Begin = 0;
	FNetProfilerTimeStamp End = std::numeric_limits<double>::infinity();
};

//struct FNetProfilerProtocol
//{
//	const TCHAR* Name;
//	uint32 ProtocolIndex;
//	uint64 ProtocolIdentifier;
//};

struct FNetProfilerObjectInstance
{
	FNetProfilerLifeTime LifeTime;				// Lifetime of this instance
	uint64 NetObjectId = 0;						// NetHandleIndex or NetGUID
	uint64 TypeId = uint64(0);					// ProtocolIdentifier
	uint32 ObjectIndex = 0U;					// Index in the object array
	FNetProfilerNameIndexType NameIndex = 0U;	// Index in the Name array
};

union FNetProfilerBunchInfo
{
	struct
	{
		// Must match FNetTraceBunchInfo in runtime code.
		uint64 ChannelIndex : 20;
		uint64 Seq : 12;
		uint64 ChannelCloseReason : 4;
		uint64 bPartial : 1;
		uint64 bPartialInitial : 1;
		uint64 bPartialCustomExportsFinal : 1;
		uint64 bPartialFinal : 1;
		UE_DEPRECATED(5.3, "Replication pausing is now deprecated.")
		uint64 bIsReplicationPaused : 1;
		uint64 bOpen : 1;
		uint64 bClose : 1;
		uint64 bReliable : 1;
		uint64 bHasPackageMapExports : 1;
		uint64 bHasMustBeMappedGUIDs : 1;

		// Reserved
		uint64 bIsValid : 1;
		uint64 Padding : 17;
	};
	uint64 Value;

	static FNetProfilerBunchInfo MakeBunchInfo(uint64 InValue) { FNetProfilerBunchInfo Info; Info.Value = InValue; return Info; }
};

static_assert(sizeof(FNetProfilerBunchInfo) == sizeof(uint64), "Unexpected size of FNetProfilerBunchInfo.");

struct FNetProfilerContentEvent
{
	uint64 StartPos : 24;					// Inclusive start position in the packet
	uint64 EndPos : 24;						// Exclusive end position in the packet; BitSize = EndPos - StartPos
	uint64 Level : 8;						// Level
	uint64 Padding : 8;						// Padding

	FNetProfilerBunchInfo BunchInfo;

	uint32 EventTypeIndex;					// Will replace name index
	FNetProfilerNameIndexType NameIndex;	// Identify the name / type, should we store the actual Name as well
	uint32 ObjectInstanceIndex;				// Object instance, Non zero if this is a NetObject, we can then look up data by indexing into ObjectInstances
};

// This must be kept in sync with ENetTraceStatsCounterType in NetTrace.h
enum class ENetProfilerStatsCounterType : uint8
{
	Packet = 0,
	Frame = 1,
};

struct FNetProfilerStatsCounterType
{
	uint32 StatsCounterTypeIndex;
	FNetProfilerNameIndexType NameIndex;
	ENetProfilerStatsCounterType Type;
};

struct FNetProfilerStats
{
	uint32 StatsCounterTypeIndex;
	uint32 StatsValue;
};

struct FNetProfilerPacket
{
	FNetProfilerTimeStamp TimeStamp;
	uint32 SequenceNumber;
	uint32 ContentSizeInBits;						// This is the part that is tracked by the PacketContents
	uint32 TotalPacketSizeInBytes;					// This is the actual size of the packet sent on the socket

	// Index into Events
	uint32 StartEventIndex;
	uint32 EventCount;

	uint32 StartStatsIndex;
	uint32 StatsCount;

	// Index into network profiler frames
	uint32 NetProfilerFrameIndex;

	ENetProfilerDeliveryStatus DeliveryStatus;		// Indicates if the packet was delivered or not, updated as soon as we know
	ENetProfilerConnectionState ConnectionState;
};

struct FNetProfilerFrame
{
	FNetProfilerTimeStamp TimeStamp;
	uint64 EngineFrameNumber = 0;
	uint32 StartStatsIndex = 0;
	uint32 StatsCount = 0;
};

struct FNetProfilerConnection
{
	const TCHAR* Name = nullptr;
	const TCHAR* AddressString = nullptr;
	FNetProfilerLifeTime LifeTime;
	uint32 GameInstanceIndex;
	uint32 ConnectionIndex : 16;
	uint32 ConnectionId : 14;
	uint32 bHasIncomingData : 1;
	uint32 bHasOutgoingData : 1;
};

struct FNetProfilerGameInstance
{
	FNetProfilerLifeTime LifeTime;
	uint32 GameInstanceIndex;
	uint32 GameInstanceId;
	const TCHAR* InstanceName = nullptr;
	bool bIsUsingIrisReplication = false;
	bool bIsServer = false;
};

struct FNetProfilerAggregatedStats
{
	uint32 EventTypeIndex;
	uint32 InstanceCount = 0U;
	uint32 TotalInclusive = 0U;
	uint32 MaxInclusive = 0U;
	uint32 AverageInclusive = 0U;
	uint32 TotalExclusive = 0U;
	uint32 MaxExclusive = 0U;
};

struct FNetProfilerAggregatedStatsCounterStats
{
	uint32 StatsCounterTypeIndex;
	uint32 Sum;
	uint32 Min;
	uint32 Max;
	uint32 Average;
	uint32 Count;

	void Reset()
	{
		Sum = 0;
		Min = 0;
		Max = 0;
		Average = 0;
		Count = 0;
	}
};

// What queries do we need?
class INetProfilerProvider : public IProvider
{
public:
	virtual ~INetProfilerProvider() = default;

	// Return the version reported in the trace
	// A return value of 0 indicates no network trace data
	virtual uint32 GetNetTraceVersion() const = 0;

	// Access Names
	virtual uint32 GetNameCount() const = 0;
	virtual void ReadNames(TFunctionRef<void(const FNetProfilerName*, uint64)> Callback) const = 0;
	virtual void ReadName(uint32 NameIndex, TFunctionRef<void(const FNetProfilerName&)> Callback) const = 0;

	// Access EventTypes
	virtual uint32 GetEventTypesCount() const = 0;
	virtual void ReadEventTypes(TFunctionRef<void(const FNetProfilerEventType*, uint64)> Callback) const = 0;
	virtual void ReadEventType(uint32 EventTypeIndex, TFunctionRef<void(const FNetProfilerEventType&)> Callback) const = 0;

	// Access GameInstances
	virtual uint32 GetGameInstanceCount() const = 0;
	virtual void ReadGameInstances(TFunctionRef<void(const FNetProfilerGameInstance&)> Callback) const = 0;
	virtual uint32 GetGameInstanceChangeCount() const = 0;

	// Access Connections
	virtual uint32 GetConnectionCount(uint32 GameInstanceIndex) const = 0;
	virtual void ReadConnections(uint32 GameInstanceIndex, TFunctionRef<void(const FNetProfilerConnection&)> Callback) const = 0;
	virtual void ReadConnection(uint32 ConnectionIndex, TFunctionRef<void(const FNetProfilerConnection&)> Callback) const = 0;
	virtual uint32 GetConnectionChangeCount() const = 0;

	// Access Object Instances
	virtual uint32 GetObjectCount(uint32 GameInstanceIndex) const = 0;
	virtual void ReadObjects(uint32 GameInstanceIndex, TFunctionRef<void(const FNetProfilerObjectInstance&)> Callback) const = 0;
	virtual void ReadObject(uint32 GameInstanceIndex, uint32 ObjectIndex, TFunctionRef<void(const FNetProfilerObjectInstance&)> Callback) const = 0;
	virtual uint32 GetObjectsChangeCount(uint32 GameInstanceIndex) const = 0;

	// Find Packet Index from SequenceNumber
	virtual int32 FindPacketIndexFromPacketSequence(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 SequenceNumber) const = 0;

	// Gets the number of packets for the specified connection and connection mode.
	virtual uint32 GetPacketCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const = 0;

	// Enumerates packets in the provided inclusive packet interval, [PacketIndexIntervalStart, PacketIndexIntervalEnd].
	virtual void EnumeratePackets(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd, TFunctionRef<void(const FNetProfilerPacket&)> Callback) const = 0;
	// Returns a change number incremented each time a change occurs in the packets for the specified connection and connection mode.
	virtual uint32 GetPacketChangeCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const = 0;

	// Enumerates packet content events in the inclusive event index interval [StartEventIndex, EndEventIndex]. */
	virtual void EnumeratePacketContentEventsByIndex(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 StartEventIndex, uint32 EndEventIndex, TFunctionRef<void(const FNetProfilerContentEvent&)> Callback) const = 0;
	// Enumerates packet content events for a packet, in the exclusive bit range [StartPosition, EndPosition). */
	virtual void EnumeratePacketContentEventsByPosition(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndex, uint32 StartPosition, uint32 EndPosition, TFunctionRef<void(const FNetProfilerContentEvent&)> Callback) const = 0;
	// Returns a change number incremented each time a change occurs in the packet content events for the specified connection and connection mode. */
	virtual uint32 GetPacketContentEventChangeCount(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode) const = 0;

	// Access StatsCounterTypes
	virtual uint32 GetNetStatsCounterTypesCount() const = 0;
	virtual void ReadNetStatsCounterTypes(TFunctionRef<void(const FNetProfilerStatsCounterType*, uint64)> Callback) const = 0;
	virtual void ReadNetStatsCounterType(uint32 TypeIndex, TFunctionRef<void(const FNetProfilerStatsCounterType&)> Callback) const = 0;

	// Computes aggregated stats for a packet interval or for a range of content events in a single packet.
	// [PacketIndexIntervalStart, PacketIndexIntervalEnd] is the inclusive packet interval.
	// [StartPosition, EndPosition) is the exclusive bit range interval; only used when PacketIndexIntervalStart == PacketIndexIntervalEnd.
	virtual ITable<FNetProfilerAggregatedStats>* CreateAggregation(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd, uint32 StartPosition, uint32 EndPosition) const = 0;

	// Computes aggregated statscounters for a packet interval
	// [PacketIndexIntervalStart, PacketIndexIntervalEnd] is the inclusive packet interval.
	virtual ITable<FNetProfilerAggregatedStatsCounterStats>* CreateStatsCountersAggregation(uint32 ConnectionIndex, ENetProfilerConnectionMode Mode, uint32 PacketIndexIntervalStart, uint32 PacketIndexIntervalEnd) const = 0;
};

TRACESERVICES_API FName GetNetProfilerProviderName();
TRACESERVICES_API const INetProfilerProvider* ReadNetProfilerProvider(const IAnalysisSession& Session);

} // namespace TraceServices
