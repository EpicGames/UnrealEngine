// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/Function.h"
#include "UObject/NameTypes.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

FUtf8String FunctionFlagsToUtf8String(uint32 Flags);

enum class EObjectInfoFlags : uint32
{
	None = 0,
	IsField  = 1 << 0,
	IsStruct = 1 << 1,
	IsClass = 1 << 2,
	IsFunction = 1 << 3,
	IsPackage = 1 << 4,
};
ENUM_CLASS_FLAGS(EObjectInfoFlags);

struct FObjectInfo
{
	uint32 Id = uint32(-1);
	uint32 ClassId = uint32(-1);
	uint32 OuterId = uint32(-1);
	const TCHAR* Name = nullptr;
	uint32 Flags = 0;
	EObjectInfoFlags FlagsEx = EObjectInfoFlags::None;

	const TCHAR* VersePath = nullptr;

	uint64 SystemMemoryBytes = 0;
	uint64 VideoMemoryBytes = 0;
	uint64 TotalSystemMemoryBytes = 0;
	uint64 TotalVideoMemoryBytes = 0;

	uint32 SuperId = uint32(-1);
	uint32 InheritanceSuperId = uint32(-1);
	int32 StructureSize = 0;

	uint32 FunctionFlags = 0;
	uint8 FunctionNumParms = 0;
	uint16 FunctionParmsSize = 0;

	uint64 PackageId = 0;
	const TCHAR* PackagePath = nullptr;
	const TCHAR* SourcePackageName = nullptr; // Editor source package name; set when it differs from the runtime Name (e.g. streamed/instanced level packages)
};

struct FObjectReferenceInfo
{
	uint32 ReferencerId = 0;
	uint32 ObjectId = 0;
};

class IObjectSnapshot
{
public:
	virtual ~IObjectSnapshot() = default;
	virtual uint32 GetId() const = 0;
	virtual double GetStartTime() const = 0;
	virtual double GetEndTime() const = 0;
	virtual const FObjectInfo* GetObject(uint32 Id) const = 0;
	virtual uint32 GetObjectArrayNum() const = 0;
	virtual uint32 GetObjectCount() const = 0;
	virtual uint32 GetNumReferences() const = 0;
	virtual uint32 GetTracedObjectArrayNum() const = 0;
	virtual uint32 GetTracedObjectArrayNumMinusAvailable() const = 0;
	virtual uint32 GetTracedObjectArrayNumPermanent() const = 0;
	virtual bool HasTotalMemorySizes() const = 0;
	virtual TConstArrayView<FObjectInfo> GetObjectsArray() const = 0;
	virtual TConstArrayView<FObjectReferenceInfo> GetReferencesArray() const = 0;
	virtual void SaveAs(const TCHAR* CsvFileName) const = 0;
};

class IObjectEditableSnapshot : public IObjectSnapshot
{
public:
	virtual ~IObjectEditableSnapshot() = default;

	virtual FObjectInfo* GetEditableObject(uint32 ObjectId) = 0;
	virtual void EnableTotalMemorySizes() = 0;
	virtual void SetTracedObjectArrayNum(uint32 Count) = 0;
	virtual void SetTracedObjectArrayNumMinusAvailable(uint32 Count) = 0;
	virtual void SetTracedObjectArrayNumPermanent(uint32 Count) = 0;
};

class IObjectProvider : public IProvider
{
public:
	virtual ~IObjectProvider() = default;

	virtual void BeginRead() const = 0;
	virtual void EndRead() const = 0;
	virtual void ReadAccessCheck() const = 0;

	/**
	 * Enumerates the available objects.
	 * The enumeration stops if the callback returns false.
	 */
	virtual void EnumerateObjects(uint32 SnapshotId, TFunctionRef<bool(const FObjectInfo&)> Callback) const = 0;

	/**
	 * Gets the object with the specified Id.
	 * The resulting pointer is valid only under the current read lock.
	 */
	virtual const FObjectInfo* GetObject(uint32 SnapshotId, uint32 ObjectId) const = 0;

	/**
	 * Finds the object with the specified name.
	 * The resulting pointer is valid only under the current read lock.
	 */
	virtual const FObjectInfo* FindObject(uint32 SnapshotId, FStringView Name) const = 0;

	/**
	 * Gets the current number of analyzed objects in the snapshot being currently modified.
	 */
	virtual uint32 GetCurrentSnapshotObjectCount() const = 0;

	/**
	 * Gets the total number of objects traced in the snapshot being currently modified.
	 */
	virtual uint32 GetCurrentSnapshotTotalObjectCount() const = 0;

	/**
	 * Gets the number of available snapshots.
	 */
	virtual uint32 GetNumSnapshots() const = 0;

	/**
	 * Enumerates the available snapshots.
	 * The enumeration stops if the callback returns false.
	 */
	virtual void EnumerateSnapshots(TFunctionRef<bool(const IObjectSnapshot&)> Callback) const = 0;

	/**
	 * Gets the snapshot with the specified Id.
	 * The resulting pointer is valid only under the current read lock.
	 */
	virtual const IObjectSnapshot* GetSnapshot(uint32 SnapshotId) const = 0;

	/**
	 * Checks the validity of the snapshot being currently modified.
	 */
	virtual bool IsCurrentSnapshotValid() const = 0;

	/**
	 * Finds the last snapshot with StartTime <= provided Time.
	 * The resulting pointer is valid only under the current read lock.
	 */
	virtual const IObjectSnapshot* GetLowerBoundSnapshot(double Time) const = 0;
};

class IEditableObjectProvider : public IEditableProvider
{
public:
	virtual ~IEditableObjectProvider() = default;

	/**
	 * Gets the current snapshot.
	 * The resulting pointer is valid only under the current edit lock.
	 */
	virtual IObjectEditableSnapshot* GetCurrentSnapshot() = 0;

	/**
	 * Begins a new snapshot.
	 * The resulting pointer is valid only under the current edit lock.
	 */
	virtual IObjectEditableSnapshot* BeginSnapshot(double Time) = 0;

	/**
	 * Ends the current snapshot.
	 * The resulting pointer is valid only under the current edit lock.
	 */
	virtual IObjectEditableSnapshot* EndSnapshot(double Time) = 0;

	/**
	 * Adds an object declaration, to the current snapshot.
	 * It is only valid to call this between BeginSnapshot() and EndSnapshot() calls.
	 * The resulting pointer is valid only under the current edit lock.
	 */
	virtual FObjectInfo* AddObject(uint32 ObjectId, uint32 ClassId, uint32 OuterId, uint32 Flags, const TCHAR* PersistentName) = 0;

	/**
	 * Gets the object identified by id.
	 * It is only valid to call this between BeginSnapshot() and EndSnapshot() calls.
	 * The resulting pointer is valid only under the current edit lock.
	 */
	virtual FObjectInfo* GetEditableObject(uint32 Id) = 0;

	/**
	 * Adds a dependency between objects, to the current snapshot.
	 * It is only valid to call this between BeginSnapshot() and EndSnapshot() calls.
	 * The resulting pointer is valid only under the current edit lock.
	 */
	virtual FObjectReferenceInfo* AddObjectReference(uint32 ReferencerId, uint32 ObjectId) = 0;

	/**
	 * Gets the read provider.
	 * @returns The read only provider or nullptr if not available.
	 */
	virtual const IObjectProvider* GetReadProvider() const { return nullptr; }
};

TRACESERVICES_API FName GetObjectProviderName();
TRACESERVICES_API const IObjectProvider* ReadObjectProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableObjectProvider* EditObjectProvider(IAnalysisSession& Session);

} // namespace TraceServices
