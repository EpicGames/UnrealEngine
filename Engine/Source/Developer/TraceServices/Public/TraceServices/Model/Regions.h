// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

// TraceServices
#include "Common/PagedArray.h"
#include "TraceServices/Model/AnalysisSession.h"

#define UE_API TRACESERVICES_API

namespace TraceServices
{

struct FRegionCategory
{
	const TCHAR* Name = nullptr;
};

struct FRegionTimer
{
	const TCHAR* Name = nullptr;
	const FRegionCategory* Category = nullptr;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FTimeRegion
{
	const FRegionTimer* Timer = nullptr; // name and category of the region

	double BeginTime = std::numeric_limits<double>::infinity();
	double EndTime = std::numeric_limits<double>::infinity();

	uint64 Id = 0; // 0 if the region is identified by Name only
	int32 Depth = -1;

	UE_DEPRECATED(5.7, "Use instead Timer->Name")
	const TCHAR* Text = nullptr;

	UE_DEPRECATED(5.7, "Use instead Timer->Category->Name")
	const TCHAR* Category = nullptr; // this will be nullptr for regions without a Category
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class FRegionLane
{
	friend class FRegionTimeline;
	friend struct FLaneEnumeration;

public:
	FRegionLane(ILinearAllocator& InAllocator) : Regions(InAllocator, 512) {}

	int32 Num() const { return static_cast<int32>(Regions.Num()); }

	/**
	 * Call Callback for every region overlapping the interval defined by IntervalStart and IntervalEnd
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	UE_API bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion&)> Callback) const;

	/**
	 * Call Callback for every region overlapping the interval defined by IntervalEnd and IntervalStart
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	UE_API bool EnumerateRegionsBackwards(double IntervalEnd, double IntervalStart, TFunctionRef<bool(const FTimeRegion&)> Callback) const;

private:
	TPagedArray<FTimeRegion> Regions;
};

struct FLaneEnumeration
{
	const FRegionLane& Lane;
	int32 IntervalStart;
	int32 IntervalEnd;

	bool operator<(const FLaneEnumeration& rhs) const
	{
		return Lane.Regions[IntervalStart].BeginTime < rhs.Lane.Regions[rhs.IntervalStart].BeginTime;
	}

	bool operator>(const FLaneEnumeration& rhs) const
	{
		return Lane.Regions[IntervalStart].BeginTime > rhs.Lane.Regions[rhs.IntervalStart].BeginTime;
	}
};

/*
 *	Sorts a set of Timing Regions into a stack of individual lanes without overlaps for display
 */
class IRegionTimeline
{
public:
	/**
	 * @return the category name of this timeline if filtered or nullptr
	 */
	virtual const TCHAR* GetCategoryName() const = 0;

	/**
	 * @return the category of this timeline if filtered or nullptr
	 */
	virtual const FRegionCategory* GetCategory() const = 0;

	/**
	 * @return the number of lanes
	 */
	virtual int32 GetLaneCount() const = 0;

	/**
	 * Direct access to a certain lane at a given index/depth.
	 * The pointer is valid only in the current read scope.
	 * @return a pointer to the lane at the specified depth index or nullptr if Index > GetLaneCount()-1
	 */
	virtual const FRegionLane* GetLane(int32 Index) const = 0;

	/**
	 * Enumerates all regions that overlap a certain time interval. Will enumerate by depth but does not expose lanes.
	 *
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	virtual bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion&)> Callback) const = 0;

	/**
	 * Enumerates all regions that overlap a certain time interval in reverse. Will enumerate by depth but does not expose lanes.
	 *
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	virtual bool EnumerateRegionsBackwards(double IntervalEnd, double IntervalStart, TFunctionRef<bool(const FTimeRegion&)> Callback) const = 0;
	/**
	 * Will call Callback(Lane, Depth) for each lane in order.
	 */
	virtual void EnumerateLanes(TFunctionRef<void(const FRegionLane&, const int32)> Callback) const = 0;
};

class IRegionProvider
	: public IProvider
{
public:
	virtual ~IRegionProvider() override = default;

	/**
	 * Enumerates all region categories.
	 * @param Callback a callback called for each region category. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	virtual bool EnumerateCategories(TFunctionRef<bool(const FRegionCategory&)> Callback) const = 0;

	/**
	 * Enumerates all region timers (unique regions by name).
	 * @param Callback a callback called for each region timer. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	virtual bool EnumerateTimers(TFunctionRef<bool(const FRegionTimer&)> Callback) const = 0;

	/**
	 * Enumerates all timelines, this includes the uncategorized timeline with uncategorized regions and individual timelines for each Category
	 * @param Callback a callback called for each timeline. Second parameter contains Category or nullptr for the uncategorized timeline.
	 */
	virtual void EnumerateTimelinesByCategory(TFunctionRef<void(const IRegionTimeline& /*Region*/, const TCHAR* /*Category*/)> Callback) const = 0;

	/**
	 * @return the default timeline containing all regions without filtering
	 */
	virtual const IRegionTimeline& GetDefaultTimeline() const = 0;

	/**
	 * @return the timeline for a given Category or the timeline with uncategorized regions if Category is nullptr. Returns nullptr if the category was invalid/not found
	 */
	virtual const IRegionTimeline* GetTimelineForCategory(const TCHAR* Category) const = 0;

	/**
	 * @return the string used to store regions with no explicit category set
	 */
	virtual const TCHAR* GetUncategorizedRegionCategoryName() const = 0;

	/**
	 * @return the amount of currently known regions (including open-ended ones)
	 */
	virtual uint64 GetRegionCount() const = 0;

	/**
	 * @return the number of lanes of the default timeline
	 */
	UE_DEPRECATED(5.6, "Use GetDefaultTimeline().GetLaneCount() or EnumerateTimelinesByCategory() to access timelines instead.")
	virtual int32 GetLaneCount() const = 0;

	/**
	 * Direct access to a certain lane at a given index/depth of the default timeline.
	 * The pointer is valid only in the current read scope.
	 * @return a pointer to the lane at the specified depth index or nullptr if Index > GetLaneCount()-1
	 */
	UE_DEPRECATED(5.6, "Use GetDefaultTimeline().GetLane() or EnumerateTimelinesByCategory() to access timelines instead.")
	virtual const FRegionLane* GetLane(int32 Index) const = 0;

	/**
	 * Enumerates all regions that overlap a certain time interval. Will enumerate by depth but does not expose lanes.
	 *
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	UE_DEPRECATED(5.6, "Use GetDefaultTimeline().EnumerateRegions() or EnumerateTimelinesByCategory() to access timelines instead.")
	virtual bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FTimeRegion& /*Region*/)> Callback) const = 0;

	/**
	 * Will call Callback(Lane, Depth) for each lane in order.
	 */
	UE_DEPRECATED(5.6, "Use GetDefaultTimeline().EnumerateLanes() or EnumerateTimelinesByCategory() to access timelines instead.")
	virtual void EnumerateLanes(TFunctionRef<void(const FRegionLane& /*Lane*/, const int32 /*Depth*/)> Callback) const = 0;

	/**
	 * @return a monotonically increasing counter that is updated each time new data is added to the provider.
	 * This can be used to detect when to update any (UI-)state dependent on the provider during analysis.
	 */
	virtual uint64 GetUpdateCounter() const = 0;
};

/**
 * The interface to a provider that can consume mutations of region events from a session.
 */
class IEditableRegionProvider
	: public IEditableProvider
{
public:
	virtual ~IEditableRegionProvider() override = default;

	virtual const FRegionCategory* RegisterRegionCategory(const TCHAR* Name) = 0;
	virtual const FRegionTimer* RegisterRegionTimer(const TCHAR* Name, const FRegionCategory* Category = nullptr) = 0;

	/**
	 * Append a new begin event of a region from the trace session.
	 * Prefer opening/closing regions with an Id, since string names are not unique.
	 *
	 * @param Name		The string name of the region.
	 * @param Time		The time in seconds of the begin event of this region.
	 * @param Category	The category associated with this region, use nullptr if no category has been set
	 */
	virtual void AppendRegionBegin(const TCHAR* Name, double Time, const TCHAR* Category = nullptr) = 0;

	/**
	 * Append a new begin event of a region from the trace session.
	 * Id will be used to uniquely identify the new region.
	 *
	 * @param Name		The string name of the region.
	 * @param Id		The Id of the region. Used to uniquely identify regions with the same name.
	 * @param Time		The time in seconds of the begin event of this region.
	 * @param Category	The category associated with this region, use nullptr if no category has been set
	 */
	virtual void AppendRegionBeginWithId(const TCHAR* Name, uint64 Id, double Time, const TCHAR* Category = nullptr) = 0;

	/**
	 * Append a new end event of a region from the trace session.
	 * Prefer opening/closing regions with an Id, since string names are not unique.
	 *
	 * @param Name		The string name of the region.
	 * @param Time		The time in seconds of the end event of this region.
	 */
	virtual void AppendRegionEnd(const TCHAR* Name, double Time) = 0;

	/**
	 * Append a new end event of a region from the trace session.
	 * The region is identified by Id.
	 * @param Id		The Id of the region.
	 * @param Time		The time in seconds of the end event of this region.
	 */
	virtual void AppendRegionEndWithId(const uint64 Id, double Time) = 0;

	/**
	 * Called from the analyzer once all events have been processed.
	 * Allows post-processing and error reporting for regions that were never closed.
	 */
	virtual void OnAnalysisSessionEnded() = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

UE_API FName GetRegionProviderName();
UE_API const IRegionProvider& ReadRegionProvider(const IAnalysisSession& Session);
UE_API IEditableRegionProvider& EditRegionProvider(IAnalysisSession& Session);

} // namespace TraceServices

#undef UE_API
