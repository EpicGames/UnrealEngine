// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FScreenshot
{
	uint32 Id;
	FString Name;
	double Timestamp;
	uint32 Width;
	uint32 Height;
	uint32 ChunkNum;
	uint32 Size;
	TArray<uint8> Data;

	static const uint32 InvalidScreenshotId = (uint32)-1;
};

class IScreenshotProvider
	: public IProvider
{
public:
	virtual ~IScreenshotProvider() = default;

	virtual const TSharedPtr<const FScreenshot> GetScreenshot(uint32 Id) const = 0;
};

TRACESERVICES_API FName GetScreenshotProviderName();
TRACESERVICES_API const IScreenshotProvider& ReadScreenshotProvider(const IAnalysisSession& Session);

} // namespace TraceServices
