// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

// TraceServices
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/TimingProfiler.h"

namespace TraceServices
{

struct FResolvedSymbol;

struct FStackSampleFrame
{
	uint64 Address = 0;
	uint32 TimerId = 0;
	const FResolvedSymbol* Symbol = nullptr;
};

typedef ITimeline<FTimingProfilerEvent> FStackSampleTimeline;

struct FStackSampleThread
{
	uint32 SystemThreadId = 0;
	const TCHAR* Name = nullptr;
	const FStackSampleTimeline* Timeline = nullptr;
};

class IStackSamplesProvider
	: public IProvider
{
public:
	virtual ~IStackSamplesProvider() = default;

	virtual uint32 GetStackFrameCount() const = 0;
	virtual void EnumerateStackFrames(TFunctionRef<void(const FStackSampleFrame& Frame)> Callback) const = 0;

	virtual uint32 GetThreadCount() const = 0;
	virtual void EnumerateThreads(TFunctionRef<void(const FStackSampleThread& Thread)> Callback) const = 0;

	virtual uint32 GetTimelineCount() const = 0;
	virtual void EnumerateTimelines(TFunctionRef<void(const FStackSampleTimeline& Timeline, const FStackSampleThread& Thread)> Callback) const = 0;
	virtual const FStackSampleTimeline* GetTimeline(uint32 SystemThreadId) const = 0;
};

TRACESERVICES_API FName GetStackSamplesProviderName();
TRACESERVICES_API const IStackSamplesProvider* ReadStackSamplesProvider(const IAnalysisSession& Session);

} // namespace TraceServices
