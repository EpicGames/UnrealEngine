// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "TraceServices/Model/AnalysisSession.h"
#include "Templates/Function.h"

namespace TraceServices
{
	struct FStringDefinition
	{
		/**
		 * Display string. The pointer is valid during the analysis session.
		 */
		const TCHAR* Display;

		static FString ToString(const void* Data)
		{
			const FStringDefinition* Def = (const FStringDefinition*) Data;
			return FString(Def->Display);
		}
	};
}