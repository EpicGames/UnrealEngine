// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraceServices/Containers/Tables.h"

class FTokenizedMessage;

namespace TraceServices
{

class FImportTableRow;

class ITableImportData
{
	TSharedPtr<ITable<FImportTableRow>> GetTable();
};

enum class ETableImportResult : uint32
{
	ESuccess = 0,
	EFail = 1,
};

struct FTableImportCallbackParams
{
	FName TableId;
	ETableImportResult Result;
	TSharedPtr<ITable<FImportTableRow>> Table;
	TArray<TSharedRef<FTokenizedMessage>> Messages;
};

class FTableImportService
{
public:
	typedef TFunction<void(TSharedPtr<FTableImportCallbackParams>)> TableImportCallback;

	TRACESERVICES_API static void ImportTable(const FString& InPath, FName TableId, TableImportCallback InCallback);
};

} // namespace TraceServices
