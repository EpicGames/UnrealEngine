// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraceServices/Containers/Tables.h"

class FTokenizedMessage;

namespace TraceServices
{
enum class ETableDiffResult : uint32
{
	ESuccess = 0,
	EFail = 1,
};

struct FTableDiffCallbackParams
{
	ETableDiffResult Result;
	TSharedPtr<IUntypedTable> Table;
	TArray<TSharedRef<FTokenizedMessage>> Messages;
};

class FTableMergeService
{
public:
	typedef TFunction<void(TSharedPtr<FTableDiffCallbackParams>)> TableDiffCallback;

	TRACESERVICES_API static void MergeTables(const TSharedPtr<IUntypedTable>& TableA, const TSharedPtr<IUntypedTable>& TableB, TableDiffCallback InCallback);
};
} // namespace TraceServices
