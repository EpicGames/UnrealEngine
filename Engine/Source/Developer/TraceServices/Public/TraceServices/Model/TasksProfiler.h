// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Async/TaskTrace.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

// task biography
struct FTaskInfo
{
	FTaskInfo() = default;

	TaskTrace::FId Id = TaskTrace::InvalidId;

	// this members are filled during task launch
	const TCHAR* DebugName = nullptr; // user provided or automatically generated (in format `full_path(lineno)`)
	bool bTracked = false; // if the task is awaitable, not awaitable tasks will never be "completed", they are done when their execution is finished
	int32 ThreadToExecuteOn = ENamedThreads::AnyThread; // ENamedThreads::Type - combines the info about thread to execute on (one of named threads or a worker
	// thread - ENamedThreads::AnyThread), task priority, thread priority (in case of a worker thread), and queue index (in case of a named
	// thread - main queue or local queue). See ENamedThreads to split the info into separate fields

	static constexpr double InvalidTimestamp = -std::numeric_limits<double>::max();

	// some tasks can be created with delayed launch, otherwise "created" and "launched" timestamps and thread IDs are equal
	double CreatedTimestamp = InvalidTimestamp;
	// the moment and the place when the task was launched
	double LaunchedTimestamp = InvalidTimestamp;
	// the moment and the place when all task's dependencies are resolved and it's added to the execution queue
	double ScheduledTimestamp = InvalidTimestamp;
	// execution started
	double StartedTimestamp = InvalidTimestamp;
	// execution finished, but the task can be still not completed waiting for all nested tasks to complete
	double FinishedTimestamp = InvalidTimestamp;
	// execution done and all nested tasks are completed
	double CompletedTimestamp = InvalidTimestamp;
	// the last reference is released and the task is destroyed
	double DestroyedTimestamp = InvalidTimestamp;

	uint32 CreatedThreadId = 0;
	uint32 LaunchedThreadId = 0;
	uint32 ScheduledThreadId = 0;
	uint32 StartedThreadId = 0;
	uint32 CompletedThreadId = 0;
	uint32 DestroyedThreadId = 0;

	// task size including user-provided task body
	uint64 TaskSize = 0;

	// relation with another task, when and where it was established
	struct FRelationInfo
	{
		FRelationInfo(TaskTrace::FId InRelativeId, double InTimestamp, uint32 InThreadId)
			: RelativeId(InRelativeId)
			, Timestamp(InTimestamp)
			, ThreadId(InThreadId)
		{
		}

		TaskTrace::FId RelativeId;
		double Timestamp;
		uint32 ThreadId;
	};

	// other tasks that must complete before this task can be scheduled
	TArray<FRelationInfo> Prerequisites;
	// other tasks that wait for this task completion before they'll start execution
	TArray<FRelationInfo> Subsequents;
	// other tasks that have this task as a nested
	TArray<FRelationInfo> ParentTasks;
	// the task is completed only after all nested tasks are completed
	TArray<FRelationInfo> NestedTasks;
};

struct FWaitingForTasks
{
	TArray<TaskTrace::FId> Tasks;
	double StartedTimestamp;
	double FinishedTimestamp = FTaskInfo::InvalidTimestamp;
};

enum class ETaskEnumerationOption
{
	Alive,
	Launched,
	Active,
	WaitingForPrerequisites,
	Queued,
	Executing,
	WaitingForNested,
	Completed,
	Count
};

enum class ETaskEnumerationResult
{
	Continue,
	Stop,
};

// query interface to tasks info
class ITasksProvider : public IProvider
{
public:
	typedef TFunctionRef<ETaskEnumerationResult(const FTaskInfo& Task)> TaskCallback;

	virtual ~ITasksProvider() = default;

	// returns a task for given thread and timestamp, if any, otherwise `nullptr`
	virtual const FTaskInfo* TryGetTask(uint32 ThreadId, double Timestamp) const = 0;
	// returns task info for given task ID
	virtual const FTaskInfo* TryGetTask(TaskTrace::FId TaskId) const = 0;

	// returns an info about waiting for tasks completion for given thread and timestamp, if any, otherwise `nullptr`
	virtual const FWaitingForTasks* TryGetWaiting(const TCHAR* TimerName, uint32 ThreadId, double Timestamp) const = 0;

	// returns an info about ParallelFor tasks for given timer
	virtual TArray<TaskTrace::FId> TryGetParallelForTasks(const TCHAR* TimerName, uint32 ThreadId, double StartTime, double EndTime) const = 0;

	// returns the number of tasks stored in the provider
	virtual int64 GetNumTasks() const = 0;

	// Calls the callback for each task stored in the provider with CreatedTimestamp <= EndTime and FinishedTimestamp >= StartTime
	virtual void EnumerateTasks(double StartTime, double EndTime, ETaskEnumerationOption EnumerationOption, TaskCallback Callback) const = 0;
};

TRACESERVICES_API FName GetTaskProviderName();
TRACESERVICES_API const ITasksProvider* ReadTasksProvider(const IAnalysisSession& Session);

} // namespace TraceServices
