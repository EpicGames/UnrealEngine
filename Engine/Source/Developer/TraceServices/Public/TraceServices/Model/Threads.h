// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PlatformAffinity.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

struct FThreadInfo
{
	uint32 Id;
	const TCHAR* Name;
	const TCHAR* GroupName;
};

class IThreadProvider
	: public IProvider
{
public:
	virtual ~IThreadProvider() = default;
	virtual uint64 GetModCount() const = 0;
	virtual void EnumerateThreads(TFunctionRef<void(const FThreadInfo&)> Callback) const = 0;
	virtual const TCHAR* GetThreadName(uint32 ThreadId) const = 0;
};

/*
* An interface that can consume thread related events from a session.
*/
class IEditableThreadProvider
	: public IEditableProvider
{
public:
	virtual ~IEditableThreadProvider() = default;

	/*
	* Note the existence of a new thread.
	*
	* @param Id			The thread identity.
	* @param Name		The name the user may know the thread by, if available.
	* @param Priority	The system priority level of the thread, if available.
	*/
	virtual void AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority) = 0;

	/*
	* Gets the read provider.
	* @returns The read only provider or nullptr if not available.
	*/
	virtual const IThreadProvider* GetReadProvider() const { return nullptr; }
};

TRACESERVICES_API FName GetThreadProviderName();
TRACESERVICES_API const IThreadProvider& ReadThreadProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableThreadProvider& EditThreadProvider(IAnalysisSession& Session);

} // namespace TraceServices
