// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "UObject/NameTypes.h"

// TraceServices
#include "Model/MonotonicTimeline.h"
#include "TraceServices/Common/CancellationToken.h"
#include "TraceServices/Containers/Tables.h"
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices
{

template <typename InEventType> class ITimeline;
template <typename InEventType> class IEditableTimeline;
template <typename RowType> class ITable;

struct FTimingProfilerEvent
{
	uint32 TimerIndex = uint32(-1);
};

typedef ITimeline<FTimingProfilerEvent> ITimingProfilerTimeline;
typedef IEditableTimeline<FTimingProfilerEvent> ITimingProfilerEditableTimeline;

struct FMetadataSpec
{
	const TCHAR* Format = nullptr;
	TArray<const TCHAR*> FieldNames;

	static const uint32 InvalidMetadataSpecId = (uint32)-1;
};

enum class ETimingProfilerTimerType : uint8
{
	CpuScope,
	CpuSampling,
	GpuScope,
	VerseSampling,

	Count,

	CPU = CpuScope, // temporary backward compatibility
	GPU = GpuScope, // temporary backward compatibility
	Verse = VerseSampling, // temporary backward compatibility
};

struct FTimingProfilerTimer
{
	const TCHAR* Name = nullptr;
	const TCHAR* File = nullptr;
	uint32 MetadataSpecId = FMetadataSpec::InvalidMetadataSpecId;

	uint32 Id = 0;
	union
	{
		struct
		{
			uint32 Line : 24;
			ETimingProfilerTimerType Type : 8;
		};
		uint32 LineAndType = 0; // used only to default initialize Line and Type with 0
	};

	bool HasValidMetadataSpecId() const
	{
		return MetadataSpecId != FMetadataSpec::InvalidMetadataSpecId;
	}
};

struct FTimingProfilerAggregatedStats
{
	const FTimingProfilerTimer* Timer = nullptr;
	uint64 InstanceCount = 0;
	double AverageInstanceCount = 0.0;
	double TotalInclusiveTime = 0.0;
	double MinInclusiveTime = DBL_MAX;
	double MaxInclusiveTime = -DBL_MAX;
	double AverageInclusiveTime = 0.0;
	double MedianInclusiveTime = 0.0;
	double TotalExclusiveTime = 0.0;
	double MinExclusiveTime = DBL_MAX;
	double MaxExclusiveTime = -DBL_MAX;
	double AverageExclusiveTime = 0.0;
	double MedianExclusiveTime = 0.0;
};

struct FTimingProfilerButterflyNode
{
	const FTimingProfilerTimer* Timer = nullptr;
	uint64 Count = 0;
	double InclusiveTime = 0.0;
	double ExclusiveTime = 0.0;
	const FTimingProfilerButterflyNode* Parent = nullptr;
	TArray<FTimingProfilerButterflyNode*> Children;
};

struct FCreateAggregationParams
{
	// whether to sort the created aggregation
	enum class ESortBy
	{
		DontSort,
		TotalInclusiveTime
	};

	enum class ESortOrder
	{
		DontSort,
		Descending,
		Ascending
	};

	// The start timestamp in seconds.
	double IntervalStart = 0.0;

	// The end timestamp in seconds.
	double IntervalEnd = 0.0;

	// A function to filter (by GPU Queue Id) the GPU queues to aggregate.
	TFunction<bool(uint32)> GpuQueueFilter;

	// A boolean value to specify if aggregation should include the old GPU timeline.
	bool bIncludeOldGpu1 = false;

	// A boolean value to specify if aggregation should include the old GPU timeline.
	bool bIncludeOldGpu2 = false;

	// A boolean value to specify if aggregation should include the Verse Sampling timeline.
	bool bIncludeVerseSampling = false;

	// A function to filter (by Thread Id) the CPU threads to aggregate.
	TFunction<bool(uint32)> CpuThreadFilter;

	// A list of additional custom timelines to aggregate.
	TArray<const ITimingProfilerTimeline*> CustomTimelines;

	// Whether to sort the table by a field.
	ESortBy SortBy = ESortBy::DontSort;

	// The sorting order
	ESortOrder SortOrder = ESortOrder::DontSort;

	// Limit the entries in aggregation (e.g. "top 100")
	int TableEntryLimit = 0;

	// The type of frame to use for frame stats aggregation. ETraceFrameType::TraceFrameType_Count means no frame aggregation.
	ETraceFrameType FrameType = ETraceFrameType::TraceFrameType_Count;

	TSharedPtr<TraceServices::FCancellationToken> CancellationToken;
};

struct FCreateButterflyParams
{
	// The start timestamp in seconds.
	double IntervalStart = 0.0;

	// The end timestamp in seconds.
	double IntervalEnd = 0.0;

	// A function to filter the GPU queues to aggregate.
	TFunction<bool(uint32)> GpuQueueFilter;

	// A boolean value to specify if aggregation should include the old GPU timeline.
	bool bIncludeOldGpu1 = false;

	// A boolean value to specify if aggregation should include the old GPU timeline.
	bool bIncludeOldGpu2 = false;

	// A boolean value to specify if aggregation should include the Verse Sampling timeline.
	bool bIncludeVerseSampling = false;

	// A function to filter the CPU threads to aggregate.
	TFunction<bool(uint32)> CpuThreadFilter;

	// A list of additional custom timelines to aggregate.
	TArray<const ITimingProfilerTimeline*> CustomTimelines;
};

class ITimingProfilerButterfly
{
public:
	virtual ~ITimingProfilerButterfly() = default;
	virtual const FTimingProfilerButterflyNode& GenerateCallersTree(uint32 TimerId) = 0;
	virtual const FTimingProfilerButterflyNode& GenerateCalleesTree(uint32 TimerId) = 0;
};

class ITimingProfilerTimerReader
{
public:
	/**
	 * Gets the number of timers.
	 * A valid TimerId is an index in the [0 .. TimerCount-1] range.
	 */
	virtual uint32 GetTimerCount() const = 0;

	/**
	 * Gets the timer info.
	 * The returned pointer is only valid in the current ITimingProfilerTimerReader lock scope.
	 * @param TimerId A valid timer id (index in the [0 .. TimerCount-1] range) or a bit inverted metadata id (index in the metadata table).
	 * @return The timer info.
	 */
	virtual const FTimingProfilerTimer* GetTimer(uint32 TimerId) const = 0;

	virtual uint32 GetOriginalTimerIdFromMetadata(uint32 MetadataTimerId) const { return MetadataTimerId; }

	virtual TArrayView<const uint8> GetMetadata(uint32 MetadataTimerId) const { return TArrayView<const uint8>(); }
};

struct FGpuSignalFence
{
	double Timestamp = 0.0f;
	uint64 Value = 0;
};

struct FGpuWaitFence
{
	double Timestamp = 0.0f;
	uint64 Value = 0;
	uint32 QueueToWaitForId = 0;
};

enum EGpuFenceType : uint8
{
	SignalFence = 0,
	WaitFence = 1,
};

struct FGpuFenceWrapper
{
	EGpuFenceType FenceType;
	TVariant<const FGpuSignalFence*, const FGpuWaitFence*> Fence;
};

struct FGpuQueueInfo
{
	uint32 Id = 0;
	uint8 GPU = 0;
	uint8 Index = 0;
	uint8 Type = 0;
	const TCHAR* Name = nullptr;
	uint32 TimelineIndex = ~0;
	uint32 WorkTimelineIndex = ~0;

	FString GetDisplayName() const
	{
		return FString::Printf(TEXT("GPU%u-%s%u"), GPU, Name, Index);
	}
};

enum class EEnumerateResult
{
	Continue,
	Stop,
};

class ITimingProfilerProvider
	: public IProvider
{
public:
	typedef ITimeline<FTimingProfilerEvent> Timeline;
	typedef TFunctionRef<EEnumerateResult(const FGpuSignalFence&)> EnumerateGpuSignalFencesCallback;
	typedef TFunctionRef<EEnumerateResult(const FGpuWaitFence&)> EnumerateGpuWaitFencesCallback;
	typedef TFunctionRef<EEnumerateResult(const FGpuFenceWrapper&)> EnumerateGpuFencesCallback;
	typedef TFunctionRef<EEnumerateResult(uint32 /*SignalFenceQueueId*/, const FGpuSignalFence& /*SignalFence*/, uint32 /*WaitFenceQueueId*/, const FGpuWaitFence& /*WaitFence*/)> EnumerateResolvedGpuFencesCallback;

	virtual ~ITimingProfilerProvider() = default;

	//////////////////////////////////////////////////
	// Timelines

	virtual bool ReadTimeline(uint32 Index, TFunctionRef<void(const Timeline&)> Callback) const = 0;
	virtual uint32 GetTimelineCount() const = 0;
	virtual void EnumerateTimelines(TFunctionRef<void(const Timeline&)> Callback) const = 0;

	//////////////////////////////////////////////////
	// Timers

	UE_DEPRECATED(5.8, "Use GetTimerReader() instead")
	virtual void ReadTimers(TFunctionRef<void(const ITimingProfilerTimerReader&)> Callback) const {}

	/**
	 * Gets the TimerReader provider.
	 *
	 * @return A reference to the ITimingProfilerTimerReader object.
	 *         The reference is valid for the rest of the session lifetime, but it can be used only under a Session Read Scope.
	 */
	virtual const ITimingProfilerTimerReader& GetTimerReader() const
	{
		// The concrete class that implements ITimingProfilerProvider should override GetTimerReader()!
		// GetTimerReader() is not made abstract for deprecation / backward compatibility reasons.
		ensure(!"Please override GetTimerReader()!");
		const ITimingProfilerTimerReader* TimerReader = nullptr;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReadTimers([&TimerReader](const TraceServices::ITimingProfilerTimerReader& Out) { TimerReader = &Out; });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		check(TimerReader != nullptr);
		return *TimerReader;
	}

	//////////////////////////////////////////////////
	// Metadata

	virtual uint32 GetOriginalTimerIdFromMetadata(uint32 MetadataTimerId) const { return MetadataTimerId; } // also in ITimingProfilerTimerReader
	virtual TArrayView<const uint8> GetMetadata(uint32 MetadataTimerId) const { return TArrayView<const uint8>(); } // also in ITimingProfilerTimerReader

	/**
	 * Gets the metadata spec associated with the MetadataSpecId.
	 *
	 * @param MetadataSpecId The Metadata spec id.
	 * @return A pointer to the FMetadataSpec or nullptr if no spec is associated with the provided Id. The pointer is only valid in the same Session Read Scope as the function call.
	 */
	virtual const FMetadataSpec* GetMetadataSpec(uint32 MetadataSpecId) const { return nullptr; }

	/**
	* Gets the display name for a timer, combining the static name and the format string if present.
	*
	* @param InTimer The timer to create the display name for.
	* @param InOutDisplayName A TStringBuilder where the display name will be created.
	*/
	virtual void MakeTimerDisplayName(const FTimingProfilerTimer& InTimer, TStringBuilder<1024>& InOutDisplayName) const {}

	//////////////////////////////////////////////////
	// GPU

	virtual bool HasGpuTiming() const { return false; }

	// Only used for backward compatibility with old GPU Insights.
	virtual bool GetGpuTimelineIndex(uint32& OutTimelineIndex) const { return false; }
	// Only used for backward compatibility with old GPU Insights.
	virtual bool GetGpu2TimelineIndex(uint32& OutTimelineIndex) const { return false; }

	virtual void EnumerateGpuQueues(TFunctionRef<void(const FGpuQueueInfo&)> Callback) const {}
	virtual bool GetGpuQueueTimelineIndex(uint32 QueueId, uint32& OutTimelineIndex) const { return false; }

	virtual void EnumerateGpuSignalFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuSignalFencesCallback Callback) const {}
	virtual void EnumerateGpuWaitFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuWaitFencesCallback Callback) const {}
	virtual void EnumerateGpuFences(uint32 QueueId, double StartTime, double EndTime, EnumerateGpuFencesCallback Callback) const {}
	virtual void EnumerateResolvedGpuFences(uint32 QueueId, double StartTime, double EndTime, EnumerateResolvedGpuFencesCallback Callback) const {}

	//////////////////////////////////////////////////
	// Verse

	virtual bool HasVerseTiming() const { return false; }

	virtual bool GetVerseTimelineIndex(uint32& OutTimelineIndex) const { return false; }

	//////////////////////////////////////////////////
	// CPU

	virtual bool HasCpuTiming() const { return true; }

	virtual bool GetCpuThreadTimelineIndex(uint32 ThreadId, uint32& OutTimelineIndex) const = 0;

	//////////////////////////////////////////////////
	// Aggregation

	/**
	 * Creates a table of aggregated stats.
	 *
	 * @param Params	The params for the aggregation.
	 */
	virtual ITable<FTimingProfilerAggregatedStats>* CreateAggregation(const FCreateAggregationParams& Params) const { return nullptr; }

	/**
	 * Creates a butterfly aggregation.
	 *
	 * @param Params	The params for the aggregation.
	 */
	virtual ITimingProfilerButterfly* CreateButterfly(const FCreateButterflyParams& Params) const { return nullptr; }

	UE_DEPRECATED(5.6, "Use FCreateButterflyParams instead.")
	virtual ITimingProfilerButterfly* CreateButterfly(double IntervalStart, double IntervalEnd, TFunctionRef<bool(uint32)> CpuThreadFilter, bool bIncludeGpu) const
	{
		FCreateButterflyParams Params;
		Params.IntervalStart = IntervalStart;
		Params.IntervalEnd = IntervalEnd;
		Params.bIncludeOldGpu1 = bIncludeGpu;
		Params.bIncludeOldGpu2 = bIncludeGpu;
		Params.CpuThreadFilter = [CpuThreadFilter](uint32 QueueId) { return CpuThreadFilter(QueueId); };
		return CreateButterfly(Params);
	}

	//////////////////////////////////////////////////
	// Misc

	/**
	 * Gets the Ratio of Threads to use.
	 * @return The ratio scaled between 0-1.
	 */
	virtual double GetRatioOfThreadsToUse() const { return 0.75; }
};

class IEditableTimingProfilerProvider
	: public IEditableProvider
{
public:
	virtual ~IEditableTimingProfilerProvider() = default;

	//////////////////////////////////////////////////
	// Timers

	/**
	 * Adds/registers a new timer.
	 *
	 * @param Type	The type of the new timer.
	 * @return		The identity of the new timer.
	 */
	virtual uint32 AddTimer(ETimingProfilerTimerType Type) { return 0; }

	/**
	 * Adds/registers a new timer.
	 *
	 * @param Type	The type of the new timer.
	 * @param Name	The name attached to the timer.
	 * @return		The identity of the new timer.
	 */
	virtual uint32 AddTimer(ETimingProfilerTimerType Type, FStringView Name)
	{
		const uint32 TimerId = AddTimer(Type);
		if (TimerId != 0)
		{
			SetTimerName(TimerId, Name);
		}
		return TimerId;
	}

	/**
	 * Adds/registers a new timer.
	 *
	 * @param Type	The type of the new timer.
	 * @param Name	The name attached to the timer.
	 * @param File	The source file in which the timer is defined.
	 * @param Line	The line number of the source file in which the timer is defined.
	 * @return		The identity of the new timer.
	 */
	virtual uint32 AddTimer(ETimingProfilerTimerType Type, FStringView Name, const TCHAR* File, uint32 Line)
	{
		const uint32 TimerId = AddTimer(Type);
		if (TimerId != 0)
		{
			SetTimerNameAndLocation(TimerId, Name, File, Line);
		}
		return TimerId;
	}

	/**
	 * Updates an existing timer with information. Some information is unavailable when it's created.
	 *
	 * @param TimerId	The identity of the timer to update.
	 * @param Name		The name attached to the timer.
	 */
	virtual void SetTimerName(uint32 TimerId, FStringView Name) = 0;

	/**
	 * Updates an existing timer with information. Some information is unavailable when it's created.
	 *
	 * @param TimerId	The identity of the timer to update.
	 * @param Name		The name attached to the timer.
	 * @param File		The source file in which the timer is defined.
	 * @param Line		The line number of the source file in which the timer is defined.
	 */
	virtual void SetTimerNameAndLocation(uint32 TimerId, FStringView Name, const TCHAR* File, uint32 Line)
	{
		SetTimerName(TimerId, Name);
		SetTimerLocation(TimerId, File, Line);
	}

	/**
	 * Updates an existing timer with information. Some information is unavailable when it's created.
	 *
	 * @param TimerId	The identity of the timer to update.
	 * @param File		The source file in which the timer is defined.
	 * @param Line		The line number of the source file in which the timer is defined.
	 */
	virtual void SetTimerLocation(uint32 TimerId, const TCHAR* File, uint32 Line) {}

	//////////////////////////////////////////////////
	// Metadata

	/**
	 * Sets the metadata spec for an existing timer.
	 *
	 * @param TimerId			The identity of the timer to update.
	 * @param MetadataSpecId	The metadata spec to associate with the timer.
	 */
	virtual void SetMetadataSpec(uint32 TimerId, uint32 MetadataSpecId) {}

	/**
	 * Adds metadata to a CPU or GPU timer.
	 *
	 * @param OriginalTimerId	The identity of the timer to add metadata to.
	 * @param Metadata			The metadata.
	 *
	 * @return The identity of the metadata.
	 */
	virtual uint32 AddMetadata(uint32 OriginalTimerId, TArray<uint8>&& Metadata) = 0;

	/**
	 * Sets metadata for the specified MetadataTimerId.
	 * The MetadataTimerId must be a value returned by AddMetadata.
	 * The function is meant to be used to replace the metadata at a MetadataTimerId.
	 *
	 * @param MetadataTimerId	The identity of the metadata to replace.
	 * @param Metadata			The metadata.
	 *
	 */
	virtual void SetMetadata(uint32 MetadataTimerId, TArray<uint8>&& Metadata) {}

	/**
	 * Sets metadata for the specified MetadataTimerId and replaces the OriginalTimerId with a new value.
	 * The MetadataTimerId must be a value returned by AddMetadata.
	 * The function is meant to be used to replace the metadata at a MetadataTimerId.
	 *
	 * @param MetadataTimerId	The identity of the metadata to replace.
	 * @param Metadata			The metadata.
	 * @param NewTimerId		The new TimerId. Represents the identity of the timer the metadata is attached to.
	 *
	 */
	virtual void SetMetadata(uint32 MetadataTimerId, TArray<uint8>&& Metadata, uint32 NewTimerId) {}

	/**
	 * Gets metadata by id.
	 *
	 * @param MetadataTimerId	The identity of the metadata.
	 *
	 * @return The metadata.
	 */
	virtual TArrayView<uint8> GetEditableMetadata(uint32 MetadataTimerId) = 0;

	/**
	 * Adds a metadata spec to storage.
	 *
	 * @param Metadata	The metadata spec to add.
	 *
	 * @return			The identity of the metadata spec.
	 */
	virtual uint32 AddMetadataSpec(FMetadataSpec&& Metadata) { return 0; }

	//////////////////////////////////////////////////
	// GPU

	/**
	 * Adds/registers a new GPU timer.
	 *
	 * @param Name	The name attached to the timer.
	 * @param File	The source file in which the timer is defined.
	 * @param Line	The line number of the source file in which the timer is defined.
	 *
	 * @return The identity of the GPU timer.
	 */
	virtual uint32 AddGpuTimer(FStringView Name)
	{
		return AddTimer(ETimingProfilerTimerType::GpuScope, Name);
	}

	/**
	 * Adds a new GPU queue.
	 *
	 * @param QueueId	The GPU queue for which the events are for.
	 */
	virtual void AddGpuQueue(uint32 QueueId, uint8 GPU, uint8 Index, uint8 Type, const TCHAR* Name) {}

	/**
	 * Adds a new GPU signal fence to a queue.
	 *
	 * @param QueueId		The GPU queue that issues the signal.
	 * @param SignalFence	The signal fence.
	 */
	virtual void AddGpuSignalFence(uint32 QueueId, const FGpuSignalFence& SignalFence) {};

	/**
	 * Adds a new GPU wait fence to a queue.
	 *
	 * @param QueueId		The GPU queue that waits.
	 * @param WaitFence		The wait fence.
	 */
	virtual void AddGpuWaitFence(uint32 QueueId, const FGpuWaitFence& WaitFence) {};

	/**
	 * Gets an object to receive ordered GPU timing events for a GPU queue.
	 *
	 * @param QueueId	The queue for which the events are for.
	 *
	 * @return The object to receive the serial GPU timing events for the specified queue.
	 */
	virtual IEditableTimeline<FTimingProfilerEvent>* GetGpuQueueEditableTimeline(uint32 QueueId) { return nullptr; }

	/**
	 * Gets an object to receive ordered GPU Work timing events for a GPU queue.
	 *
	 * @param QueueId	The queue for which the events are for.
	 *
	 * @return The object to receive the serial GPU Work timing events for the specified queue.
	 */
	virtual IEditableTimeline<FTimingProfilerEvent>* GetGpuQueueWorkEditableTimeline(uint32 QueueId) { return nullptr; }

	//////////////////////////////////////////////////
	// Verse Sampling

	/**
	 * Adds/registers a new Verse Sampling timer.
	 *
	 * @param Name	The name attached to the timer.
	 *
	 * @return The identity of the CPU timer.
	 */
	virtual uint32 AddVerseTimer(FStringView Name)
	{
		return AddTimer(ETimingProfilerTimerType::VerseSampling, Name);
	}

	/**
	 * Gets an object to receive ordered Verse timing events for the Verse sampling.
	 *
	 * @return The object to receive the serial Verse timing events for Verse sampling.
	 */
	virtual IEditableTimeline<FTimingProfilerEvent>* GetVerseEditableTimeline() { return nullptr; }

	//////////////////////////////////////////////////
	// CPU

	/**
	 * Adds/registers a new CPU Scope timer.
	 *
	 * @param Name	The name attached to the timer.
	 * @param File	The source file in which the timer is defined.
	 * @param Line	The line number of the source file in which the timer is defined.
	 *
	 * @return The identity of the CPU timer.
	 */
	virtual uint32 AddCpuTimer(FStringView Name, const TCHAR* File, uint32 Line)
	{
		return AddTimer(ETimingProfilerTimerType::CpuScope, Name, File, Line);
	}

	/**
	 * Gets an object to receive ordered CPU timing events for a CPU thread.
	 *
	 * @param ThreadId	The thread for which the events are for.
	 *
	 * @return The object to receive the serial CPU timing events for the specified thread.
	 */
	virtual IEditableTimeline<FTimingProfilerEvent>& GetCpuThreadEditableTimeline(uint32 ThreadId) = 0;

	//////////////////////////////////////////////////
	// Misc

	/**
	 * Sets the Ratio of Threads to be used.
	 * @param InRatioOfThreadsToUse The new ratio of threads to use.
	 */
	virtual void SetRatioOfThreadsToUse(double InRatioOfThreadsToUse) {}

	/**
	 * Gets the read provider.
	 * @returns The read only provider or nullptr if not available.
	 */
	virtual const ITimingProfilerProvider* GetReadProvider() const { return nullptr; }
};

TRACESERVICES_API FName GetTimingProfilerProviderName();
TRACESERVICES_API const ITimingProfilerProvider* ReadTimingProfilerProvider(const IAnalysisSession& Session);
TRACESERVICES_API IEditableTimingProfilerProvider* EditTimingProfilerProvider(IAnalysisSession& Session);

} // namespace TraceServices
