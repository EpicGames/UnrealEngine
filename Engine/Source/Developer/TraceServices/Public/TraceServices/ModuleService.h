// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

namespace TraceServices
{

class IAnalysisSession;

extern TRACESERVICES_API const FName ModuleFeatureName;

struct FModuleInfo
{
	FName Name;
	const TCHAR* DisplayName = nullptr;
};

struct FModuleInfoEx
{
	FModuleInfo Info;
	bool bIsEnabled = false;
};

class IModule
	: public IModularFeature
{
public:
	virtual void GetModuleInfo(FModuleInfo& OutModuleInfo) = 0;
	virtual bool ShouldBeEnabledByDefault() const { return true; }
	virtual void GetLoggers(TArray<const TCHAR*>& OutLoggers) {}
	virtual const TCHAR* GetCommandLineArgument() { return nullptr; }
	virtual void OnAnalysisBegin(IAnalysisSession& Session) = 0;
	virtual void GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) {}
};

class IModuleService
{
public:
	virtual ~IModuleService() = default;
	virtual void GetAvailableModules(TArray<FModuleInfo>& OutModules) = 0;
	virtual void GetAvailableModulesEx(TArray<FModuleInfoEx>& OutModules) = 0;
	virtual void GetEnabledModules(TArray<FModuleInfo>& OutModules) = 0;
	virtual void SetModuleEnabled(const FName& ModuleName, bool bEnabled) = 0;
	virtual void GenerateReports(const IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) = 0;
};

} // namespace TraceServices
