// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"

namespace TraceServices {
namespace FTraceAnalyzerUtils {

template <typename AttachedCharType>
inline FString LegacyAttachmentString(
	const ANSICHAR* FieldName,
	const UE::Trace::IAnalyzer::FOnEventContext& Context)
{
	FString Out;
	if (!Context.EventData.GetString(FieldName, Out))
	{
		Out = FString::ConstructFromPtrSize(
				(const AttachedCharType*)(Context.EventData.GetAttachment()),
				Context.EventData.GetAttachmentSize() / sizeof(AttachedCharType));
	}
	return Out;
}

inline TArrayView<const uint8> LegacyAttachmentArray(
	const ANSICHAR* FieldName,
	const UE::Trace::IAnalyzer::FOnEventContext& Context)
{
	TArrayView<const uint8> Out = Context.EventData.GetArrayView<uint8>(FieldName);
	if (Out.GetData() == nullptr)
	{
		Out = TArrayView<const uint8>(
			Context.EventData.GetAttachment(),
			Context.EventData.GetAttachmentSize());
	}
	return Out;
}

} // namespace FTraceAnalyzerUtils

TRACESERVICES_API void StringFormat(TCHAR* Out, uint64 MaxOut, TCHAR* Temp, uint64 MaxTemp, const TCHAR* FormatString, const uint8* FormatArgs);

} // namespace TraceServices
