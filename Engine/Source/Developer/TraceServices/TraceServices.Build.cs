// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceServices : ModuleRules
{
	public TraceServices(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Cbor",
				"Core",
				"CoreUObject",
				"SymsLib",
				"TraceAnalysis",
			}
		);
	}
}
