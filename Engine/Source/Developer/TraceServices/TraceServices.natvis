<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
 <Type Name="TraceServices::TPagedArray&lt;*,TraceServices::TPagedArrayPage&lt;*&gt;&gt;">
   <DisplayString Condition="TotalItemCount == 0">Empty</DisplayString>
   <DisplayString Condition="TotalItemCount &gt; 0">Num={TotalItemCount}</DisplayString>
   <Expand>
     <CustomListItems MaxItemsPerView="500">
       <Variable Name="PageIndex" InitialValue="0" />
       <Variable Name="IndexInPage" InitialValue="0"/>
       <Size>TotalItemCount</Size>
       <Loop>
         <Break Condition="PageIndex >= PagesArray.ArrayNum" />
         <If Condition="IndexInPage &lt; PagesArray[PageIndex].Count">
           <Item>PagesArray[PageIndex].Items[IndexInPage]</Item>
           <Exec>++IndexInPage</Exec>
         </If>
         <If Condition="IndexInPage &gt;= PagesArray[PageIndex].Count">
           <Exec>IndexInPage = 0</Exec>
           <Exec>++PageIndex</Exec>
         </If>
       </Loop>
     </CustomListItems>
   </Expand>
 </Type>
</AutoVisualizer>
